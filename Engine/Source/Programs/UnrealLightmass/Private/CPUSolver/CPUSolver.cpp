// Copyright Epic Games, Inc. All Rights Reserved.

#include "CPUSolver.h"
#include "LightmassScene.h"
#include "Importer.h"
#include "Exporter.h"
#include "LightmassSwarm.h"
#include "LightingSystem.h"
#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "UnrealLightmass.h"
#include "Misc/OutputDeviceRedirector.h"

namespace Lightmass
{

/** Global statistics */
FGlobalStatistics GStatistics;

/** Global Swarm instance. */
FLightmassSwarm* GSwarm = NULL;

/** Whether we should report detailed stats back to Unreal. */
bool GReportDetailedStats = false;

/** Whether Lightmass is running in debug mode (-debug), using a hardcoded job and not requesting tasks from Swarm. */
bool GDebugMode = false;

/** How many tasks to prefetch per worker thread. */
float GNumTasksPerThreadPrefetch = 1.0f;

/** Report statistics back to Unreal. */
void ReportStatistics()
{
	if ( GReportDetailedStats )
	{
		double RequestTime = GStatistics.ThreadStatistics.RequestTime + GStatistics.ThreadStatistics.RequestTimeoutTime;
		double TrackedTime = RequestTime +
			GStatistics.ThreadStatistics.TextureMappingTime + GStatistics.ThreadStatistics.ExportTime;
		double UnTrackedTime = GStatistics.ThreadStatistics.TotalTime - TrackedTime;

		// Send back detailed information to the Unreal log.
		GSwarm->SendTextMessage(
			TEXT("Lightmass on %s: %s total, %s importing, %s setup, %s photons, %s processing, %s extra exporting [%d/%d mappings].\n")
			TEXT("  Threads: %d threads, %.0f total thread seconds (out of %.0f available)\n")
			TEXT("  - %6.2f%% %7.1fs   Requesting tasks\n")
			TEXT("  ---> %6.2f%% %7.1fs   Requesting tasks from Swarm\n")
			TEXT("  - %6.2f%% %7.1fs   Processing texture mappings\n")
			TEXT("  - %6.2f%% %7.1fs   Exporting %d mappings\n")
			TEXT("  - %6.2f%% %7.1fs   Untracked thread time\n")
			TEXT("\n")
			, FPlatformProcess::ComputerName()
			, *FPlatformTime::PrettyTime(GStatistics.TotalTimeEnd - GStatistics.TotalTimeStart)
			, *FPlatformTime::PrettyTime(GStatistics.ImportTimeEnd - GStatistics.ImportTimeStart)
			, *FPlatformTime::PrettyTime(GStatistics.SceneSetupTime)
			, *FPlatformTime::PrettyTime(GStatistics.PhotonsEnd - GStatistics.PhotonsStart)
			, *FPlatformTime::PrettyTime(GStatistics.WorkTimeEnd - GStatistics.WorkTimeStart)
			, *FPlatformTime::PrettyTime(GStatistics.ExtraExportTime)
			, GStatistics.ThreadStatistics.NumTextureMappings
			, GStatistics.NumTotalMappings
			, GStatistics.NumThreads
			, GStatistics.ThreadStatistics.TotalTime
			, (GStatistics.WorkTimeEnd - GStatistics.WorkTimeStart) * GStatistics.NumThreads
			, RequestTime / GStatistics.ThreadStatistics.TotalTime * 100.0
			, RequestTime
			, GStatistics.ThreadStatistics.SwarmRequestTime / GStatistics.ThreadStatistics.TotalTime * 100.0
			, GStatistics.ThreadStatistics.SwarmRequestTime
			, GStatistics.ThreadStatistics.TextureMappingTime / GStatistics.ThreadStatistics.TotalTime * 100.0
			, GStatistics.ThreadStatistics.TextureMappingTime
			, GStatistics.ThreadStatistics.ExportTime / GStatistics.ThreadStatistics.TotalTime * 100.0
			, GStatistics.ThreadStatistics.ExportTime
			, GStatistics.NumExportedMappings
			, UnTrackedTime / GStatistics.ThreadStatistics.TotalTime * 100.0
			, UnTrackedTime
			);

		GSwarm->SendTextMessage(
			TEXT("  Read amount: %3.2fMB (%.3f sec, %d calls)\n")
			TEXT("  Write amount: %3.2fMB (%.3f sec, %d calls)\n")
			, (double)GSwarm->GetTotalBytesRead() / 1000.0 / 1000.0
			, GSwarm->GetTotalSecondsRead()
			, GSwarm->GetTotalNumReads()
			, (double)GSwarm->GetTotalBytesWritten() / 1000.0 / 1000.0
			, GSwarm->GetTotalSecondsWritten()
			, GSwarm->GetTotalNumWrites()
			);

		if ( GDebugMode == false )
		{
			UE_LOGF(LogLightmass, Log,  "Time in SendMessage() = %ls", *FPlatformTime::PrettyTime(GStatistics.SendMessageTime) );
			UE_LOGF(LogLightmass, Log,  "Task request roundtrip = %ls", *FPlatformTime::PrettyTime(FTiming::GetAverageTiming()) );
		}
	}
	else
	{
		// Send back timing information to the Unreal log.
		GSwarm->SendTextMessage(
			TEXT("Lightmass on %s: %s total, %s importing, %s setup, %s photons, %s processing, %s extra exporting [%d/%d mappings]. Threads: %s total, %s processing.")
			, FPlatformProcess::ComputerName()
			, *FPlatformTime::PrettyTime(GStatistics.TotalTimeEnd - GStatistics.TotalTimeStart)
			, *FPlatformTime::PrettyTime(GStatistics.ImportTimeEnd - GStatistics.ImportTimeStart)
			, *FPlatformTime::PrettyTime(GStatistics.SceneSetupTime)
			, *FPlatformTime::PrettyTime(GStatistics.PhotonsEnd - GStatistics.PhotonsStart)
			, *FPlatformTime::PrettyTime(GStatistics.WorkTimeEnd - GStatistics.WorkTimeStart)
			, *FPlatformTime::PrettyTime(GStatistics.ExtraExportTime)
			, GStatistics.ThreadStatistics.NumTextureMappings
			, GStatistics.NumTotalMappings
			, *FPlatformTime::PrettyTime(GStatistics.ThreadStatistics.TotalTime)
			, *FPlatformTime::PrettyTime(GStatistics.ThreadStatistics.TextureMappingTime)
			);
	}
}

/** Transfers back the current log file to the instigator. */
void ReportLogFile()
{
	GLog->Flush();

	// Get the log filename and replace ".log" with "_Result.log"
	FString LogFilename = FLightmassLog::Get()->GetLogFilename();
	FString ChannelName = LogFilename.Left( LogFilename.Len() - 4 ) + TEXT("_Result.log");
	FArchive* File = IFileManager::Get().CreateFileReader(*LogFilename);
	bool bIsOk = false;
	if ( File != NULL )
	{
		int64 Filesize = File->TotalSize();

		int32 Channel = GSwarm->OpenChannel( *ChannelName, NSwarm::SWARM_JOB_CHANNEL_WRITE, true );
		if ( Channel >= 0 )
		{
			static char Buffer[4096];
			bIsOk = true;
			while ( Filesize > 0 )
			{
				int64 NumBytesToRead = FMath::Min( Filesize, 4096LL );
				File->Serialize( Buffer, NumBytesToRead );
				GSwarm->Write( Buffer, NumBytesToRead );
				Filesize -= NumBytesToRead;
			}
			GSwarm->PopChannel( true );
		}
		delete File;
	}
	if ( !bIsOk )
	{
		UE_LOGF(LogLightmass, Log, "Failed to send back log file through Swarm!");
	}
}

/**
 * Entry point for starting the static lighting process
 * 
 * @param SceneGuid		Guid of the scene to process
 * @param NumThreads	Number of concurrent threads to use for lighting building
 * @param bDumpTextures	If true, 2d lightmaps will be dumped too 
 */
void BuildStaticLighting(const FGuid& SceneGuid, int32 NumThreads, bool bDumpTextures)
{
	// Place a marker in the memory profile data.
	UE::Private::GMalloc->Exec(NULL, TEXT("SNAPSHOTMEMORY"), *GLog);

	UE_LOGF(LogLightmass, Log, "Building static lighting...");

	// time it
	double SetupTimeStart = FPlatformTime::Seconds();

	// Start initializing GCPUFrequency.
	StartInitCPUFrequency();

	// Startup Swarm.
	GStatistics.ImportTimeStart = FPlatformTime::Seconds();
	NSwarm::FSwarmInterface::Initialize(*(FString(FPlatformProcess::BaseDir()) + TEXT("..\\DotNET\\SwarmInterface.dll")));
	check(&NSwarm::FSwarmInterface::Get() != NULL);
	GSwarm = new FLightmassSwarm( NSwarm::FSwarmInterface::Get(), SceneGuid, FMath::TruncToInt(GNumTasksPerThreadPrefetch*NumThreads) );
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_BeginJob, -1 ) );

	FLightmassImporter Importer( GSwarm );
	FScene Scene;
	if( !Importer.ImportScene( Scene, SceneGuid ) )
	{
		UE_LOGF(LogLightmass, Log, "Failed to import scene file");
		exit( 1 );
	}
	GStatistics.ImportTimeEnd = FPlatformTime::Seconds();

	// Finish initializing GCPUFrequency.
	FinishInitCPUFrequency();

	// setup the desired lighting options
	FLightingBuildOptions LightingOptions;

	FLightmassSolverExporter Exporter( GSwarm, Scene, bDumpTextures );


	// Place a marker in the memory profile data.
	UE::Private::GMalloc->Exec(NULL, TEXT("SNAPSHOTMEMORY"), *GLog);

	double LightTimeStart = FPlatformTime::Seconds();

	// Create the global lighting system to kick off the processing
	FStaticLightingSystem LightingSystem(LightingOptions, Scene, Exporter, NumThreads );	

	GStatistics.TotalTimeEnd = FPlatformTime::Seconds();

	// Place a marker in the memory profile data.
	UE::Private::GMalloc->Exec(NULL, TEXT("SNAPSHOTMEMORY"), *GLog);

	// Report back statistics over Swarm.
	ReportStatistics();

	double EndTime = FPlatformTime::Seconds();

	UE_LOGF(LogLightmass, Log, "Lighting complete [Startup = %ls, Lighting = %ls]", *FPlatformTime::PrettyTime(LightTimeStart - SetupTimeStart), *FPlatformTime::PrettyTime(EndTime - LightTimeStart));

	if (GReportDetailedStats)
	{
		LightingSystem.GetAggregateMesh().DumpPostBuildStats();
	}

	// Transfer back the log to the instigator.
	ReportLogFile();

	// Shutdown Swarm
	FLightmassSwarm* Swarm = GSwarm;
	GSwarm = NULL;
	delete Swarm;
	
	// Write out memory profiling data to the .mprof file.
	UE::Private::GMalloc->Exec(NULL, TEXT("DUMPALLOCSTOFILE"), *GLog);
}

} //namespace Lightmass
