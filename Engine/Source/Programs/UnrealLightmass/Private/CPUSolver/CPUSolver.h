// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

namespace Lightmass
{

/**
 * Entry point for starting the static lighting process
 * 
 * @param SceneGuid		Guid of the scene to process
 * @param NumThreads	Number of concurrent threads to use for lighting building
 * @param bDumpTextures	If true, 2d lightmaps will be dumped to 
 */
void BuildStaticLighting(const FGuid& SceneGuid, int32 NumThreads, bool bDumpTextures);

/** Helper struct that contain global statistics for the Lightmass thread execution. */
struct FThreadStatistics
{
	FThreadStatistics()
	:	TotalTime(0.0)
	,	RequestTime(0.0)
	,	ExportTime(0.0)
	,	TextureMappingTime(0.0)
	,	RequestTimeoutTime(0.0)
	,	SwarmRequestTime(0.0)
	,	NumTextureMappings(0)
	{
	}

	void operator+=( const FThreadStatistics& Other )
	{
		TotalTime += Other.TotalTime;
		RequestTime += Other.RequestTime;
		ExportTime += Other.ExportTime;
		TextureMappingTime += Other.TextureMappingTime;
		RequestTimeoutTime += Other.RequestTimeoutTime;
		SwarmRequestTime += Other.SwarmRequestTime;
		NumTextureMappings += Other.NumTextureMappings;
	}

	double	TotalTime;
	double	RequestTime;
	double	ExportTime;
	double	TextureMappingTime;
	double	RequestTimeoutTime;
	double	SwarmRequestTime;
	int32		NumTextureMappings;
};

/** Helper struct that contain global statistics for the Lightmass execution. */
struct FGlobalStatistics
{
	FGlobalStatistics()
	:	NumThreads(0)
	,	NumThreadsFinished(0)
	,	NumTotalMappings(0)
	,	NumExportedMappings(0)
	,	TotalTimeStart(0.0)
	,	TotalTimeEnd(0.0)
	,	ImportTimeStart(0.0)
	,	ImportTimeEnd(0.0)
	,	PhotonsStart(0.0)
	,	PhotonsEnd(0.0)
	,	WorkTimeStart(0.0)
	,	WorkTimeEnd(0.0)
	,	ExtraExportTime(0.0)
	,	SendMessageTime(0.0)
	,	SceneSetupTime(0.0)
	{
	}
	FThreadStatistics	ThreadStatistics;
	int32					NumThreads;
	volatile int32		NumThreadsFinished;		// Incremented by each thread when they finish.
	int32					NumTotalMappings;
	int32					NumExportedMappings;	// Only incremented while threads are still running.
	double				TotalTimeStart;
	double				TotalTimeEnd;
	double				ImportTimeStart;
	double				ImportTimeEnd;
	double				PhotonsStart;
	double				PhotonsEnd;
	double				WorkTimeStart;
	double				WorkTimeEnd;
	double				ExtraExportTime;
	/** Time spent in SendMessage(), in seconds. */
	double				SendMessageTime;
	/** Time spent setting up the scene, in seconds. */
	double				SceneSetupTime;
};

/** Global statistics */
extern FGlobalStatistics GStatistics;

/** Global Swarm instance. */
extern class FLightmassSwarm* GSwarm;

/** Whether we should report detailed stats back to Unreal. */
extern bool GReportDetailedStats;

/** 
 * Whether Lightmass is running in debug mode (-debug), using a hardcoded job and not requesting tasks from Swarm. 
 * Warning!  This will only process mapping tasks and will skip other types of tasks.
 */
extern bool GDebugMode;

} //namespace Lightmass
