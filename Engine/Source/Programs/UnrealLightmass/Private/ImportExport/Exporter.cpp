// Copyright Epic Games, Inc. All Rights Reserved.

#include "Exporter.h"
#include "LightmassSwarm.h"
#include "LightingSystem.h"
#include "UnrealLightmass.h"
#include "HAL/FileManager.h"

namespace Lightmass
{

	FLightmassSolverExporter::FLightmassSolverExporter( FLightmassSwarm* InSwarm, const FScene& InScene, bool bInDumpTextures )
	:	Swarm( InSwarm )
	,	Scene( InScene )
	,	bDumpTextures( bInDumpTextures )
	{
	}

	FLightmassSolverExporter::~FLightmassSolverExporter()
	{
	}

	FLightmassSwarm* FLightmassSolverExporter::GetSwarm()
	{
		return Swarm;
	}

	void FLightmassSolverExporter::EndExportResults() const
	{
		Swarm->CloseCurrentChannel();
	}

	/** Exports volume lighting samples to Unreal. */
	void FLightmassSolverExporter::ExportVolumeLightingSamples(
		bool bExportVolumeLightingDebugOutput,
		const FVolumeLightingDebugOutput& DebugOutput,
		const FVector4f& VolumeCenter, 
		const FVector4f& VolumeExtent, 
		const TMap<FGuid,TArray<FVolumeLightingSample> >& VolumeSamples) const
	{
		if (bExportVolumeLightingDebugOutput)
		{
			const FString ChannelName = CreateChannelName(VolumeLightingDebugOutputGuid, LM_VOLUMEDEBUGOUTPUT_VERSION, LM_VOLUMEDEBUGOUTPUT_EXTENSION);
			const int32 ErrorCode = Swarm->OpenChannel(*ChannelName, LM_VOLUMEDEBUGOUTPUT_CHANNEL_FLAGS, true);
			if( ErrorCode >= 0 )
			{
				WriteArray(DebugOutput.VolumeLightingSamples);
				Swarm->CloseCurrentChannel();
			}
			else
			{
				UE_LOGF(LogLightmass, Log, "Failed to open volume sample debug output channel!");
			}
		}

		const FString ChannelName = CreateChannelName(PrecomputedVolumeLightingGuid, LM_VOLUMESAMPLES_VERSION, LM_VOLUMESAMPLES_EXTENSION);
		const int32 ErrorCode = Swarm->OpenChannel(*ChannelName, LM_VOLUMESAMPLES_CHANNEL_FLAGS, true);
		if( ErrorCode >= 0 )
		{
			Swarm->Write(&VolumeCenter, sizeof(VolumeCenter));
			Swarm->Write(&VolumeExtent, sizeof(VolumeExtent));
			const int32 NumVolumeSampleArrays = VolumeSamples.Num();
			Swarm->Write(&NumVolumeSampleArrays, sizeof(NumVolumeSampleArrays));
			for (TMap<FGuid,TArray<FVolumeLightingSample> >::TConstIterator It(VolumeSamples); It; ++It)
			{
				Swarm->Write(&It.Key(), sizeof(It.Key()));
				static_assert(sizeof(FVolumeLightingSample) == sizeof(FVolumeLightingSampleData), "Volume derived size must match.");
				WriteArray(It.Value());
			}
			Swarm->CloseCurrentChannel();
		}
		else
		{
			UE_LOGF(LogLightmass, Log, "Failed to open volume samples channel!");
		}
	}

	/** Exports dominant shadow information to Unreal. */
	void FLightmassSolverExporter::ExportStaticShadowDepthMap(const FGuid& LightGuid, const FStaticShadowDepthMap& StaticShadowDepthMap) const
	{
		const FString ChannelName = CreateChannelName(LightGuid, LM_DOMINANTSHADOW_VERSION, LM_DOMINANTSHADOW_EXTENSION);
		const int32 ErrorCode = Swarm->OpenChannel(*ChannelName, LM_DOMINANTSHADOW_CHANNEL_FLAGS, true);
		if( ErrorCode >= 0 )
		{
			Swarm->Write(&StaticShadowDepthMap, sizeof(FStaticShadowDepthMapData));
			static_assert(sizeof(FStaticShadowDepthMapSample) == sizeof(FStaticShadowDepthMapSampleData), "ShadowDerivedSizeMustMatch");
			WriteArray(StaticShadowDepthMap.ShadowMap);
			Swarm->CloseCurrentChannel();
		}
		else
		{
			UE_LOGF(LogLightmass, Log, "Failed to open static shadow depth map channel!");
		}
	}

	/** 
	 * Exports information about mesh area lights back to Unreal, 
	 * So that UE5 can create dynamic lights to approximate the mesh area light's influence on dynamic objects.
	 */
	void FLightmassSolverExporter::ExportMeshAreaLightData(const TIndirectArray<FMeshAreaLight>& MeshAreaLights, float MeshAreaLightGeneratedDynamicLightSurfaceOffset) const
	{
		const FString ChannelName = CreateChannelName(MeshAreaLightDataGuid, LM_MESHAREALIGHTDATA_VERSION, LM_MESHAREALIGHTDATA_EXTENSION);
		const int32 ErrorCode = Swarm->OpenChannel(*ChannelName, LM_MESHAREALIGHT_CHANNEL_FLAGS, true);
		if( ErrorCode >= 0 )
		{
			int32 NumMeshAreaLights = MeshAreaLights.Num();
			Swarm->Write(&NumMeshAreaLights, sizeof(NumMeshAreaLights));

			for (int32 LightIndex = 0; LightIndex < MeshAreaLights.Num(); LightIndex++)
			{
				const FMeshAreaLight& CurrentLight = MeshAreaLights[LightIndex];
				FMeshAreaLightData LightData;
				LightData.LevelGuid = CurrentLight.LevelGuid;

				FVector4f AverageNormal(0,0,0);
				for (int32 PrimitiveIndex = 0; PrimitiveIndex < CurrentLight.Primitives.Num(); PrimitiveIndex++)
				{
					AverageNormal += CurrentLight.Primitives[PrimitiveIndex].SurfaceNormal * CurrentLight.Primitives[PrimitiveIndex].SurfaceArea;
				}
				if (AverageNormal.SizeSquared3() > KINDA_SMALL_NUMBER)
				{
					AverageNormal = AverageNormal.GetUnsafeNormal3();
				}
				else
				{
					AverageNormal = FVector4f(1,0,0);
				}
				// Offset the position somewhat to reduce the chance of the generated light being inside the mesh
				LightData.Position = CurrentLight.Position + AverageNormal *MeshAreaLightGeneratedDynamicLightSurfaceOffset;
				// Use the average normal for the generated light's direction
				LightData.Direction = AverageNormal;
				LightData.Radius = CurrentLight.InfluenceRadius;
				// Approximate the mesh area light's cosine lobe falloff using a Unreal spotlight's cone angle falloff
				LightData.ConeAngle = PI / 2.0f;
				FLinearColor LightIntensity = CurrentLight.TotalPower / CurrentLight.TotalSurfaceArea;
				// Extract an LDR light color and brightness scale
				float MaxComponent = FMath::Max(LightIntensity.R, FMath::Max(LightIntensity.G, LightIntensity.B));
				LightData.Color = ( LightIntensity / FMath::Max(MaxComponent, (float)KINDA_SMALL_NUMBER) ).ToFColor(true);
				LightData.Brightness = MaxComponent;
				LightData.FalloffExponent = CurrentLight.FalloffExponent;
				Swarm->Write(&LightData, sizeof(LightData));
			}
			
			Swarm->CloseCurrentChannel();
		}
		else
		{
			UE_LOGF(LogLightmass, Log, "Failed to open dominant shadow channel!");
		}
	}

	/** Exports the volume distance field. */
	void FLightmassSolverExporter::ExportVolumeDistanceField(int32 VolumeSizeX, int32 VolumeSizeY, int32 VolumeSizeZ, float VolumeMaxDistance, const FBox3f& DistanceFieldVolumeBounds, const TArray<FColor>& VolumeDistanceField) const
	{
		const FString ChannelName = CreateChannelName(VolumeDistanceFieldGuid, LM_MESHAREALIGHTDATA_VERSION, LM_MESHAREALIGHTDATA_EXTENSION);
		const int32 ErrorCode = Swarm->OpenChannel(*ChannelName, LM_MESHAREALIGHT_CHANNEL_FLAGS, true);
		if( ErrorCode >= 0 )
		{
			Swarm->Write(&VolumeSizeX, sizeof(VolumeSizeX));
			Swarm->Write(&VolumeSizeY, sizeof(VolumeSizeY));
			Swarm->Write(&VolumeSizeZ, sizeof(VolumeSizeZ));
			Swarm->Write(&VolumeMaxDistance, sizeof(VolumeMaxDistance));
			Swarm->Write(&DistanceFieldVolumeBounds.Min, sizeof(DistanceFieldVolumeBounds.Min));
			Swarm->Write(&DistanceFieldVolumeBounds.Max, sizeof(DistanceFieldVolumeBounds.Max));

			WriteArray(VolumeDistanceField);

			Swarm->CloseCurrentChannel();
		}
		else
		{
			UE_LOGF(LogLightmass, Log, "Failed to open dominant shadow channel!");
		}
	}

	void FLightmassSolverExporter::WriteDebugLightingOutput(const FDebugLightingOutput& DebugOutput) const
	{
		Swarm->Write(&DebugOutput.bValid, sizeof(DebugOutput.bValid));

		if (DebugOutput.bValid)
		{
			WriteArray(DebugOutput.PathRays);
			WriteArray(DebugOutput.ShadowRays);
			WriteArray(DebugOutput.IndirectPhotonPaths);
			WriteArray(DebugOutput.SelectedVertexIndices);
			WriteArray(DebugOutput.Vertices);
			WriteArray(DebugOutput.CacheRecords);
			WriteArray(DebugOutput.DirectPhotons);
			WriteArray(DebugOutput.IndirectPhotons);
			WriteArray(DebugOutput.IrradiancePhotons);
			WriteArray(DebugOutput.GatheredPhotons);
			WriteArray(DebugOutput.GatheredImportancePhotons);
			WriteArray(DebugOutput.GatheredPhotonNodes);
			Swarm->Write(&DebugOutput.bDirectPhotonValid, sizeof(DebugOutput.bDirectPhotonValid));
			Swarm->Write(&DebugOutput.GatheredDirectPhoton, sizeof(DebugOutput.GatheredDirectPhoton));
			Swarm->Write(&DebugOutput.TexelCorners, sizeof(DebugOutput.TexelCorners));
			Swarm->Write(&DebugOutput.bCornerValid, sizeof(DebugOutput.bCornerValid));
			Swarm->Write(&DebugOutput.SampleRadius, sizeof(DebugOutput.SampleRadius));
		}
	}

	/** Writes a TArray to the channel on the top of the Swarm stack. */
	template<class T>
	void FLightmassSolverExporter::WriteArray(const TArray<T>& Array) const
	{
		const int32 ArrayNum = Array.Num();
		Swarm->Write((void*)&ArrayNum, sizeof(ArrayNum));
		if (ArrayNum > 0)
		{
			Swarm->Write(Array.GetData(), Array.GetTypeSize() * ArrayNum);
		}
	}

	/**
	 * Write out bitmap files for a texture map
	 * @param BitmapBaseName Base file name for the bitmap (will have info appended for multiple components)
	 * @param Samples Texture map sample data
	 * @param Width Width of the texture map
	 * @param Height Height of the texture map
	 */
	template<int32 NumComponents, typename SampleType>
	void WriteBitmap(const TCHAR* BitmapBaseName, SampleType* Samples, int32 Width, int32 Height)
	{
		// Without any data, just return
		if( Samples == NULL )
		{
			return;
		}

		// look to make sure that any texel has been mapped
		bool bTextureIsMapped = false;

		for(int32 Y = 0; Y < Height; Y++)
		{
			for(int32 X = 0; X < Width; X++)
			{
				const SampleType& Sample = Samples[Y * Width + X];
				if (Sample.bIsMapped)
				{
					bTextureIsMapped = true;
					break;
				}
			}
		}

		// if it's all black, just punt out
		if (!bTextureIsMapped)
		{
			return;
		}

#pragma pack (push,1)
		struct BITMAPFILEHEADER
		{
			uint16	bfType GCC_PACK(1);
			uint32	bfSize GCC_PACK(1);
			uint16	bfReserved1 GCC_PACK(1); 
			uint16	bfReserved2 GCC_PACK(1);
			uint32	bfOffBits GCC_PACK(1);
		} FH; 
		struct BITMAPINFOHEADER
		{
			uint32	biSize GCC_PACK(1); 
			int32		biWidth GCC_PACK(1);
			int32		biHeight GCC_PACK(1);
			uint16	biPlanes GCC_PACK(1);
			uint16	biBitCount GCC_PACK(1);
			uint32	biCompression GCC_PACK(1);
			uint32	biSizeImage GCC_PACK(1);
			int32		biXPelsPerMeter GCC_PACK(1); 
			int32		biYPelsPerMeter GCC_PACK(1);
			uint32	biClrUsed GCC_PACK(1);
			uint32	biClrImportant GCC_PACK(1); 
		} IH;
#pragma pack (pop)

		int32 BytesPerLine = Align(Width * 3,4);

		FArchive* Files[NumComponents];
		for (int32 CompIndex = 0; CompIndex < NumComponents; CompIndex++)
		{
			if (NumComponents == 1)
			{
				Files[CompIndex] = IFileManager::Get().CreateFileWriter(*FString::Printf(TEXT("%s.bmp"), BitmapBaseName));
			}
			else
			{
				Files[CompIndex] = IFileManager::Get().CreateFileWriter(*FString::Printf(TEXT("%s_Dir%d.bmp"), BitmapBaseName, CompIndex));
			}
		}

		// File header.
		FH.bfType       		= (uint16) ('B' + 256*'M');
		FH.bfSize       		= (uint32) (sizeof(BITMAPFILEHEADER) + sizeof(BITMAPINFOHEADER) + BytesPerLine * Height);
		FH.bfReserved1  		= (uint16) 0;
		FH.bfReserved2  		= (uint16) 0;
		FH.bfOffBits    		= (uint32) (sizeof(BITMAPFILEHEADER) + sizeof(BITMAPINFOHEADER));
		for (int32 CompIndex = 0; CompIndex < NumComponents; CompIndex++)
		{
			Files[CompIndex]->Serialize( &FH, sizeof(FH) );
		}

		// Info header.
		IH.biSize               = (uint32) sizeof(BITMAPINFOHEADER);
		IH.biWidth              = (uint32) Width;
		IH.biHeight             = (uint32) Height;
		IH.biPlanes             = (uint16) 1;
		IH.biBitCount           = (uint16) 24;
		IH.biCompression        = (uint32) 0; //BI_RGB
		IH.biSizeImage          = (uint32) BytesPerLine * Height;
		IH.biXPelsPerMeter      = (uint32) 0;
		IH.biYPelsPerMeter      = (uint32) 0;
		IH.biClrUsed            = (uint32) 0;
		IH.biClrImportant       = (uint32) 0;
		for (int32 CompIndex = 0; CompIndex < NumComponents; CompIndex++)
		{
			Files[CompIndex]->Serialize( &IH, sizeof(IH) );
		}

		// write out the image, bottom up
		for (int32 Y = Height - 1; Y >= 0; Y--)
		{
			for (int32 X = 0; X < Width; X++)
			{
				const SampleType& Sample = Samples[Y * Width + X];
				for (int32 CompIndex = 0; CompIndex < NumComponents; CompIndex++)
				{
					FColor Color = Sample.GetColor(CompIndex);
					Files[CompIndex]->Serialize( &Color, 3 );
				}
			}

			// pad if necessary
			static uint8 Zero[3] = {0, 0, 0};
			if (Width * 3 < BytesPerLine)
			{
				for (int32 CompIndex = 0; CompIndex < NumComponents; CompIndex++)
				{
					Files[CompIndex]->Serialize( &Zero, BytesPerLine - Width * 3 );
				}
			}
		}

		// close the open files
		for (int32 CompIndex = 0; CompIndex < NumComponents; CompIndex++)
		{
			delete Files[CompIndex];
		}
	}

	int32 FLightmassSolverExporter::BeginExportResults(struct FTextureMappingStaticLightingData& LightingData, uint32 NumMappings) const
	{
		const FString ChannelName = CreateChannelName(LightingData.Mapping->Guid, LM_TEXTUREMAPPING_VERSION, LM_TEXTUREMAPPING_EXTENSION);
		const int32 ErrorCode = Swarm->OpenChannel(*ChannelName, LM_TEXTUREMAPPING_CHANNEL_FLAGS, true);
		if( ErrorCode < 0 )
		{
			UE_LOGF(LogLightmass, Log, "Failed to open texture mapping channel %ls!", *ChannelName);
		}
		else
		{
			// Write out the number of mappings this channel will contain
			Swarm->Write(&NumMappings, sizeof(NumMappings));
		}
		return ErrorCode;
	}

	/**
	 * Send complete lighting data to Unreal
	 *
	 * @param LightingData - Object containing the computed data
	 */
	void FLightmassSolverExporter::ExportResults( FTextureMappingStaticLightingData& LightingData, bool bUseUniqueChannel ) const
	{
		UE_LOGF(LogLightmass, Verbose, "Exporting texture lighting %ls [%.3fs]", *(LightingData.Mapping->Guid.ToString()), LightingData.ExecutionTime);

		// If requested, use a unique channel for this mapping, otherwise, just use the one that is already open
		if (bUseUniqueChannel)
		{
			if (BeginExportResults(LightingData, 1) < 0)
			{
				return;
			}
		}

		const int32 PaddedOffset = LightingData.Mapping->bPadded ? 1 : 0;
		const int32 DebugSampleIndex = LightingData.Mapping == Scene.DebugMapping
			? (Scene.DebugInput.LocalY + PaddedOffset) * LightingData.Mapping->SizeX + Scene.DebugInput.LocalX + PaddedOffset
			: INDEX_NONE;

		if (bDumpTextures)
		{
			WriteBitmap<4>(*(LightingData.Mapping->Guid.ToString() + TEXT("_LM")), LightingData.LightMapData->GetData(), LightingData.LightMapData->GetSizeX(), LightingData.LightMapData->GetSizeY());
		}

		// If we need to compress the data before writing out, do it now
		LightingData.LightMapData->Compress(DebugSampleIndex);
//		UE_LOGF(LogLightmass, Log, "LM data went from %d to %d bytes", LightingData.LightMapData->UncompressedDataSize, LightingData.LightMapData->CompressedDataSize);

		const int32 ShadowMapCount = LightingData.ShadowMaps.Num();
		const int32 SignedDistanceFieldShadowMapCount = LightingData.SignedDistanceFieldShadowMaps.Num();
		const int32 NumLights = LightingData.LightMapData->Lights.Num();
	
#pragma pack (push,1)
		struct FTextureHeader
		{
			FTextureHeader(FGuid& InGuid, double InExecutionTime, FLightMapData2DData& InData, int32 InShadowMapCount, int32 InSignedDistanceFieldShadowMapCount, int32 InLightCount)
				: Guid(InGuid), ExecutionTime(InExecutionTime), Data(InData), ShadowMapCount(InShadowMapCount), SignedDistanceFieldShadowMapCount(InSignedDistanceFieldShadowMapCount), LightCount(InLightCount)
			{}
			FGuid Guid;
			double ExecutionTime;
			FLightMapData2DData Data;
			int32 ShadowMapCount;
			int32 SignedDistanceFieldShadowMapCount;
			int32 LightCount;
		};
#pragma pack (pop)
		FTextureHeader Header(LightingData.Mapping->Guid, LightingData.ExecutionTime, *(FLightMapData2DData*)LightingData.LightMapData, LightingData.ShadowMaps.Num(), LightingData.SignedDistanceFieldShadowMaps.Num(), NumLights);
		Swarm->Write(&Header, sizeof(Header));

		for (int32 LightIndex = 0; LightIndex < NumLights; LightIndex++)
		{
			FGuid CurrentGuid = LightingData.LightMapData->Lights[LightIndex]->Guid;
			Swarm->Write(&CurrentGuid, sizeof(CurrentGuid));
		}

		// Write out compressed data if supported
		Swarm->Write(LightingData.LightMapData->GetCompressedData(), LightingData.LightMapData->CompressedDataSize ? LightingData.LightMapData->CompressedDataSize : LightingData.LightMapData->UncompressedDataSize);

		// The resulting light GUID --> shadow map data
		int32 ShadowIndex = 0;

		for (TMap<const FLight*,FSignedDistanceFieldShadowMapData2D*>::TIterator It(LightingData.SignedDistanceFieldShadowMaps); It; ++It, ++ShadowIndex)
		{
			FGuid OutGuid = It.Key()->Guid;
			FSignedDistanceFieldShadowMapData2D* OutData = It.Value();

			// If we need to compress the data before writing out, do it now
			OutData->Compress(INDEX_NONE);

			Swarm->Write(&OutGuid, sizeof(FGuid));
			Swarm->Write((FSignedDistanceFieldShadowMapData2DData*)OutData, sizeof(FSignedDistanceFieldShadowMapData2DData));

			// Write out compressed data if supported
			Swarm->Write(OutData->GetCompressedData(), OutData->CompressedDataSize ? OutData->CompressedDataSize : OutData->UncompressedDataSize);
		}

		WriteDebugLightingOutput(LightingData.DebugOutput);

		// free up the calculated data
		delete LightingData.LightMapData;
		LightingData.ShadowMaps.Empty();
		LightingData.SignedDistanceFieldShadowMaps.Empty();

		// Only close the channel if we opened it
		if (bUseUniqueChannel)
		{
			EndExportResults();
		}
	}

	void FLightmassSolverExporter::ExportResults(const FPrecomputedVisibilityData& TaskData) const
	{
		const FString ChannelName = CreateChannelName(TaskData.Guid, LM_PRECOMPUTEDVISIBILITY_VERSION, LM_PRECOMPUTEDVISIBILITY_EXTENSION);
		const int32 ErrorCode = Swarm->OpenChannel(*ChannelName, LM_PRECOMPUTEDVISIBILITY_CHANNEL_FLAGS, true);
		if( ErrorCode >= 0 )
		{
			const int32 NumCells = TaskData.PrecomputedVisibilityCells.Num();
			Swarm->Write(&NumCells, sizeof(NumCells));
			for (int32 CellIndex = 0; CellIndex < NumCells; CellIndex++)
			{
				Swarm->Write(&TaskData.PrecomputedVisibilityCells[CellIndex].Bounds, sizeof(TaskData.PrecomputedVisibilityCells[CellIndex].Bounds));
				WriteArray(TaskData.PrecomputedVisibilityCells[CellIndex].VisibilityData);
			}
			WriteArray(TaskData.DebugVisibilityRays);
			Swarm->CloseCurrentChannel();
		}
		else
		{
			UE_LOGF(LogLightmass, Log, "Failed to open precomputed visibility channel!");
		}
	}

	void FLightmassSolverExporter::ExportResults(const FVolumetricLightmapTaskData& TaskData) const
	{
		const FString ChannelName = CreateChannelName(TaskData.Guid, LM_VOLUMETRICLIGHTMAP_VERSION, LM_VOLUMETRICLIGHTMAP_EXTENSION);
		const int32 ErrorCode = Swarm->OpenChannel(*ChannelName, LM_VOLUMESAMPLES_CHANNEL_FLAGS, true);
		if( ErrorCode >= 0 )
		{
			const int32 NumBricks = TaskData.BrickData.Num();
			Swarm->Write(&NumBricks, sizeof(NumBricks));

			for (int32 BrickIndex = 0; BrickIndex < NumBricks; BrickIndex++)
			{
				Swarm->Write(&TaskData.BrickData[BrickIndex].IntersectingLevelGuid, sizeof(TaskData.BrickData[BrickIndex].IntersectingLevelGuid));
				Swarm->Write(&TaskData.BrickData[BrickIndex].IndirectionTexturePosition, sizeof(TaskData.BrickData[BrickIndex].IndirectionTexturePosition));
				Swarm->Write(&TaskData.BrickData[BrickIndex].TreeDepth, sizeof(TaskData.BrickData[BrickIndex].TreeDepth));
				Swarm->Write(&TaskData.BrickData[BrickIndex].AverageClosestGeometryDistance, sizeof(TaskData.BrickData[BrickIndex].AverageClosestGeometryDistance));
				WriteArray(TaskData.BrickData[BrickIndex].AmbientVector);

				for (int32 i = 0; i < UE_ARRAY_COUNT(TaskData.BrickData[BrickIndex].SHCoefficients); i++)
				{
					WriteArray(TaskData.BrickData[BrickIndex].SHCoefficients[i]);
				}

				WriteArray(TaskData.BrickData[BrickIndex].SkyBentNormal);
				WriteArray(TaskData.BrickData[BrickIndex].DirectionalLightShadowing);
				WriteArray(TaskData.BrickData[BrickIndex].VoxelImportProcessingData);
			}

			WriteDebugLightingOutput(TaskData.DebugOutput);
			
			Swarm->CloseCurrentChannel();
		}
		else
		{
			UE_LOGF(LogLightmass, Log, "Failed to open volumetric lightmap channel!");
		}
	}

}	//Lightmass
