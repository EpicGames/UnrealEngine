// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LMCore.h"
#include "Containers/IndirectArray.h"


namespace Lightmass
{
	class FScene;
	class FMeshAreaLight;
	class FLightmassSwarm;
	class FStaticLightingTextureMappingResult;
	class FStaticLightingVertexMappingResult;

	//@todo - need to pass to Serialization
	class FLightmassSolverExporter
	{
	public:

		/**
		 * Constructor
		 * @param InSwarm Wrapper object around the Swarm interface
		 * @param bInDumpTextures If true, the 2d lightmap exporter will dump out textures
		 */
		FLightmassSolverExporter( class FLightmassSwarm* InSwarm, const FScene& InScene, bool bInDumpTextures );
		~FLightmassSolverExporter();

		class FLightmassSwarm* GetSwarm();

		/**
		 * Send complete lighting data to Unreal
		 *
		 * @param LightingData - Object containing the computed data
		 */
		void ExportResults(struct FTextureMappingStaticLightingData& LightingData, bool bUseUniqueChannel) const;
		void ExportResults(const struct FPrecomputedVisibilityData& TaskData) const;
		void ExportResults(const struct FVolumetricLightmapTaskData& TaskData) const;

		/**
		 * Used when exporting multiple mappings into a single file
		 */
		int32 BeginExportResults(struct FTextureMappingStaticLightingData& LightingData, uint32 NumMappings) const;
		void EndExportResults() const;

		/** Exports volume lighting samples to Unreal. */
		void ExportVolumeLightingSamples(
			bool bExportVolumeLightingDebugOutput,
			const struct FVolumeLightingDebugOutput& DebugOutput,
			const FVector4f& VolumeCenter, 
			const FVector4f& VolumeExtent, 
			const TMap<FGuid,TArray<class FVolumeLightingSample> >& VolumeSamples) const;

		/** Exports dominant shadow information to Unreal. */
		void ExportStaticShadowDepthMap(const FGuid& LightGuid, const class FStaticShadowDepthMap& StaticShadowDepthMap) const;

		/** 
		 * Exports information about mesh area lights back to Unreal, 
		 * So that Unreal can create dynamic lights to approximate the mesh area light's influence on dynamic objects.
		 */
		void ExportMeshAreaLightData(const class TIndirectArray<FMeshAreaLight>& MeshAreaLights, float MeshAreaLightGeneratedDynamicLightSurfaceOffset) const;

		/** Exports the volume distance field. */
		void ExportVolumeDistanceField(int32 VolumeSizeX, int32 VolumeSizeY, int32 VolumeSizeZ, float VolumeMaxDistance, const FBox3f& DistanceFieldVolumeBounds, const TArray<FColor>& VolumeDistanceField) const;

	private:
		class FLightmassSwarm*	Swarm;
		const class FScene& Scene;

		/** true if the exporter should dump out textures to disk for previewing */
		bool bDumpTextures;

		/** Writes a TArray to the channel on the top of the Swarm stack. */
		template<class T>
		void WriteArray(const TArray<T>& Array) const;

		void WriteDebugLightingOutput(const struct FDebugLightingOutput& DebugOutput) const;
	};

}	//Lightmass
