// Copyright Epic Games, Inc. All Rights Reserved.

#include "Importer.h"
#include "LightmassSwarm.h"
#include "LightmassScene.h"
#include "Misc/FileHelper.h"
#include "Misc/SecureHash.h"

namespace Lightmass
{

FLightmassImporter::FLightmassImporter( FLightmassSwarm* InSwarm )
:	Swarm( InSwarm )
,	LevelScale(0.0f)
{
	// Compute the hash of UnrealLightmass executable (which is the current running process) and store it into LightmassExecutableHash
	TArray<uint8> Bytes;
	FFileHelper::LoadFileToArray(Bytes, FPlatformProcess::ExecutablePath());
	FSHA1::HashBuffer(&Bytes[0], Bytes.Num(), LightmassExecutableHash.Hash);
}

FLightmassImporter::~FLightmassImporter()
{
}

/**
 * Imports a scene and all required dependent objects
 *
 * @param Scene Scene object to fill out
 * @param SceneGuid Guid of the scene to load from a swarm channel
 */
bool FLightmassImporter::ImportScene( class FScene& Scene, const FGuid& SceneGuid )
{
	int32 ErrorCode = Swarm->OpenChannel( *CreateChannelName( SceneGuid, LM_SCENE_VERSION, LM_SCENE_EXTENSION ), LM_SCENE_CHANNEL_FLAGS, true );
	if( ErrorCode >= 0 )
	{
		Scene.Import( *this );

		Swarm->CloseCurrentChannel( );
		return true;
	}
	else
	{
		Swarm->SendTextMessage( TEXT( "Failed to open scene channel with GUID {%08x}:{%08x}:{%08x}:{%08x}" ), SceneGuid.A, SceneGuid.B, SceneGuid.C, SceneGuid.D );
	}

	return false;
}

bool FLightmassImporter::Read( void* Data, int32 NumBytes )
{
	int32 NumRead = Swarm->Read(Data, NumBytes);
	return NumRead == NumBytes;
}

}	//Lightmass
