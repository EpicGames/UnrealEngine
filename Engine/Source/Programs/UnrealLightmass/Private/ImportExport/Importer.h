// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LMCore.h"
#include "LightmassSwarm.h"


namespace Lightmass
{


//@todo - need to pass to Serialization
class FLightmassImporter
{
protected:

	/**
	 * Finds existing or imports new object by Guid
	 *
	 * @param Key Key of object
	 * @param Version Version of object to load
	 * @param LMExecutableHash Hash of Lightmass executable to avoid stale cache content
	 * @param Extension Type of object to load (@todo UE5: This could be removed if Version could imply extension)
	 * @return The object that was loaded or found, or NULL if the Guid failed
	 */
	template <class ObjType, class LookupMapType, class KeyType>
	ObjType* ConditionalImportObjectWithKey(const KeyType& Key, const int32 Version, const FSHAHash& LMExecutableHash, const TCHAR* Extension, int32 ChannelFlags, LookupMapType& LookupMap);

public:

	FLightmassImporter( class FLightmassSwarm* InSwarm );
	~FLightmassImporter();

	/**
	 * Imports a scene and all required dependent objects
	 *
	 * @param Scene Scene object to fill out
	 * @param SceneGuid Guid of the scene to load from a swarm channel
	 */
	bool ImportScene( class FScene& Scene, const FGuid& SceneGuid );

	/** Imports a buffer of raw data */
	bool Read( void* Data, int32 NumBytes );

	/** Imports one object */
	template <class DataType>
	bool ImportData( DataType* Data );

	/** Imports a TArray of simple elements in one bulk read. */
	template <class ArrayType>
	bool ImportArray( ArrayType& Array, int32 Count );

	/** Imports a TArray of objects, while also adding them to the specified LookupMap. */
	template <class ObjType, class LookupMapType>
	bool ImportObjectArray( TArray<ObjType>& Array, int32 Count, LookupMapType& LookupMap );

	/** Imports an array of GUIDs and stores the corresponding pointers into a TArray */
	template <class ArrayType, class LookupMapType>
	bool ImportGuidArray( ArrayType& Array, int32 Count, const LookupMapType& LookupMap );

	/**
	 * Finds existing or imports new object by Guid
	 *
	 * @param Guid Guid of object
	 * @param Version Version of object to load
	 * @param Extension Type of object to load (@todo UE5: This could be removed if Version could imply extension)
	 * @return The object that was loaded or found, or NULL if the Guid failed
	 */
	template <class ObjType, class LookupMapType>
	ObjType* ConditionalImportObject(const FGuid& Guid, const int32 Version, const TCHAR* Extension, int32 ChannelFlags, LookupMapType& LookupMap)
	{
		return ConditionalImportObjectWithKey<ObjType, LookupMapType, FGuid>(Guid, Version, LightmassExecutableHash, Extension, ChannelFlags, LookupMap);
	}

	/**
	 * Finds existing or imports new object by Hash
	 *
	 * @param Hash Id of object
	 * @param Version Version of object to load
	 * @param Extension Type of object to load (@todo UE5: This could be removed if Version could imply extension)
	 * @return The object that was loaded or found, or NULL if the Guid failed
	 */
	template <class ObjType, class LookupMapType>
	ObjType* ConditionalImportObject(const FSHAHash& Hash, const int32 Version, const TCHAR* Extension, int32 ChannelFlags, LookupMapType& LookupMap)
	{
		return ConditionalImportObjectWithKey<ObjType, LookupMapType, FSHAHash>(Hash, Version, LightmassExecutableHash, Extension, ChannelFlags, LookupMap);
	}

	void SetLevelScale(float InScale) { LevelScale = InScale; }
	float GetLevelScale() const
	{
		checkf(LevelScale > 0.0f, TEXT("LevelScale must be set by the scene before it can be used"));
		return LevelScale;
	}

	TMap<FGuid,class FLight*>&										GetLights()					{ return Lights; }
	TMap<FGuid,class FStaticMeshStaticLightingMesh*>&				GetStaticMeshInstances()	{ return StaticMeshInstances; }
	TMap<FGuid,class FFluidSurfaceStaticLightingMesh*>&				GetFluidMeshInstances()		{ return FluidMeshInstances; }
	TMap<FGuid,class FLandscapeStaticLightingMesh*>&				GetLandscapeMeshInstances()	{ return LandscapeMeshInstances; }
	TMap<FGuid,class FStaticMeshStaticLightingTextureMapping*>&		GetTextureMappings()		{ return StaticMeshTextureMappings; }
	TMap<FGuid,class FBSPSurfaceStaticLighting*>&					GetBSPMappings()			{ return BSPTextureMappings; }
	TMap<FGuid,class FStaticMesh*>&									GetStaticMeshes()			{ return StaticMeshes; }
	TMap<FGuid,class FFluidSurfaceStaticLightingTextureMapping*>&	GetFluidMappings()			{ return FluidMappings; }
	TMap<FGuid,class FLandscapeStaticLightingTextureMapping*>&		GetLandscapeMappings()		{ return LandscapeMappings; }
	TMap<FGuid,class FStaticLightingGlobalVolumeMapping*>&			GetVolumeMappings()			{ return VolumeMappings; }
	TMap<FGuid,class FLandscapeStaticLightingGlobalVolumeMapping*>&	GetLandscapeVolumeMappings(){ return LandscapeVolumeMappings; }

	TMap<FSHAHash,class FMaterial*>&								GetMaterials()				{ return Materials; }

private:

	class FLightmassSwarm*	Swarm;
	FSHAHash LightmassExecutableHash;

	TMap<FGuid,class FLight*>										Lights;
	TMap<FGuid,class FStaticMesh*>									StaticMeshes;
	TMap<FGuid,class FStaticMeshStaticLightingMesh*>				StaticMeshInstances;
	TMap<FGuid,class FFluidSurfaceStaticLightingMesh*>				FluidMeshInstances;
	TMap<FGuid,class FLandscapeStaticLightingMesh*>					LandscapeMeshInstances;
	TMap<FGuid,class FStaticMeshStaticLightingTextureMapping*>		StaticMeshTextureMappings;
	TMap<FGuid,class FBSPSurfaceStaticLighting*>					BSPTextureMappings;	
	TMap<FGuid,class FFluidSurfaceStaticLightingTextureMapping*>	FluidMappings;
	TMap<FGuid,class FLandscapeStaticLightingTextureMapping*>		LandscapeMappings;
	TMap<FGuid,class FStaticLightingGlobalVolumeMapping*>			VolumeMappings;
	TMap<FGuid,class FLandscapeStaticLightingGlobalVolumeMapping*>	LandscapeVolumeMappings;
	TMap<FSHAHash,class FMaterial*>									Materials;

	float LevelScale;
};


template <typename DataType>
FORCEINLINE bool FLightmassImporter::ImportData( DataType* Data )
{
	return Read( Data, sizeof(DataType) );
}


/** Imports a TArray of simple elements in one bulk read. */
template <class ArrayType>
bool FLightmassImporter::ImportArray( ArrayType& Array, int32 Count )
{
	Array.Empty( Count );
	Array.AddUninitialized( Count );
	return Read( Array.GetData(), Count*sizeof(typename ArrayType::ElementType) );
}


/** Imports a TArray of objects, while also adding them to the specified LookupMap. */
template <class ObjType, class LookupMapType>
bool FLightmassImporter::ImportObjectArray( TArray<ObjType>& Array, int32 Count, LookupMapType& LookupMap )
{
	Array.Empty( Count );
	for ( int32 Index=0; Index < Count; ++Index )
	{
		ObjType* Item = new(Array)ObjType;
		Item->Import( *this );
		LookupMap.Add( Item->Guid, Item );
	}
	return true;
}


/** Imports an array of GUIDs and stores the corresponding pointers into a TArray */
template <class ArrayType, class LookupMapType>
bool FLightmassImporter::ImportGuidArray( ArrayType& Array, int32 Count, const LookupMapType& LookupMap )
{
	bool bOk = true;
	Array.Empty( Count );
	for ( int32 Index=0; bOk && Index < Count; ++Index )
	{
		FGuid Guid;
		bOk = ImportData( &Guid );
		Array.Add( LookupMap.FindRef( Guid ) );
	}
	return bOk;
}


/**
 * Finds existing or imports new object by Guid
 *
 * @param Key Key of object
 * @param Version Version of object to load
 * @param Extension Type of object to load (@todo UE5: This could be removed if Version could imply extension)
 * @return The object that was loaded or found, or NULL if the Guid failed
 */
template <class ObjType, class LookupMapType, class KeyType>
ObjType* FLightmassImporter::ConditionalImportObjectWithKey(const KeyType& Key, const int32 Version, const FSHAHash& LMExecutableHash, const TCHAR* Extension, int32 ChannelFlags, LookupMapType& LookupMap)
{
	// look to see if it exists already
	ObjType* Obj = LookupMap.FindRef(Key);
	if (Obj == NULL)
	{
		// open a new channel and make it current
		if (Swarm->OpenChannel(*CreateChannelNameWithLMExecutableHash(Key, Version, LMExecutableHash, Extension), ChannelFlags, true) >= 0)
		{
			Obj = new ObjType;

			// import the object from its own channel
			Obj->Import(*this);
		
			// close the object channel
			Swarm->CloseCurrentChannel();

			// cache this object so it can be found later by another call to this function
			LookupMap.Add(Key, Obj);
		}
	}

	return Obj;
}

}
