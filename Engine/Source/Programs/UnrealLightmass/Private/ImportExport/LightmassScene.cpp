// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmassScene.h"
#include "Importer.h"
#include "MonteCarlo.h"
#include "LightingSystem.h"
#include "LMDebug.h"
#include "HAL/LowLevelMemTracker.h"

namespace Lightmass
{

/** Copy constructor that doesn't modify padding in FSceneFileHeader. */
FSceneFileHeader::FSceneFileHeader(const FSceneFileHeader& Other)
{
	/** FourCC cookie: 'SCEN' */
	Cookie = Other.Cookie;
	FormatVersion = Other.FormatVersion;
	Guid = Other.Guid;
	GeneralSettings = Other.GeneralSettings;
	SceneConstants = Other.SceneConstants;
	DynamicObjectSettings = Other.DynamicObjectSettings;
	VolumetricLightmapSettings = Other.VolumetricLightmapSettings;
	PrecomputedVisibilitySettings = Other.PrecomputedVisibilitySettings;
	VolumeDistanceFieldSettings = Other.VolumeDistanceFieldSettings;
	MeshAreaLightSettings = Other.MeshAreaLightSettings;
	AmbientOcclusionSettings = Other.AmbientOcclusionSettings;
	ShadowSettings = Other.ShadowSettings;
	ImportanceTracingSettings = Other.ImportanceTracingSettings;
	PhotonMappingSettings = Other.PhotonMappingSettings;
	IrradianceCachingSettings = Other.IrradianceCachingSettings;
	MaterialSettings = Other.MaterialSettings;
	DebugInput = Other.DebugInput;

	/** If true, pad the mappings (shrink the requested size and then pad) */
	bPadMappings = Other.bPadMappings;

	/** If true, draw a solid border as the padding around mappings */
	bDebugPadding = Other.bDebugPadding;

	bOnlyCalcDebugTexelMappings = Other.bOnlyCalcDebugTexelMappings;
	bColorBordersGreen = Other.bColorBordersGreen;
	bUseRandomColors = Other.bUseRandomColors;
	bColorByExecutionTime = Other.bColorByExecutionTime;
	ExecutionTimeDivisor = Other.ExecutionTimeDivisor;

	NumImportanceVolumes = Other.NumImportanceVolumes;
	NumCharacterIndirectDetailVolumes = Other.NumCharacterIndirectDetailVolumes;
	NumVolumetricLightmapDensityVolumes = Other.NumVolumetricLightmapDensityVolumes;
	NumDirectionalLights = Other.NumDirectionalLights;
	NumPointLights = Other.NumPointLights;
	NumSpotLights = Other.NumSpotLights;
	NumRectLights = Other.NumRectLights;
	NumSkyLights = Other.NumSkyLights;
	NumStaticMeshes = Other.NumStaticMeshes;
	NumStaticMeshInstances = Other.NumStaticMeshInstances;
	NumFluidSurfaceInstances = Other.NumFluidSurfaceInstances;
	NumLandscapeInstances = Other.NumLandscapeInstances;
	NumBSPMappings = Other.NumBSPMappings;
	NumStaticMeshTextureMappings = Other.NumStaticMeshTextureMappings;
	NumFluidSurfaceTextureMappings = Other.NumFluidSurfaceTextureMappings;
	NumLandscapeTextureMappings = Other.NumLandscapeTextureMappings;
	NumSpeedTreeMappings = Other.NumSpeedTreeMappings;
	NumVolumeMappings = Other.NumVolumeMappings;
	NumLandscapeVolumeMappings = Other.NumLandscapeVolumeMappings;
	NumPortals = Other.NumPortals;
}

//----------------------------------------------------------------------------
//	Scene class
//----------------------------------------------------------------------------
FScene::FScene() : 
	EmbreeDevice(NULL),
	bVerifyEmbree(false)
{
	FMemory::Memzero( (FSceneFileHeader*)this, sizeof(FSceneFileHeader) );
}

FScene::~FScene()
{
#if USE_EMBREE
#if USE_EMBREE_MAJOR_VERSION >= 3
	rtcReleaseDevice(EmbreeDevice);
#else
	rtcDeleteDevice(EmbreeDevice);
#endif // USE_EMBREE_MAJOR_VERSION >= 3
#endif
}

#if USE_EMBREE
LLM_DECLARE_TAG(Embree);

#if USE_EMBREE_MAJOR_VERSION >= 3
static bool EmbreeLlmMemoryMonitor(void* ptr, ssize_t bytes, bool post)
#else
static bool EmbreeLlmMemoryMonitor(const ssize_t bytes, const bool post)
#endif // USE_EMBREE_MAJOR_VERSION >= 3
{
	LLM_SCOPE_BYTAG(Embree);
	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelChangeInMemoryUse(ELLMTracker::Default, static_cast<int64>(bytes)));

	return true;
}
#endif

void FScene::Import( FLightmassImporter& Importer )
{
	FSceneFileHeader TempHeader;
	// Import into a temp header, since padding in the header overlaps with data members in FScene and ImportData stomps on that padding
	Importer.ImportData(&TempHeader);
	// Copy header members without modifying padding in FSceneFileHeader
	(FSceneFileHeader&)(*this) = TempHeader;

#if USE_EMBREE
	if (TempHeader.GeneralSettings.bUseEmbree)
	{
		EmbreeDevice = rtcNewDevice(NULL);
#if USE_EMBREE_MAJOR_VERSION >= 3
		check(rtcGetDeviceError(EmbreeDevice) == RTC_ERROR_NONE);
		LLM_IF_ENABLED(rtcSetDeviceMemoryMonitorFunction(EmbreeDevice, EmbreeLlmMemoryMonitor, NULL));
#else
		check(rtcDeviceGetError(EmbreeDevice) == RTC_NO_ERROR);
		LLM_IF_ENABLED(rtcDeviceSetMemoryMonitorFunction(EmbreeDevice, EmbreeLlmMemoryMonitor));
#endif // USE_EMBREE_MAJOR_VERSION >= 3
		bVerifyEmbree = TempHeader.GeneralSettings.bVerifyEmbree;
	}
#endif

	// The assignment above overwrites ImportanceVolumes since FSceneFileHeader has some padding which coincides with ImportanceVolumes
	FMemory::Memzero(&ImportanceVolumes, sizeof(ImportanceVolumes));

	Importer.SetLevelScale(SceneConstants.StaticLightingLevelScale);
	ApplyStaticLightingScale();
	
	FStaticLightingMapping::s_bShowLightmapBorders = bDebugPadding;

	TArray<TCHAR, FString::AllocatorType>& InstigatorUserNameArray = InstigatorUserName.GetCharArray();
	int32 UserNameLen;
	Importer.ImportData(&UserNameLen);
	Importer.ImportArray(InstigatorUserNameArray, UserNameLen);
	InstigatorUserNameArray.Add('\0');

	FString PersistentLevelName;
	TArray<TCHAR, FString::AllocatorType>& PersistentLevelNameArray = PersistentLevelName.GetCharArray();
	int32 PersistentLevelNameLen;
	Importer.ImportData(&PersistentLevelNameLen);
	Importer.ImportArray(PersistentLevelNameArray, PersistentLevelNameLen);
	PersistentLevelNameArray.Add('\0');

	ImportanceBoundingBox.Init();
	for (int32 VolumeIndex = 0; VolumeIndex < NumImportanceVolumes; VolumeIndex++)
	{
		FBox3f LMBox;
		Importer.ImportData(&LMBox);
		ImportanceBoundingBox += LMBox;
		ImportanceVolumes.Add(LMBox);
	}

	if (NumImportanceVolumes == 0)
	{
		ImportanceBoundingBox = FBox3f(FVector4f(0,0,0), FVector4f(0,0,0));
	}

	for (int32 VolumeIndex = 0; VolumeIndex < NumCharacterIndirectDetailVolumes; VolumeIndex++)
	{
		FBox3f LMBox;
		Importer.ImportData(&LMBox);
		CharacterIndirectDetailVolumes.Add(LMBox);
	}

	for (int32 PortalIndex = 0; PortalIndex < NumPortals; PortalIndex++)
	{
		FMatrix44f LMPortal;
		Importer.ImportData(&LMPortal);
		Portals.Add(FSphere3f(LMPortal.GetOrigin(), FVector2f(LMPortal.GetScaleVector().Y, LMPortal.GetScaleVector().Z).Size()));
	}

	Importer.ImportArray(VisibilityBucketGuids, NumPrecomputedVisibilityBuckets);

	int32 NumVisVolumes;
	Importer.ImportData(&NumVisVolumes);
	PrecomputedVisibilityVolumes.Empty(NumVisVolumes);
	PrecomputedVisibilityVolumes.AddZeroed(NumVisVolumes);
	for (int32 VolumeIndex = 0; VolumeIndex < NumVisVolumes; VolumeIndex++)
	{
		FPrecomputedVisibilityVolume& CurrentVolume = PrecomputedVisibilityVolumes[VolumeIndex];
		Importer.ImportData(&CurrentVolume.Bounds);
		int32 NumPlanes;
		Importer.ImportData(&NumPlanes);
		Importer.ImportArray(CurrentVolume.Planes, NumPlanes);
	}

	int32 NumVisOverrideVolumes;
	Importer.ImportData(&NumVisOverrideVolumes);
	PrecomputedVisibilityOverrideVolumes.Empty(NumVisOverrideVolumes);
	PrecomputedVisibilityOverrideVolumes.AddZeroed(NumVisOverrideVolumes);
	for (int32 VolumeIndex = 0; VolumeIndex < NumVisOverrideVolumes; VolumeIndex++)
	{
		FPrecomputedVisibilityOverrideVolume& CurrentVolume = PrecomputedVisibilityOverrideVolumes[VolumeIndex];
		Importer.ImportData(&CurrentVolume.Bounds);
		int32 NumVisibilityIds;
		Importer.ImportData(&NumVisibilityIds);
		Importer.ImportArray(CurrentVolume.OverrideVisibilityIds, NumVisibilityIds);
		int32 NumInvisibilityIds;
		Importer.ImportData(&NumInvisibilityIds);
		Importer.ImportArray(CurrentVolume.OverrideInvisibilityIds, NumInvisibilityIds);
	}

	int32 NumCameraTrackPositions;
	Importer.ImportData(&NumCameraTrackPositions);
	Importer.ImportArray(CameraTrackPositions, NumCameraTrackPositions);

	for (int32 VolumeIndex = 0; VolumeIndex < NumVolumetricLightmapDensityVolumes; VolumeIndex++)
	{
		FVolumetricLightmapDensityVolumeData LMVolumeData;
		Importer.ImportData(&LMVolumeData);
		static_assert(sizeof(LMVolumeData) == sizeof(FBox3f) + sizeof(FIntPoint) + sizeof(int32), "Update member copy");
		FVolumetricLightmapDensityVolume LMVolume;
		LMVolume.Bounds = LMVolumeData.Bounds;
		LMVolume.AllowedMipLevelRange = LMVolumeData.AllowedMipLevelRange;
		LMVolume.NumPlanes = LMVolumeData.NumPlanes;
		Importer.ImportArray(LMVolume.Planes, LMVolume.NumPlanes);
		VolumetricLightmapDensityVolumes.Add(LMVolume);
	}

	Importer.ImportArray(VolumetricLightmapTaskGuids, NumVolumetricLightmapTasks);

	Importer.ImportObjectArray( DirectionalLights, NumDirectionalLights, Importer.GetLights() );
	Importer.ImportObjectArray( PointLights, NumPointLights, Importer.GetLights() );
	Importer.ImportObjectArray( SpotLights, NumSpotLights, Importer.GetLights() );
	Importer.ImportObjectArray( RectLights, NumRectLights, Importer.GetLights() );
	Importer.ImportObjectArray( SkyLights, NumSkyLights, Importer.GetLights() );

	Importer.ImportObjectArray( StaticMeshInstances, NumStaticMeshInstances, Importer.GetStaticMeshInstances() );
	Importer.ImportObjectArray( FluidMeshInstances, NumFluidSurfaceInstances, Importer.GetFluidMeshInstances() );
	Importer.ImportObjectArray( LandscapeMeshInstances, NumLandscapeInstances, Importer.GetLandscapeMeshInstances() );
	Importer.ImportObjectArray( BspMappings, NumBSPMappings, Importer.GetBSPMappings() );
	Importer.ImportObjectArray( TextureLightingMappings, NumStaticMeshTextureMappings, Importer.GetTextureMappings() );
	Importer.ImportObjectArray( FluidMappings, NumFluidSurfaceTextureMappings, Importer.GetFluidMappings() );
	Importer.ImportObjectArray( LandscapeMappings, NumLandscapeTextureMappings, Importer.GetLandscapeMappings() );
	Importer.ImportObjectArray( VolumeMappings, NumVolumeMappings, Importer.GetVolumeMappings() );
	Importer.ImportObjectArray( LandscapeVolumeMappings, NumLandscapeVolumeMappings, Importer.GetLandscapeVolumeMappings() );

	DebugMapping = FindMappingByGuid(DebugInput.MappingGuid);
	if (DebugMapping)
	{
#if !ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
		checkf(false, TEXT("Texel Debugging active, but Lightmass was compiled with ALLOW_LIGHTMAP_SAMPLE_DEBUGGING == 0"));
#endif
		const FStaticLightingTextureMapping* TextureMapping = DebugMapping->GetTextureMapping();

		// Verify debug input is valid, otherwise there will be an access violation later
		if (TextureMapping)
		{
			check(DebugInput.LocalX >= 0 && DebugInput.LocalX < TextureMapping->CachedSizeX);
			check(DebugInput.LocalY >= 0 && DebugInput.LocalY < TextureMapping->CachedSizeY);
			check(DebugInput.MappingSizeX == TextureMapping->CachedSizeX && DebugInput.MappingSizeY == TextureMapping->CachedSizeY);
		}
	}

	if (bPadMappings == true)
	{
		// BSP mappings
		for (int32 MappingIdx = 0; MappingIdx < BspMappings.Num(); MappingIdx++)
		{
			int32 SizeX = BspMappings[MappingIdx].Mapping.SizeX;
			int32 SizeY = BspMappings[MappingIdx].Mapping.SizeY;

			if (((SizeX - 2) > 0) && ((SizeY - 2) > 0))
			{
				BspMappings[MappingIdx].Mapping.CachedSizeX = FMath::Clamp<int32>(SizeX, 0, SizeX - 2);
				BspMappings[MappingIdx].Mapping.CachedSizeY = FMath::Clamp<int32>(SizeY, 0, SizeY - 2);
				BspMappings[MappingIdx].Mapping.bPadded = true;
			}
		}

		// Static mesh texture mappings
		for (int32 MappingIdx = 0; MappingIdx < TextureLightingMappings.Num(); MappingIdx++)
		{
			int32 SizeX = TextureLightingMappings[MappingIdx].SizeX;
			int32 SizeY = TextureLightingMappings[MappingIdx].SizeY;

			if (((SizeX - 2) > 0) && ((SizeY - 2) > 0))
			{
				TextureLightingMappings[MappingIdx].CachedSizeX = FMath::Clamp<int32>(SizeX, 0, SizeX - 2);
				TextureLightingMappings[MappingIdx].CachedSizeY = FMath::Clamp<int32>(SizeY, 0, SizeY - 2);
				TextureLightingMappings[MappingIdx].bPadded = true;
			}
		}


		// Fluid mappings
		for (int32 MappingIdx = 0; MappingIdx < FluidMappings.Num(); MappingIdx++)
		{
			int32 SizeX = FluidMappings[MappingIdx].SizeX;
			int32 SizeY = FluidMappings[MappingIdx].SizeY;

			if (((SizeX - 2) > 0) && ((SizeY - 2) > 0))
			{
				FluidMappings[MappingIdx].CachedSizeX = FMath::Clamp<int32>(SizeX, 0, SizeX - 2);
				FluidMappings[MappingIdx].CachedSizeY = FMath::Clamp<int32>(SizeY, 0, SizeY - 2);
				FluidMappings[MappingIdx].bPadded = true;
			}
		}

		// Landscape mappings - do not get padded by Lightmass...
		for (int32 MappingIdx = 0; MappingIdx < LandscapeMappings.Num(); MappingIdx++)
		{
			LandscapeMappings[MappingIdx].CachedSizeX = LandscapeMappings[MappingIdx].SizeX;
			LandscapeMappings[MappingIdx].CachedSizeY = LandscapeMappings[MappingIdx].SizeY;
			LandscapeMappings[MappingIdx].bPadded = false;
		}

	}

	if (DebugMapping)
	{
		const FStaticLightingTextureMapping* TextureMapping = DebugMapping->GetTextureMapping();

		// Verify debug input is valid, otherwise there will be an access violation later
		if (TextureMapping)
		{
			check(DebugInput.LocalX >= 0 && DebugInput.LocalX < TextureMapping->CachedSizeX);
			check(DebugInput.LocalY >= 0 && DebugInput.LocalY < TextureMapping->CachedSizeY);
			check(DebugInput.MappingSizeX == TextureMapping->SizeX && DebugInput.MappingSizeY == TextureMapping->SizeY);
		}
	}
}

FBoxSphereBounds3f FScene::GetImportanceBounds() const
{
	const FBoxSphereBounds3f ImportanceBoundSphere(ImportanceBoundingBox);
	return ImportanceBoundSphere;
}

const FLight* FScene::FindLightByGuid(const FGuid& InGuid) const
{
	for (int32 i = 0; i < DirectionalLights.Num(); i++)
	{
		if (DirectionalLights[i].Guid == InGuid)
		{
			return &DirectionalLights[i];
		}
	}
	for (int32 i = 0; i < PointLights.Num(); i++)
	{
		if (PointLights[i].Guid == InGuid)
		{
			return &PointLights[i];
		}
	}
	for (int32 i = 0; i < SpotLights.Num(); i++)
	{
		if (SpotLights[i].Guid == InGuid)
		{
			return &SpotLights[i];
		}
	}
	for (int32 i = 0; i < RectLights.Num(); i++)
	{
		if (RectLights[i].Guid == InGuid)
		{
			return &RectLights[i];
		}
	}
	for (int32 i = 0; i < SkyLights.Num(); i++)
	{
		if (SkyLights[i].Guid == InGuid)
		{
			return &SkyLights[i];
		}
	}
	return NULL;
}

/** Searches through all mapping arrays for the mapping matching FindGuid. */
const FStaticLightingMapping* FScene::FindMappingByGuid(FGuid FindGuid) const
{ 
	// Note: FindGuid can be all 0's and still be valid due to deterministic lighting overriding the Guid
	for (int32 i = 0; i < BspMappings.Num(); i++)
	{
		if (BspMappings[i].Mapping.Guid == FindGuid)
		{
			return &BspMappings[i].Mapping;
		}
	}

	for (int32 i = 0; i < TextureLightingMappings.Num(); i++)
	{
		if (TextureLightingMappings[i].Guid == FindGuid)
		{
			return &TextureLightingMappings[i];
		}
	}

	for (int32 i = 0; i < FluidMappings.Num(); i++)
	{
		if (FluidMappings[i].Guid == FindGuid)
		{
			return &FluidMappings[i];
		}
	}

	for (int32 i = 0; i < LandscapeMappings.Num(); i++)
	{
		if (LandscapeMappings[i].Guid == FindGuid)
		{
			return &LandscapeMappings[i];
		}
	}

	for (int32 i = 0; i < VolumeMappings.Num(); i++)
	{
		if (VolumeMappings[i].Guid == FindGuid)
		{
			return &VolumeMappings[i];
		}
	}

	for (int32 i = 0; i < LandscapeVolumeMappings.Num(); i++)
	{
		if (LandscapeVolumeMappings[i].Guid == FindGuid)
		{
			return &LandscapeVolumeMappings[i];
		}
	}


	return NULL;
}

/** Returns true if the specified position is inside any of the importance volumes. */
bool FScene::IsPointInImportanceVolume(const FVector4f& Position, float Tolerance) const
{
	for (int32 VolumeIndex = 0; VolumeIndex < ImportanceVolumes.Num(); VolumeIndex++)
	{
		FBox3f Volume = ImportanceVolumes[VolumeIndex];

		if (Position.X + Tolerance > Volume.Min.X && Position.X - Tolerance < Volume.Max.X
			&& Position.Y + Tolerance > Volume.Min.Y && Position.Y - Tolerance < Volume.Max.Y
			&& Position.Z + Tolerance > Volume.Min.Z && Position.Z - Tolerance < Volume.Max.Z)
		{
			return true;
		}
	}
	return false;
}

bool FScene::IsBoxInImportanceVolume(const FBox3f& QueryBox) const
{
	for (int32 VolumeIndex = 0; VolumeIndex < ImportanceVolumes.Num(); VolumeIndex++)
	{
		FBox3f Volume = ImportanceVolumes[VolumeIndex];

		if (Volume.Intersect(QueryBox))
		{
			return true;
		}
	}

	return false;
}

/** Returns true if the specified position is inside any of the visibility volumes. */
bool FScene::IsPointInVisibilityVolume(const FVector4f& Position) const
{
	for (int32 VolumeIndex = 0; VolumeIndex < PrecomputedVisibilityVolumes.Num(); VolumeIndex++)
	{
		const FPrecomputedVisibilityVolume& Volume = PrecomputedVisibilityVolumes[VolumeIndex];
		bool bInsideAllPlanes = true;
		for (int32 PlaneIndex = 0; PlaneIndex < Volume.Planes.Num() && bInsideAllPlanes; PlaneIndex++)
		{
			const FPlane4f& Plane = Volume.Planes[PlaneIndex];
			bInsideAllPlanes = bInsideAllPlanes && Plane.PlaneDot(Position) < 0.0f;
		}
		if (bInsideAllPlanes)
		{
			return true;
		}
	}
	return false;
}

bool FScene::DoesBoxIntersectVisibilityVolume(const FBox3f& TestBounds) const
{
	for (int32 VolumeIndex = 0; VolumeIndex < PrecomputedVisibilityVolumes.Num(); VolumeIndex++)
	{
		const FPrecomputedVisibilityVolume& Volume = PrecomputedVisibilityVolumes[VolumeIndex];
		if (Volume.Bounds.Intersect(TestBounds))
		{
			return true;
		}
	}
	return false;
}

/** Returns accumulated bounds from all the visibility volumes. */
FBox3f FScene::GetVisibilityVolumeBounds() const
{
	FBox3f Bounds(ForceInit);
	for (int32 VolumeIndex = 0; VolumeIndex < PrecomputedVisibilityVolumes.Num(); VolumeIndex++)
	{
		const FPrecomputedVisibilityVolume& Volume = PrecomputedVisibilityVolumes[VolumeIndex];
		Bounds += Volume.Bounds;
	}
	if (PrecomputedVisibilityVolumes.Num() > 0)
	{
		FVector4f DoubleExtent = Bounds.GetExtent() * 2;
		DoubleExtent.X = DoubleExtent.X - FMath::Fmod(DoubleExtent.X, PrecomputedVisibilitySettings.CellSize) + PrecomputedVisibilitySettings.CellSize;
		DoubleExtent.Y = DoubleExtent.Y - FMath::Fmod(DoubleExtent.Y, PrecomputedVisibilitySettings.CellSize) + PrecomputedVisibilitySettings.CellSize;
		// Round the max up to the next cell boundary
		Bounds.Max = Bounds.Min + DoubleExtent;
		return Bounds;
	}
	else
	{
		return FBox3f(FVector4f(0,0,0),FVector4f(0,0,0));
	}
}

bool FScene::GetVolumetricLightmapAllowedMipRange(const FVector4f& Position, FIntPoint& OutRange) const
{
	FIntPoint Range(INT_MAX, INT_MAX);

	bool bVolumeFound = false;

	for (int32 VolumeIndex = 0; VolumeIndex < VolumetricLightmapDensityVolumes.Num(); VolumeIndex++)
	{
		const FVolumetricLightmapDensityVolume& Volume = VolumetricLightmapDensityVolumes[VolumeIndex];
		bool bInsideAllPlanes = true;
		for (int32 PlaneIndex = 0; PlaneIndex < Volume.Planes.Num() && bInsideAllPlanes; PlaneIndex++)
		{
			const FPlane4f& Plane = Volume.Planes[PlaneIndex];
			bInsideAllPlanes = bInsideAllPlanes && Plane.PlaneDot(Position) < 0.0f;
		}
		if (bInsideAllPlanes)
		{
			bVolumeFound = true;
			Range.X = FMath::Min(Range.X, Volume.AllowedMipLevelRange.X);
			Range.Y = FMath::Min(Range.Y, Volume.AllowedMipLevelRange.Y);
		}
	}

	OutRange = Range;
	return bVolumeFound;
}

/** Applies GeneralSettings.StaticLightingLevelScale to all scale dependent settings. */
void FScene::ApplyStaticLightingScale()
{
	// Scale world space distances directly
	SceneConstants.VisibilityRayOffsetDistance *= SceneConstants.StaticLightingLevelScale;
	SceneConstants.VisibilityNormalOffsetDistance *= SceneConstants.StaticLightingLevelScale;
	SceneConstants.SmallestTexelRadius *= SceneConstants.StaticLightingLevelScale;
	MeshAreaLightSettings.MeshAreaLightSimplifyCornerDistanceThreshold *= SceneConstants.StaticLightingLevelScale;
	MeshAreaLightSettings.MeshAreaLightGeneratedDynamicLightSurfaceOffset *= SceneConstants.StaticLightingLevelScale;
	DynamicObjectSettings.FirstSurfaceSampleLayerHeight *= SceneConstants.StaticLightingLevelScale;
	DynamicObjectSettings.SurfaceLightSampleSpacing *= SceneConstants.StaticLightingLevelScale;
	DynamicObjectSettings.SurfaceSampleLayerHeightSpacing *= SceneConstants.StaticLightingLevelScale;
	DynamicObjectSettings.DetailVolumeSampleSpacing *= SceneConstants.StaticLightingLevelScale;
	DynamicObjectSettings.VolumeLightSampleSpacing *= SceneConstants.StaticLightingLevelScale;
	VolumeDistanceFieldSettings.VoxelSize *= SceneConstants.StaticLightingLevelScale;
	VolumeDistanceFieldSettings.VolumeMaxDistance *= SceneConstants.StaticLightingLevelScale;
	ShadowSettings.MaxTransitionDistanceWorldSpace *= SceneConstants.StaticLightingLevelScale;
	ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceX *= SceneConstants.StaticLightingLevelScale;
	ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceY *= SceneConstants.StaticLightingLevelScale;
	IrradianceCachingSettings.RecordRadiusScale *= SceneConstants.StaticLightingLevelScale;
	IrradianceCachingSettings.MaxRecordRadius *= SceneConstants.StaticLightingLevelScale;

	// Photon mapping does not scale down properly, so this is disabled
	/*
	PhotonMappingSettings.IndirectPhotonEmitDiskRadius *= SceneConstants.StaticLightingLevelScale;
	PhotonMappingSettings.MaxImportancePhotonSearchDistance *= SceneConstants.StaticLightingLevelScale;
	PhotonMappingSettings.MinImportancePhotonSearchDistance *= SceneConstants.StaticLightingLevelScale;
	// Scale surface densities in world units
	const float ScaleSquared = SceneConstants.StaticLightingLevelScale * SceneConstants.StaticLightingLevelScale;
	PhotonMappingSettings.DirectPhotonDensity /= ScaleSquared;
	PhotonMappingSettings.DirectIrradiancePhotonDensity /= ScaleSquared;
	PhotonMappingSettings.DirectPhotonSearchDistance *= SceneConstants.StaticLightingLevelScale;
	PhotonMappingSettings.IndirectPhotonPathDensity /= ScaleSquared;
	PhotonMappingSettings.IndirectPhotonDensity /= ScaleSquared;
	PhotonMappingSettings.IndirectIrradiancePhotonDensity /= ScaleSquared;
	PhotonMappingSettings.IndirectPhotonSearchDistance *= SceneConstants.StaticLightingLevelScale;
	*/
}

//----------------------------------------------------------------------------
//	Light base class
//----------------------------------------------------------------------------
void FLight::Import( FLightmassImporter& Importer )
{
	Importer.ImportData( (FLightData*)this );
	Importer.ImportArray( LightTextureProfileData, FLightData::LightProfileTextureDataSize );

	// The read above stomps on CachedLightSurfaceSamples since that memory is padding in FLightData
	FMemory::Memzero(&CachedLightSurfaceSamples, sizeof(CachedLightSurfaceSamples));
	
	// Precalculate the light's indirect color
	IndirectColor = FLinearColorUtils::AdjustSaturation(FLinearColor(Color), IndirectLightingSaturation) * IndirectLightingScale;
}

/**
 * Tests whether the light affects the given bounding volume.
 * @param Bounds - The bounding volume to test.
 * @return True if the light affects the bounding volume
 */
bool FLight::AffectsBounds(const FBoxSphereBounds3f& Bounds) const
{
	return true;
}

FSphere3f FLight::GetBoundingSphere() const
{
	// Directional lights will have a radius of WORLD_MAX
	return FSphere3f(FVector3f(0, 0, 0), (float)WORLD_MAX);
}

/**
 * Computes the intensity of the direct lighting from this light on a specific point.
 */
FLinearColor FLight::GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const
{
	// light profile (IES)
	float LightProfileAttenuation;
	{
		LightProfileAttenuation = ComputeLightProfileMultiplier(LightTextureProfileData, Point, Position, Direction, GetLightTangent());
	}

	if (bCalculateForIndirectLighting)
	{
		return IndirectColor * (LightProfileAttenuation * Brightness);
	}
	else
	{
		return FLinearColor(Color) * (LightProfileAttenuation * Brightness);
	}
}

/** Generates and caches samples on the light's surface. */
void FLight::CacheSurfaceSamples(int32 BounceIndex, int32 NumSamples, int32 NumPenumbraSamples, FLMRandomStream& RandomStream)
{
	checkSlow(NumSamples > 0);
	// Assuming bounce number starts from 0 and increments each time
	//@todo - remove the slack
	CachedLightSurfaceSamples.AddZeroed(1);
	// Allocate for both normal and penumbra even if there aren't any penumbra samples, so we can return an empty array from GetCachedSurfaceSamples
	CachedLightSurfaceSamples[BounceIndex].AddZeroed(2);
	const int32 NumPenumbraTypes = NumPenumbraSamples > 0 ? 2 : 1;
	for (int32 PenumbraType = 0; PenumbraType < NumPenumbraTypes; PenumbraType++)
	{
		const int32 CurrentNumSamples = PenumbraType == 0 ? NumSamples : NumPenumbraSamples;
		CachedLightSurfaceSamples[BounceIndex][PenumbraType].Empty(CurrentNumSamples);
		for (int32 SampleIndex = 0; SampleIndex < CurrentNumSamples; SampleIndex++)
		{
			FLightSurfaceSample LightSample;
			SampleLightSurface(RandomStream, LightSample);
			CachedLightSurfaceSamples[BounceIndex][PenumbraType].Add(LightSample);
		}
	}
}

/** Retrieves the array of cached light surface samples. */
const TArray<FLightSurfaceSample>& FLight::GetCachedSurfaceSamples(int32 BounceIndex, bool bPenumbra) const
{
	return CachedLightSurfaceSamples[BounceIndex][bPenumbra];
}

//----------------------------------------------------------------------------
//	Directional light class
//----------------------------------------------------------------------------
void FDirectionalLight::Import( FLightmassImporter& Importer )
{
	FLight::Import( Importer );

	Importer.ImportData( (FDirectionalLightData*)this );
}

void FDirectionalLight::Initialize(
	const FBoxSphereBounds3f& InSceneBounds, 
	bool bInEmitPhotonsOutsideImportanceVolume,
	const FBoxSphereBounds3f& InImportanceBounds, 
	float InIndirectDiskRadius, 
	int32 InGridSize,
	float InDirectPhotonDensity,
	float InOutsideImportanceVolumeDensity)
{
	GenerateCoordinateSystem(Direction, XAxis, YAxis);

	SceneBounds = InSceneBounds;
	ImportanceBounds = InImportanceBounds;

	// Vector through the scene bound's origin, along the direction of the light
	const FVector4f SceneAxis = (SceneBounds.Origin + Direction * SceneBounds.SphereRadius) - (SceneBounds.Origin - Direction * SceneBounds.SphereRadius);
	const float SceneAxisLength = SceneBounds.SphereRadius * 2.0f;
	const FVector4f DirectionalLightOriginToImportanceOrigin = ImportanceBounds.Origin - (SceneBounds.Origin - Direction * SceneBounds.SphereRadius);
	// Find the closest point on the scene's axis to the importance volume's origin by projecting DirectionalLightOriginToImportanceOrigin onto SceneAxis.
	// This gives the offset in the directional light's disk from the scene bound's origin.
	const FVector4f ClosestPositionOnAxis = Dot3(SceneAxis, DirectionalLightOriginToImportanceOrigin) / (SceneAxisLength * SceneAxisLength) * SceneAxis + SceneBounds.Origin - Direction * SceneBounds.SphereRadius;

	// Find the disk offset from the world space origin and transform into the [-1,1] space of the directional light's disk, still in 3d.
	const FVector4f DiskOffset = (ImportanceBounds.Origin - ClosestPositionOnAxis) / SceneBounds.SphereRadius;

	const float DebugLength = (ImportanceBounds.Origin - ClosestPositionOnAxis).Size();
	const float DebugDot = ((ImportanceBounds.Origin - ClosestPositionOnAxis) / DebugLength) | Direction;
	// Verify that ImportanceBounds.Origin is either on the scene's axis or the vector between it and ClosestPositionOnAxis is orthogonal to the light's direction
	//checkSlow(DebugLength < KINDA_SMALL_NUMBER * 10.0f || FMath::Abs(DebugDot) < DELTA * 10.0f);

	// Decompose DiskOffset into it's corresponding parts along XAxis and YAxis
	const FVector4f XAxisProjection = Dot3(XAxis, DiskOffset) * XAxis;
	const FVector4f YAxisProjection = Dot3(YAxis, DiskOffset) * YAxis;
	ImportanceDiskOrigin = FVector2f(Dot3(XAxisProjection, XAxis), Dot3(YAxisProjection, YAxis));

	// Transform the importance volume's radius into the [-1,1] space of the directional light's disk
	LightSpaceImportanceDiskRadius = ImportanceBounds.SphereRadius / SceneBounds.SphereRadius;

	const FVector4f DebugPosition = (ImportanceDiskOrigin.X * XAxis + ImportanceDiskOrigin.Y * YAxis);
	const float DebugLength2 = (DiskOffset - DebugPosition).Size3();
	// Verify that DiskOffset was decomposed correctly by reconstructing it
	checkSlow(DebugLength2 < KINDA_SMALL_NUMBER);
	
	IndirectDiskRadius = InIndirectDiskRadius;
	GridSize = InGridSize;
	OutsideImportanceVolumeDensity = InOutsideImportanceVolumeDensity;

	const float ImportanceDiskAreaMillions = (float)PI * FMath::Square(ImportanceBounds.SphereRadius) / 1000000.0f;
	checkSlow(SceneBounds.SphereRadius >= ImportanceBounds.SphereRadius);
	const float OutsideImportanceDiskAreaMillions = (float)PI * (FMath::Square(SceneBounds.SphereRadius) - FMath::Square(ImportanceBounds.SphereRadius)) / 1000000.0f;
	// Calculate the probability that a generated sample will be in the importance volume,
	// Based on the fraction of total photons that should be gathered in the importance volume.
	ImportanceBoundsSampleProbability = ImportanceDiskAreaMillions * InDirectPhotonDensity
		/ (ImportanceDiskAreaMillions * InDirectPhotonDensity + OutsideImportanceDiskAreaMillions * OutsideImportanceVolumeDensity);

	// Calculate the size of the directional light source using Tangent(LightSourceAngle) = LightSourceRadius / DistanceToReceiver
	LightSourceRadius = 2.0f * SceneBounds.SphereRadius * FMath::Tan(LightSourceAngle);

	if (!bInEmitPhotonsOutsideImportanceVolume && ImportanceBounds.SphereRadius > DELTA)
	{
		// Always sample inside the importance volume
		ImportanceBoundsSampleProbability = 1.0f;
		OutsideImportanceVolumeDensity = 0.0f;
	}
}

/** Returns the number of direct photons to gather required by this light. */
int32 FDirectionalLight::GetNumDirectPhotons(float DirectPhotonDensity) const
{
	int32 NumDirectPhotons = 0;
	if (ImportanceBounds.SphereRadius > DELTA)
	{
		// The importance volume is valid, so only gather enough direct photons to meet DirectPhotonDensity inside the importance volume
		const float ImportanceDiskAreaMillions = (float)PI * FMath::Square(ImportanceBounds.SphereRadius) / 1000000.0f;
		checkSlow(SceneBounds.SphereRadius > ImportanceBounds.SphereRadius);
		const float OutsideImportanceDiskAreaMillions = (float)PI * (FMath::Square(SceneBounds.SphereRadius) - FMath::Square(ImportanceBounds.SphereRadius)) / 1000000.0f;
		NumDirectPhotons = FMath::TruncToInt(ImportanceDiskAreaMillions * DirectPhotonDensity + OutsideImportanceDiskAreaMillions * OutsideImportanceVolumeDensity);
	}
	else
	{
		// Gather enough photons to meet DirectPhotonDensity everywhere in the scene
		const float SceneDiskAreaMillions = (float)PI * FMath::Square(SceneBounds.SphereRadius) / 1000000.0f;
		NumDirectPhotons = FMath::TruncToInt(SceneDiskAreaMillions * DirectPhotonDensity);
	}
	return NumDirectPhotons == appTruncErrorCode ? INT_MAX : NumDirectPhotons;
}

/** Generates a direction sample from the light's domain */
void FDirectionalLight::SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const
{
	FVector4f DiskPosition3D;
	// If the importance volume is valid, generate samples in the importance volume with a probability of ImportanceBoundsSampleProbability
	if (ImportanceBounds.SphereRadius > DELTA 
		&& RandomStream.GetFraction() < ImportanceBoundsSampleProbability)
	{
		const FVector2f DiskPosition2D = GetUniformUnitDiskPosition(RandomStream);
		LightSurfacePosition = ImportanceDiskOrigin + DiskPosition2D * LightSpaceImportanceDiskRadius;
		DiskPosition3D = SceneBounds.Origin + SceneBounds.SphereRadius * (LightSurfacePosition.X * XAxis + LightSurfacePosition.Y * YAxis);
		RayPDF = ImportanceBoundsSampleProbability / ((float)PI * FMath::Square(ImportanceBounds.SphereRadius));
	}
	else
	{
		float DistanceToImportanceDiskOriginSq;
		do 
		{
			LightSurfacePosition = GetUniformUnitDiskPosition(RandomStream);
			DistanceToImportanceDiskOriginSq = (LightSurfacePosition - ImportanceDiskOrigin).SizeSquared();
		} 
		// Use rejection sampling to prevent any samples from being generated inside the importance volume
		while (DistanceToImportanceDiskOriginSq < FMath::Square(LightSpaceImportanceDiskRadius));
		
		// Create the ray using a disk centered at the scene's origin, whose radius is the size of the scene
		DiskPosition3D = SceneBounds.Origin + SceneBounds.SphereRadius * (LightSurfacePosition.X * XAxis + LightSurfacePosition.Y * YAxis);
		// Calculate the probability of generating a uniform disk sample in the scene, minus the importance volume's disk
		RayPDF = (1.0f - ImportanceBoundsSampleProbability) / ((float)PI * (FMath::Square(SceneBounds.SphereRadius) - FMath::Square(ImportanceBounds.SphereRadius)));
	}
	
	//@todo - take light source radius into account
	SampleRay = FLightRay(
		DiskPosition3D - SceneBounds.SphereRadius * Direction,
		DiskPosition3D + SceneBounds.SphereRadius * Direction,
		NULL,
		this
		);

	LightSourceNormal = Direction;

	checkSlow(RayPDF > 0);
	Power = IndirectColor * Brightness;
}

/** Gives the light an opportunity to precalculate information about the indirect path rays that will be used to generate new directions. */
void FDirectionalLight::CachePathRays(const TArray<FIndirectPathRay>& IndirectPathRays)
{
	if (IndirectPathRays.Num() == 0)
	{
		return;
	}
	// The indirect disk radius in the [-1, 1] space of the directional light's disk
	const float LightSpaceIndirectDiskRadius = IndirectDiskRadius / SceneBounds.SphereRadius;

	// Find the minimum and maximum position in the [-1, 1] space of the directional light's disk
	// That a position can be generated from in FDirectionalLight::SampleDirection
	FVector2f GridMin(1.0f, 1.0f);
	FVector2f GridMax(-1.0f, -1.0f);
	for (int32 RayIndex = 0; RayIndex < IndirectPathRays.Num(); RayIndex++)
	{
		const FIndirectPathRay& CurrentRay = IndirectPathRays[RayIndex];
		GridMin.X = FMath::Min(GridMin.X, CurrentRay.LightSurfacePosition.X - LightSpaceIndirectDiskRadius);
		GridMin.Y = FMath::Min(GridMin.Y, CurrentRay.LightSurfacePosition.Y - LightSpaceIndirectDiskRadius);
		GridMax.X = FMath::Max(GridMax.X, CurrentRay.LightSurfacePosition.X + LightSpaceIndirectDiskRadius);
		GridMax.Y = FMath::Max(GridMax.Y, CurrentRay.LightSurfacePosition.Y + LightSpaceIndirectDiskRadius);
	}
	GridMin.X = FMath::Min(GridMin.X, 1.0f);
	GridMin.Y = FMath::Min(GridMin.Y, 1.0f);
	GridMax.X = FMath::Max(GridMax.X, -1.0f);
	GridMax.Y = FMath::Max(GridMax.Y, -1.0f);
	//checkSlow(GridMax > GridMin);
	const FVector2f GridExtent2D = 0.5f * (GridMax - GridMin);
	// Keep the grid space square to simplify logic
	GridExtent = FMath::Max(GridExtent2D.X, GridExtent2D.Y);
	GridCenter = 0.5f * (GridMin + GridMax);

	// Allocate the grid
	PathRayGrid.Empty(GridSize * GridSize);
	PathRayGrid.AddZeroed(GridSize * GridSize);

	const float GridSpaceIndirectDiskRadius = IndirectDiskRadius * GridExtent / SceneBounds.SphereRadius;
	const float InvGridSize = 1.0f / (float)GridSize;

	// For each grid cell, store the indices into IndirectPathRays of the path rays that affect the grid cell
	for (int32 Y = 0; Y < GridSize; Y++)
	{
		for (int32 X = 0; X < GridSize; X++)
		{
			// Center and Extent of the cell in the [0, 1] grid space
			const FVector2f BoxCenter((X + .5f) * InvGridSize, (Y + .5f) * InvGridSize);
			const float BoxExtent = .5f * InvGridSize;

			// Corners of the cell
			const int32 NumBoxCorners = 4;
			FVector2f BoxCorners[NumBoxCorners];
			BoxCorners[0] = BoxCenter + FVector2f(BoxExtent, BoxExtent);
			BoxCorners[1] = BoxCenter + FVector2f(-BoxExtent, BoxExtent);
			BoxCorners[2] = BoxCenter + FVector2f(BoxExtent, -BoxExtent);
			BoxCorners[3] = BoxCenter + FVector2f(-BoxExtent, -BoxExtent);

			// Calculate the world space positions of each corner of the cell
			FVector4f WorldBoxCorners[NumBoxCorners];
			for (int32 i = 0; i < NumBoxCorners; i++)
			{				
				// Transform the cell corner from [0, 1] grid space to [-1, 1] in the directional light's disk
				const FVector2f LightBoxCorner(2.0f * GridExtent * BoxCorners[i] + GridCenter - FVector2f(GridExtent, GridExtent));
				// Calculate the world position of the cell corner
				WorldBoxCorners[i] = SceneBounds.Origin + SceneBounds.SphereRadius * (LightBoxCorner.X * XAxis + LightBoxCorner.Y * YAxis) - SceneBounds.SphereRadius * Direction;
			}
			// Calculate the world space distance along the diagonal of the box
			const float DiagonalBoxDistance = (WorldBoxCorners[0] - WorldBoxCorners[3]).Size3();
			const float DiagonalBoxDistanceAndRadiusSquared = FMath::Square(DiagonalBoxDistance + IndirectDiskRadius);

			for (int32 RayIndex = 0; RayIndex < IndirectPathRays.Num(); RayIndex++)
			{
				const FIndirectPathRay& CurrentRay = IndirectPathRays[RayIndex];
				bool bAnyCornerInCircle = false;
				bool bWithinDiagonalDistance = true;
				// If any of the box corners lie within the disk around the current path ray, then they intersect
				for (int32 i = 0; i < NumBoxCorners; i++)
				{				
					const float SampleDistanceSquared = (WorldBoxCorners[i] - CurrentRay.Start).SizeSquared3(); 
					bWithinDiagonalDistance = bWithinDiagonalDistance && SampleDistanceSquared < DiagonalBoxDistanceAndRadiusSquared;
					if (SampleDistanceSquared < IndirectDiskRadius * IndirectDiskRadius)
					{
						bAnyCornerInCircle = true;
						PathRayGrid[Y * GridSize + X].Add(RayIndex);
						break;
					}
				}

				// If none of the box corners lie within the disk but the disk is less than the diagonal + the disk radius, treat them as intersecting.  
				// This is a conservative test, they might not actually intersect.
				if (!bAnyCornerInCircle && bWithinDiagonalDistance)
				{
					PathRayGrid[Y * GridSize + X].Add(RayIndex);
				}
			}
		}
	}
}

/** Generates a direction sample from the light based on the given rays */
void FDirectionalLight::SampleDirection(
	const TArray<FIndirectPathRay>& IndirectPathRays, 
	FLMRandomStream& RandomStream, 
	FLightRay& SampleRay, 
	float& RayPDF,
	FLinearColor& Power) const
{
	checkSlow(IndirectPathRays.Num() > 0);

	const FVector2f DiskPosition2D = GetUniformUnitDiskPosition(RandomStream);
	const int32 RayIndex = FMath::TruncToInt(RandomStream.GetFraction() * IndirectPathRays.Num());
	checkSlow(RayIndex >= 0 && RayIndex < IndirectPathRays.Num());

	// Create the ray using a disk centered at the scene's origin, whose radius is the size of the scene
	const FVector4f DiskPosition3D = IndirectPathRays[RayIndex].Start + IndirectDiskRadius * (DiskPosition2D.X * XAxis + DiskPosition2D.Y * YAxis);
	
	SampleRay = FLightRay(
		DiskPosition3D,
		DiskPosition3D + 2.0f * SceneBounds.SphereRadius * Direction,
		NULL,
		this
		);

	const float DiskPDF = 1.0f / ((float)PI * IndirectDiskRadius * IndirectDiskRadius);
	const float LightSpaceIndirectDiskRadius = IndirectDiskRadius / SceneBounds.SphereRadius;
	FVector2f SampleLightSurfacePosition;
	// Clamp the generated position to lie within the [-1, 1] space of the directional light's disk
	SampleLightSurfacePosition.X = FMath::Clamp(DiskPosition2D.X * LightSpaceIndirectDiskRadius + IndirectPathRays[RayIndex].LightSurfacePosition.X, -1.0f, 1.0f - DELTA);
	SampleLightSurfacePosition.Y = FMath::Clamp(DiskPosition2D.Y * LightSpaceIndirectDiskRadius + IndirectPathRays[RayIndex].LightSurfacePosition.Y, -1.0f, 1.0f - DELTA);

	checkSlow(SampleLightSurfacePosition.X >= GridCenter.X - GridExtent && SampleLightSurfacePosition.X <= GridCenter.X + GridExtent);
	checkSlow(SampleLightSurfacePosition.Y >= GridCenter.Y - GridExtent && SampleLightSurfacePosition.Y <= GridCenter.Y + GridExtent);
	// Calculate the cell indices that the generated position falls into
	const int32 CellX = FMath::Clamp(FMath::TruncToInt(GridSize * (SampleLightSurfacePosition.X - GridCenter.X + GridExtent) / (2.0f * GridExtent)), 0, GridSize - 1);
	const int32 CellY = FMath::Clamp(FMath::TruncToInt(GridSize * (SampleLightSurfacePosition.Y - GridCenter.Y + GridExtent) / (2.0f * GridExtent)), 0, GridSize - 1);
	const TArray<int32>& CurrentGridCell = PathRayGrid[CellY * GridSize + CellX];
	// The cell containing the sample position must contain at least the index of the path used to generate this sample position
	checkSlow(CurrentGridCell.Num() > 0);
	// Initialize the total PDF to the PDF contribution from the path used to generate this sample position
	RayPDF = DiskPDF;
	// Calculate the probability that this sample was chosen by other paths
	// Iterating only over paths that affect the sample position's cell as an optimization
	for (int32 OtherRayIndex = 0; OtherRayIndex < CurrentGridCell.Num(); OtherRayIndex++)
	{
		const int32 CurrentPathIndex = CurrentGridCell[OtherRayIndex];
		const FIndirectPathRay& CurrentPath = IndirectPathRays[CurrentPathIndex];
		const float SampleDistanceSquared = (DiskPosition3D - CurrentPath.Start).SizeSquared3();
		// Accumulate the disk probability for all the disks which contain the sample position
		if (SampleDistanceSquared < IndirectDiskRadius * IndirectDiskRadius
			// The path that was used to generate the sample has already been counted
			&& CurrentPathIndex != RayIndex)
		{
			RayPDF += DiskPDF; 
		}
	}
	
	RayPDF /= IndirectPathRays.Num();
	
	check(RayPDF > 0);
	Power = IndirectColor * Brightness;
}

/** Returns the light's radiant power. */
float FDirectionalLight::Power() const
{
	const float EffectiveRadius = ImportanceBounds.SphereRadius > DELTA ? ImportanceBounds.SphereRadius : SceneBounds.SphereRadius;
	const FLinearColor LightPower = GetDirectIntensity(FVector4f(0,0,0), false) * IndirectLightingScale * (float)PI * EffectiveRadius * EffectiveRadius;
	return FLinearColorUtils::LinearRGBToXYZ(LightPower).G;
}

/** Validates a surface sample given the position that sample is affecting. */
void FDirectionalLight::ValidateSurfaceSample(const FVector4f& Point, FLightSurfaceSample& Sample) const
{
	// Directional light samples are generated on a disk the size of the light source radius, centered on the origin
	// Move the disk to the other side of the scene along the light's reverse direction
	Sample.Position += Point - Direction * 2.0f * SceneBounds.SphereRadius;
}

/** Gets a single position which represents the center of the area light source from the ReceivingPosition's point of view. */
FVector4f FDirectionalLight::LightCenterPosition(const FVector4f& ReceivingPosition, const FVector4f& ReceivingNormal) const
{
	return ReceivingPosition - Direction * 2.0f * SceneBounds.SphereRadius;
}

/** Returns true if all parts of the light are behind the surface being tested. */
bool FDirectionalLight::BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const
{
	const float NormalDotLight = Dot3(TriangleNormal, FDirectionalLight::GetDirectLightingDirection(TrianglePoint, TriangleNormal));
	return NormalDotLight < 0.0f;
}

/** Gets a single direction to use for direct lighting that is representative of the whole area light. */
FVector4f FDirectionalLight::GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const
{
	// The position on the directional light surface disk that will first be visible to a triangle rotating toward the light
	const FVector4f FirstVisibleLightPoint = Point - Direction * 2.0f * SceneBounds.SphereRadius + PointNormal * LightSourceRadius;
	return FirstVisibleLightPoint - Point;
}

/** Generates a sample on the light's surface. */
void FDirectionalLight::SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const
{
	// Create samples on a disk the size of the light source radius, centered at the origin
	// This disk will be moved based on the receiver position
	//@todo - stratify
	Sample.DiskPosition = GetUniformUnitDiskPosition(RandomStream);
	Sample.Position = LightSourceRadius * (Sample.DiskPosition.X * XAxis + Sample.DiskPosition.Y * YAxis);
	Sample.Normal = Direction;
	Sample.PDF = 1.0f / ((float)PI * LightSourceRadius * LightSourceRadius);
}

//----------------------------------------------------------------------------
//	Point light class
//----------------------------------------------------------------------------
void FPointLight::Import( FLightmassImporter& Importer )
{
	FLight::Import( Importer );

	Importer.ImportData( (FPointLightData*)this );
}

void FPointLight::Initialize(float InIndirectPhotonEmitConeAngle)
{
	CosIndirectPhotonEmitConeAngle = FMath::Cos(InIndirectPhotonEmitConeAngle);
}

/** Returns the number of direct photons to gather required by this light. */
int32 FPointLight::GetNumDirectPhotons(float DirectPhotonDensity) const
{
	// Gather enough photons to meet DirectPhotonDensity at the influence radius of the point light.
	const float InfluenceSphereSurfaceAreaMillions = 4.0f * (float)PI * FMath::Square(Radius) / 1000000.0f;
	const int32 NumDirectPhotons = FMath::TruncToInt(InfluenceSphereSurfaceAreaMillions * DirectPhotonDensity);
	return NumDirectPhotons == appTruncErrorCode ? INT_MAX : NumDirectPhotons;
}

/**
 * Tests whether the light affects the given bounding volume.
 * @param Bounds - The bounding volume to test.
 * @return True if the light affects the bounding volume
 */
bool FPointLight::AffectsBounds(const FBoxSphereBounds3f& Bounds) const
{
	if((Bounds.Origin - Position).SizeSquared() > FMath::Square(Radius + Bounds.SphereRadius))
	{
		return false;
	}

	if(!FLight::AffectsBounds(Bounds))
	{
		return false;
	}

	return true;
}

/**
 * Computes the intensity of the direct lighting from this light on a specific point.
 */
FLinearColor FPointLight::GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const
{
	if (LightFlags & GI_LIGHT_INVERSE_SQUARED)
	{
		FVector4f ToLight = Position - Point;
		float DistanceSqr = ToLight.SizeSquared3();

		float DistanceAttenuation = 0.0f;
		if( LightSourceLength > 0.0f )
		{
			// Line segment irradiance
			FVector4f L01 =  GetLightTangent() * LightSourceLength;
			FVector4f L0 = ToLight - 0.5f * L01;
			FVector4f L1 = ToLight + 0.5f * L01;
			float LengthL0 = L0.Size3();
			float LengthL1 = L1.Size3();

			DistanceAttenuation = 1.0f / ( ( LengthL0 * LengthL1 + Dot3( L0, L1 ) ) * 0.5f + 1.0f );
			DistanceAttenuation *= 0.5f * ( L0 / LengthL0 + L1 / LengthL1 ).Size3();
		}
		else
		{
			// Sphere irradiance (technically just 1/d^2 but this avoids inf)
			DistanceAttenuation = 1.0f / ( DistanceSqr + 1.0f );
		}

		float LightRadiusMask = FMath::Square(FMath::Max(0.0f, 1.0f - FMath::Square(DistanceSqr / (Radius * Radius))));
		DistanceAttenuation *= LightRadiusMask;

		return FLight::GetDirectIntensity(Point, bCalculateForIndirectLighting) * DistanceAttenuation;
	}
	else
	{
		float RadialAttenuation = FMath::Pow(FMath::Max(1.0f - ((Position - Point) / Radius).SizeSquared3(), 0.0f), (FVector4f::FReal)FalloffExponent);

		return FLight::GetDirectIntensity(Point, bCalculateForIndirectLighting) * RadialAttenuation;
	}
}

/** Returns an intensity scale based on the receiving point. */
float FPointLight::CustomAttenuation(const FVector4f& Point, FLMRandomStream& RandomStream, bool bMaintainEvenDensity) const
{
	// Remove the physical attenuation, then attenuation using Unreal point light radial falloff
	const float PointDistanceSquared = (Position - Point).SizeSquared3();
	const float PhysicalAttenuation = 1.0f / ( PointDistanceSquared + 0.0001f );

	float UnrealAttenuation = 1.0f;
	if( LightFlags & GI_LIGHT_INVERSE_SQUARED )
	{
		const float LightRadiusMask = FMath::Square( FMath::Max( 0.0f, 1.0f - FMath::Square( PointDistanceSquared / (Radius * Radius) ) ) );
		UnrealAttenuation = PhysicalAttenuation * LightRadiusMask;
	}
	else
	{
		UnrealAttenuation = FMath::Pow(FMath::Max(1.0f - ((Position - Point) / Radius).SizeSquared3(), 0.0f), (FVector4f::FReal)FalloffExponent);
	}

	// light profile (IES)
	{
		UnrealAttenuation *= ComputeLightProfileMultiplier(LightTextureProfileData, Point, Position, Direction, GetLightTangent());
	}

	// Thin out photons near the light source.
	// This is partly an optimization since the photon density near light sources doesn't need to be high, and the natural 1 / R^2 density is overkill, 
	// But this also improves quality since we are doing a nearest N photon neighbor search when calculating irradiance.  
	// If the photon map has a high density of low power photons near light sources,
	// Combined with sparse, high power photons from other light sources (directional lights for example), the result will be very splotchy.
	const float FullProbabilityDistance = .5f * Radius;
	const float DepositProbability = bMaintainEvenDensity ?
		FMath::Clamp(PointDistanceSquared / (FullProbabilityDistance * FullProbabilityDistance), 0.0f, 1.0f) : 
		1.0f;

	if (RandomStream.GetFraction() < DepositProbability)
	{
		// Re-weight the photon since it survived the thinning based on the probability of being deposited
		return UnrealAttenuation / (PhysicalAttenuation * DepositProbability);
	}
	else
	{
		return 0.0f;
	}
}

// Fudge factor to get point light photon intensities to match direct lighting more closely.
static const float PointLightIntensityScale = 1.0f; 

/** Generates a direction sample from the light's domain */
void FPointLight::SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const
{
	const FVector4f RandomDirection = GetUnitVector(RandomStream);

	FLightSurfaceSample SurfaceSample;
	SampleLightSurface(RandomStream, SurfaceSample);

	const float SurfacePositionDotDirection = Dot3((SurfaceSample.Position - Position), RandomDirection);
	if (SurfacePositionDotDirection < 0.0f)
	{
		// Reflect the surface position about the origin so that it lies in the same hemisphere as the RandomDirection
		const FVector4f LocalSamplePosition = SurfaceSample.Position - Position;
		SurfaceSample.Position = -LocalSamplePosition + Position;
	}

	SampleRay = FLightRay(
		SurfaceSample.Position,
		SurfaceSample.Position + RandomDirection * FMath::Max((Radius - LightSourceRadius), 0.0f),
		NULL,
		this
		);

	LightSourceNormal = (SurfaceSample.Position - Position).GetSafeNormal();

	// Approximate the probability of generating this direction as uniform over all the solid angles
	// This diverges from the actual probability for positions inside the light source radius
	RayPDF = 1.0f / (4.0f * (float)PI);
	Power = IndirectColor * Brightness * PointLightIntensityScale;
}

/** Generates a direction sample from the light based on the given rays */
void FPointLight::SampleDirection(
	const TArray<FIndirectPathRay>& IndirectPathRays, 
	FLMRandomStream& RandomStream, 
	FLightRay& SampleRay, 
	float& RayPDF, 
	FLinearColor& Power) const
{
	checkSlow(IndirectPathRays.Num() > 0);
	// Pick an indirect path ray with uniform probability
	const int32 RayIndex = FMath::TruncToInt(RandomStream.GetFraction() * IndirectPathRays.Num());
	checkSlow(RayIndex >= 0 && RayIndex < IndirectPathRays.Num());

	const FVector4f PathRayDirection = IndirectPathRays[RayIndex].UnitDirection;

	FVector4f XAxis(0,0,0);
	FVector4f YAxis(0,0,0);
	GenerateCoordinateSystem(PathRayDirection, XAxis, YAxis);

	// Generate a sample direction within a cone about the indirect path
	const FVector4f ConeSampleDirection = UniformSampleCone(RandomStream, CosIndirectPhotonEmitConeAngle, XAxis, YAxis, PathRayDirection);

	FLightSurfaceSample SurfaceSample;
	// Generate a surface sample, not taking the indirect path into account
	SampleLightSurface(RandomStream, SurfaceSample);

	const float SurfacePositionDotDirection = Dot3((SurfaceSample.Position - Position), ConeSampleDirection);
	if (SurfacePositionDotDirection < 0.0f)
	{
		// Reflect the surface position about the origin so that it lies in the same hemisphere as the ConeSampleDirection
		const FVector4f LocalSamplePosition = SurfaceSample.Position - Position;
		SurfaceSample.Position = -LocalSamplePosition + Position;
	}

	SampleRay = FLightRay(
		SurfaceSample.Position,
		SurfaceSample.Position + ConeSampleDirection * FMath::Max((Radius - LightSourceRadius), 0.0f),
		NULL,
		this
		);

	const float ConePDF = UniformConePDF(CosIndirectPhotonEmitConeAngle);
	RayPDF = 0.0f;
	// Calculate the probability that this direction was chosen
	for (int32 OtherRayIndex = 0; OtherRayIndex < IndirectPathRays.Num(); OtherRayIndex++)
	{
		// Accumulate the disk probability for all the disks which contain the sample position
		if (Dot3(IndirectPathRays[OtherRayIndex].UnitDirection, ConeSampleDirection) > (1.0f - DELTA) * CosIndirectPhotonEmitConeAngle)
		{
			RayPDF += ConePDF;
		}
	}
	RayPDF /= IndirectPathRays.Num();
	checkSlow(RayPDF > 0);
	Power = IndirectColor * Brightness * PointLightIntensityScale;
}

/** Validates a surface sample given the position that sample is affecting. */
void FPointLight::ValidateSurfaceSample(const FVector4f& Point, FLightSurfaceSample& Sample) const
{
	// Only attempt to fixup sphere light source sample positions as the light source is radially symmetric
	if (LightSourceLength <= 0)
	{
		const FVector4f LightToPoint = Point - Position;
		const float LightToPointDistanceSquared = LightToPoint.SizeSquared3();
		if (LightToPointDistanceSquared < FMath::Square(LightSourceRadius * 2.0f))
		{
			// Point is inside the light source radius * 2
			FVector4f LocalSamplePosition = Sample.Position - Position;
			// Reposition the light surface sample on a sphere whose radius is half of the distance from the light to Point
			LocalSamplePosition *= FMath::Sqrt(LightToPointDistanceSquared) / (2.0f * LightSourceRadius);
			Sample.Position = LocalSamplePosition + Position;
		}
	
		const float SurfacePositionDotDirection = Dot3((Sample.Position - Position), LightToPoint);
		if (SurfacePositionDotDirection < 0.0f)
		{
			// Reflect the surface position about the origin so that it lies in the hemisphere facing Point
			// The sample's PDF is unchanged
			const FVector4f LocalSamplePosition = Sample.Position - Position;
			Sample.Position = -LocalSamplePosition + Position;
		}
	}
}

/** Returns the light's radiant power. */
float FPointLight::Power() const
{
	FLinearColor IncidentPower = FLinearColor(Color) * Brightness * IndirectLightingScale;
	// Approximate power of the light by the total amount of light passing through a sphere at half the light's radius
	const float RadiusFraction = .5f;
	const float DistanceToEvaluate = RadiusFraction * Radius;

	if (LightFlags & GI_LIGHT_INVERSE_SQUARED)
	{
		IncidentPower = IncidentPower / (DistanceToEvaluate * DistanceToEvaluate);
	}
	else
	{
		float UnrealAttenuation = FMath::Pow(FMath::Max(1.0f - RadiusFraction * RadiusFraction, 0.0f), FalloffExponent);
		// Point light power is proportional to its radius squared
		IncidentPower = IncidentPower * UnrealAttenuation;
	} 

	const FLinearColor LightPower = IncidentPower * 4.f * PI * DistanceToEvaluate * DistanceToEvaluate;
	return FLinearColorUtils::LinearRGBToXYZ(LightPower).G;
}

FVector4f FPointLight::LightCenterPosition(const FVector4f& ReceivingPosition, const FVector4f& ReceivingNormal) const
{
	if( LightSourceLength > 0 )
	{
		FVector4f ToLight = Position - ReceivingPosition;

		FVector4f Dir = GetLightTangent();
		if( Dot3( ReceivingNormal, Dir ) < 0.0f )
		{
			Dir = -Dir;
		}

		// Clip to hemisphere
		float Proj = FMath::Min( Dot3( ToLight, Dir ), Dot3( ReceivingNormal, ToLight) / Dot3( ReceivingNormal, Dir ) );

		// Point on line segment closest to Point
		return Position - Dir * FMath::Clamp( Proj, -0.5f * LightSourceLength, 0.5f * LightSourceLength );
	}
	else
	{
		return Position;
	}
}

/** Returns true if all parts of the light are behind the surface being tested. */
bool FPointLight::BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const
{
	const float NormalDotLight = Dot3(TriangleNormal, FPointLight::GetDirectLightingDirection(TrianglePoint, TriangleNormal));
	return NormalDotLight < 0.0f;
}

/** Gets a single direction to use for direct lighting that is representative of the whole area light. */
FVector4f FPointLight::GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const
{
	FVector4f LightPosition = Position;

	if( LightSourceLength > 0 )
	{
		FVector4f ToLight = Position - Point;
		FVector4f L01 =  GetLightTangent() * LightSourceLength;
		FVector4f L0 = ToLight - 0.5 * L01;
		FVector4f L1 = ToLight + 0.5 * L01;
#if 0
		// Point on line segment with smallest angle to normal
		float A = LightSourceLength * LightSourceLength;
		float B = 2.0f * Dot3( L0, L01 );
		float C = Dot3( L0, L0 );
		float D = Dot3( PointNormal, L0 );
		float E = Dot3( PointNormal, L01 );
		float t = FMath::Clamp( (B*D - 2.0f * C*E) / (B*E - 2.0f * A*D), 0.0f, 1.0f );
		return L0 + t * L01;
#else
		// Line segment irradiance
		float LengthL0 = L0.Size3();
		float LengthL1 = L1.Size3();
		return ( L0 * LengthL1 + L1 * LengthL0 ) / ( LengthL0 + LengthL1 );
#endif
	}
	else
	{
		// The position on the point light surface sphere that will first be visible to a triangle rotating toward the light
		const FVector4f FirstVisibleLightPoint = LightPosition + PointNormal * LightSourceRadius;
		return FirstVisibleLightPoint - Point;
	}
}

FVector3f FPointLight::GetLightTangent() const
{
	return LightTangent;
}

/** Generates a sample on the light's surface. */
void FPointLight::SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const
{
	Sample.DiskPosition = FVector2f(0, 0);

	if (LightSourceLength <= 0)
	{
		// Generate a sample on the surface of the sphere with uniform density over the surface area of the sphere
		//@todo - stratify
		const FVector4f UnitSpherePosition = GetUnitVector(RandomStream);
		Sample.Position = UnitSpherePosition * LightSourceRadius + Position;
		Sample.Normal = UnitSpherePosition;
		// Probability of generating this surface position is 1 / SurfaceArea
		Sample.PDF = 1.0f / (4.0f * (float)PI * LightSourceRadius * LightSourceRadius);
	}
	else
	{
		const float ClampedLightSourceRadius = FMath::Max(DELTA, LightSourceRadius);
		float CylinderSurfaceArea = 2.0f * (float)PI * ClampedLightSourceRadius * LightSourceLength;
		float SphereSurfaceArea = 4.0f * (float)PI * ClampedLightSourceRadius * ClampedLightSourceRadius;
		float TotalSurfaceArea = CylinderSurfaceArea + SphereSurfaceArea;

		const FVector3f TubeLightDirection = GetLightTangent();

		// Cylinder End caps
		// The chance of calculating a point on the end sphere is equal to it's percentage of total surface area
		if (RandomStream.GetFraction() < SphereSurfaceArea / TotalSurfaceArea)
		{
			// Generate a sample on the surface of the sphere with uniform density over the surface area of the sphere
			//@todo - stratify
			const FVector4f UnitSpherePosition = GetUnitVector(RandomStream);
			Sample.Position = UnitSpherePosition * ClampedLightSourceRadius + Position;

			if (Dot3(UnitSpherePosition, TubeLightDirection) > 0)
			{
				Sample.Position += TubeLightDirection * (LightSourceLength * 0.5f);
			}
			else
			{
				Sample.Position += -TubeLightDirection * (LightSourceLength * 0.5f);
			}

			Sample.Normal = UnitSpherePosition;
		}
		// Cylinder body
		else
		{
			// Get point along center line
			FVector4f CentreLinePosition = Position + TubeLightDirection * LightSourceLength * (RandomStream.GetFraction() - 0.5f);
			// Get point radius away from center line at random angle
			float Theta = 2.0f * (float)PI * RandomStream.GetFraction();
			FVector4f CylEdgePos = FVector4f(0, FMath::Cos(Theta), FMath::Sin(Theta), 1);
			CylEdgePos = FRotationMatrix44f::MakeFromZ( TubeLightDirection ).TransformVector( CylEdgePos );

			Sample.Position = CylEdgePos * ClampedLightSourceRadius + CentreLinePosition;
			Sample.Normal = CylEdgePos;
		}

		// Probability of generating this surface position is 1 / SurfaceArea
		Sample.PDF = 1.0f / TotalSurfaceArea;
	}
}

//----------------------------------------------------------------------------
//	Spot light class
//----------------------------------------------------------------------------
void FSpotLight::Import( FLightmassImporter& Importer )
{
	FPointLight::Import( Importer );

	Importer.ImportData( (FSpotLightData*)this );
}

void FSpotLight::Initialize(float InIndirectPhotonEmitConeAngle)
{
	FPointLight::Initialize(InIndirectPhotonEmitConeAngle);

	float ClampedInnerConeAngle = FMath::Clamp(InnerConeAngle, 0.0f, 89.0f) * (float)PI / 180.0f;
	float ClampedOuterConeAngle = FMath::Clamp(OuterConeAngle * (float)PI / 180.0f,ClampedInnerConeAngle + 0.001f,89.0f * (float)PI / 180.0f + 0.001f);

	SinOuterConeAngle = FMath::Sin(ClampedOuterConeAngle),
	CosOuterConeAngle = FMath::Cos(ClampedOuterConeAngle);
	CosInnerConeAngle = FMath::Cos(ClampedInnerConeAngle);
}

/**
 * Tests whether the light affects the given bounding volume.
 * @param Bounds - The bounding volume to test.
 * @return True if the light affects the bounding volume
 */
bool FSpotLight::AffectsBounds(const FBoxSphereBounds3f& Bounds) const
{
	if(!FLight::AffectsBounds(Bounds))
	{
		return false;
	}

	// Radial check
	if((Bounds.Origin - Position).SizeSquared() > FMath::Square(Radius + Bounds.SphereRadius))
	{
		return false;
	}

	// Cone check


	FVector4f	U = Position - (Bounds.SphereRadius / SinOuterConeAngle) * Direction,
				D = Bounds.Origin - U;
	float	dsqr = Dot3(D, D),
			E = Dot3(Direction, D);
	if(E > 0.0f && E * E >= dsqr * FMath::Square(CosOuterConeAngle))
	{
		D = Bounds.Origin - Position;
		dsqr = Dot3(D, D);
		E = -Dot3(Direction, D);
		if(E > 0.0f && E * E >= dsqr * FMath::Square(SinOuterConeAngle))
			return dsqr <= FMath::Square(Bounds.SphereRadius);
		else
			return true;
	}

	return false;
}

FSphere3f FSpotLight::GetBoundingSphere() const
{
	return (FSphere3f)FMath::ComputeBoundingSphereForCone((FVector3f)Position, (FVector3f)Direction, Radius, CosOuterConeAngle, SinOuterConeAngle);
}

/**
 * Computes the intensity of the direct lighting from this light on a specific point.
 */
FLinearColor FSpotLight::GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const
{
	FVector4f LightVector = (Point - Position).GetSafeNormal();
	float SpotAttenuation = FMath::Square(FMath::Clamp<float>((Dot3(LightVector, Direction) - CosOuterConeAngle) / (CosInnerConeAngle - CosOuterConeAngle),0.0f,1.0f));

	if( LightFlags & GI_LIGHT_INVERSE_SQUARED )
	{
		FVector4f ToLight = Position - Point;
		float DistanceSqr = ToLight.SizeSquared3();

		float DistanceAttenuation = 0.0f;
		if( LightSourceLength > 0.0f )
		{
			// Line segment irradiance
			FVector4f L01 = GetLightTangent() * LightSourceLength;
			FVector4f L0 = ToLight - 0.5 * L01;
			FVector4f L1 = ToLight + 0.5 * L01;
			float LengthL0 = L0.Size3();
			float LengthL1 = L1.Size3();

			DistanceAttenuation = 1.0f / ( ( LengthL0 * LengthL1 + Dot3( L0, L1 ) ) * 0.5f + 1.0f );
		}
		else
		{
			// Sphere irradiance (technically just 1/d^2 but this avoids inf)
			DistanceAttenuation = 1.0f / ( DistanceSqr + 1.0f );
		}

		float LightRadiusMask = FMath::Square( FMath::Max( 0.0f, 1.0f - FMath::Square( DistanceSqr / (Radius * Radius) ) ) );
		DistanceAttenuation *= LightRadiusMask;

		return FLight::GetDirectIntensity(Point, bCalculateForIndirectLighting) * DistanceAttenuation * SpotAttenuation;
	}
	else
	{
		float RadialAttenuation = FMath::Pow( FMath::Max(1.0f - ((Position - Point) / Radius).SizeSquared3(),0.0f), (FVector4f::FReal)FalloffExponent );

		return FLight::GetDirectIntensity(Point, bCalculateForIndirectLighting) * RadialAttenuation * SpotAttenuation;
	}
}

/** Returns the number of direct photons to gather required by this light. */
int32 FSpotLight::GetNumDirectPhotons(float DirectPhotonDensity) const
{
	const float InfluenceSphereSurfaceAreaMillions = 4.0f * (float)PI * FMath::Square(Radius) / 1000000.0f;
	const float ConeSolidAngle = 2.0f * float(PI) * (1.0f - CosOuterConeAngle);
	// Find the fraction of the sphere's surface area that is inside the cone
	const float ConeSurfaceAreaSphereFraction = ConeSolidAngle / (4.0f * (float)PI);
	// Gather enough photons to meet DirectPhotonDensity on the spherical cap at the influence radius of the spot light.
	const int32 NumDirectPhotons = FMath::TruncToInt(InfluenceSphereSurfaceAreaMillions * ConeSurfaceAreaSphereFraction * DirectPhotonDensity);
	return NumDirectPhotons == appTruncErrorCode ? INT_MAX : NumDirectPhotons;
}

/** Generates a direction sample from the light's domain */
void FSpotLight::SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const
{
	FVector4f XAxis(0,0,0);
	FVector4f YAxis(0,0,0);
	GenerateCoordinateSystem(Direction, XAxis, YAxis);

	//@todo - the PDF should be affected by inner cone angle too
	const FVector4f ConeSampleDirection = UniformSampleCone(RandomStream, CosOuterConeAngle, XAxis, YAxis, Direction);

	//@todo - take light source radius into account
	SampleRay = FLightRay(
		Position,
		Position + ConeSampleDirection * Radius,
		NULL,
		this
		);

	LightSourceNormal = Direction;

	RayPDF = UniformConePDF(CosOuterConeAngle);
	checkSlow(RayPDF > 0.0f);
	Power = IndirectColor * Brightness * PointLightIntensityScale;
}

/** Generates a direction sample from the light based on the given rays */
void FSpotLight::SampleDirection(
	const TArray<FIndirectPathRay>& IndirectPathRays, 
	FLMRandomStream& RandomStream, 
	FLightRay& SampleRay, 
	float& RayPDF, 
	FLinearColor& Power) const
{
	FVector4f Unused;
	FVector2f Unused2;
	FSpotLight::SampleDirection(RandomStream, SampleRay, Unused, Unused2, RayPDF, Power);
}


//----------------------------------------------------------------------------
//	Rect light class
//----------------------------------------------------------------------------
void FRectLight::Import( FLightmassImporter& Importer )
{
	FPointLight::Import( Importer );

	Importer.ImportData( (FRectLightData*)this );

#if 0
	uint32 NumPixels = 0;
	if( SourceTextureSizeX + SourceTextureSizeY > 0 )
	{
		uint32 SizeX = SourceTextureSizeX;
		uint32 SizeY = SourceTextureSizeY;
		while( SizeX > 1 || SizeY > 1 )
		{
			NumPixels += SizeX * SizeY;

			SizeX = SizeX > 1 ? SizeX >> 1 : 1;
			SizeY = SizeY > 1 ? SizeY >> 1 : 1;
		}
	}
	Importer.ImportArray( SourceTexture, NumPixels );
#endif
}


FVector3f PolygonIrradiance( FVector3f Poly[4] )
{
	FVector3f L0 = Poly[0].GetSafeNormal();
	FVector3f L1 = Poly[1].GetSafeNormal();
	FVector3f L2 = Poly[2].GetSafeNormal();
	FVector3f L3 = Poly[3].GetSafeNormal();

	float c01 = L0 | L1;
	float c12 = L1 | L2;
	float c23 = L2 | L3;
	float c30 = L3 | L0;

	float w01 = ( 1.5708f - 0.175f * c01 ) * FMath::InvSqrt( c01 + 1.0f );
	float w12 = ( 1.5708f - 0.175f * c12 ) * FMath::InvSqrt( c12 + 1.0f );
	float w23 = ( 1.5708f - 0.175f * c23 ) * FMath::InvSqrt( c23 + 1.0f );
	float w30 = ( 1.5708f - 0.175f * c30 ) * FMath::InvSqrt( c30 + 1.0f );

	FVector3f L;
	L  = L1 ^ ( -w01 * L0 +  w12 * L2 );
	L += L3 ^ (  w30 * L0 + -w23 * L2 );

	// Vector irradiance
	return 0.5f * L;
}

/**
 * Computes the intensity of the direct lighting from this light on a specific point.
 */
FLinearColor FRectLight::GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const
{
	FVector3f ToLight = Position - Point;

	FVector3f AxisY = GetLightTangent();
	FVector3f AxisZ = -Direction;
	FVector3f AxisX = AxisY ^ AxisZ;
	FVector2f Extent(
		LightSourceRadius,
		LightSourceLength
	);

	if( ( ToLight | AxisZ ) < 0.0f )
		return FLinearColor::Black;

	FVector3f Poly[4];
	Poly[0] = ToLight - AxisX * Extent.X - AxisY * Extent.Y;
	Poly[1] = ToLight + AxisX * Extent.X - AxisY * Extent.Y;
	Poly[2] = ToLight + AxisX * Extent.X + AxisY * Extent.Y;
	Poly[3] = ToLight - AxisX * Extent.X + AxisY * Extent.Y;

	float DistanceAttenuation = PolygonIrradiance( Poly ).Size();

	if (!FMath::IsFinite(DistanceAttenuation))
	{
		DistanceAttenuation = 0;
	}
			
	// TODO Move CPU cide
	DistanceAttenuation /= 2 * Extent.X * Extent.Y;

	float DistanceSqr = ToLight.SizeSquared();
	float LightRadiusMask = FMath::Square(FMath::Max(0.0f, 1.0f - FMath::Square(DistanceSqr / (Radius * Radius))));
	DistanceAttenuation *= LightRadiusMask;

	return FLight::GetDirectIntensity(Point, bCalculateForIndirectLighting) * DistanceAttenuation * SourceTextureAvgColor;
}

/** Returns the number of direct photons to gather required by this light. */
int32 FRectLight::GetNumDirectPhotons(float DirectPhotonDensity) const
{
	const float InfluenceSphereSurfaceAreaMillions = 4.0f * (float)PI * FMath::Square(Radius) / 1000000.0f;
	// Find the fraction of the sphere's surface area that is inside the cone
	const float SurfaceAreaSphereFraction = 0.25f;
	// Gather enough photons to meet DirectPhotonDensity on the spherical cap at the influence radius of the spot light.
	const int32 NumDirectPhotons = FMath::TruncToInt(InfluenceSphereSurfaceAreaMillions * SurfaceAreaSphereFraction * DirectPhotonDensity);
	return NumDirectPhotons == appTruncErrorCode ? INT_MAX : NumDirectPhotons;
}

/** Validates a surface sample given the position that sample is affecting. */
void FRectLight::ValidateSurfaceSample(const FVector4f& Point, FLightSurfaceSample& Sample) const
{}

FVector4f FRectLight::LightCenterPosition(const FVector4f& ReceivingPosition, const FVector4f& ReceivingNormal) const
{
	FVector4f ToLight = Position - ReceivingPosition;

	FVector3f AxisY = GetLightTangent();
	FVector3f AxisZ = -Direction;
	FVector3f AxisX = AxisY ^ AxisZ;
	FVector2f Extent(
		LightSourceRadius,
		LightSourceLength
	);

#if 0
	FVector3f Poly[4];
	Poly[0] = ToLight - AxisX * Extent.X - AxisY * Extent.Y;
	Poly[1] = ToLight + AxisX * Extent.X - AxisY * Extent.Y;
	Poly[2] = ToLight + AxisX * Extent.X + AxisY * Extent.Y;
	Poly[3] = ToLight - AxisX * Extent.X + AxisY * Extent.Y;

	FVector3f AvgDirection = PolygonIrradiance( Poly ).GetSafeNormal();

	// Intersect ray with plane
	float Distance = Dot3( AxisZ, ToLight ) / Dot3( AxisZ, AvgDirection );
	return ReceivingPosition + Distance * AvgDirection;
#else
	float RectLocalX = FMath::Clamp( Dot3( AxisX, -ToLight ), -Extent.X, Extent.X );
	float RectLocalY = FMath::Clamp( Dot3( AxisY, -ToLight ), -Extent.Y, Extent.Y );

	FVector4f ClosestPoint = ToLight;
	ClosestPoint += AxisX * RectLocalX;
	ClosestPoint += AxisY * RectLocalY;
	//return ClosestPoint + ReceivingPosition;

	FVector4f OppositePoint = 2.0f * ToLight - ClosestPoint;

	FVector4f L0 = ClosestPoint.GetSafeNormal();
	FVector4f L1 = OppositePoint.GetSafeNormal();
	FVector4f L  = ( L0 + L1 ).GetSafeNormal();

	// Intersect ray with plane
	float Distance = Dot3( AxisZ, ToLight ) / Dot3( AxisZ, L );
	return ReceivingPosition + L * Distance;
#endif
}

bool FRectLight::AffectsBounds(const FBoxSphereBounds3f& Bounds) const
{
	FPlane4f Plane( Position, Direction );
	float DistanceToPlane = ( Bounds.Origin - Position ) | Direction;
	if( DistanceToPlane < -Bounds.SphereRadius )
	{
		return false;
	}

	if(!FPointLight::AffectsBounds(Bounds))
	{
		return false;
	}

	return true;
}

/** Returns true if all parts of the light are behind the surface being tested. */
bool FRectLight::BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const
{
	return false;
}

/** Gets a single direction to use for direct lighting that is representative of the whole area light. */
FVector4f FRectLight::GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const
{
	FVector4f ToLight = Position - Point;

	FVector3f AxisY = GetLightTangent();
	FVector3f AxisZ = -Direction;
	FVector3f AxisX = AxisY ^ AxisZ;
	FVector2f Extent(
		LightSourceRadius,
		LightSourceLength
	);

	FVector3f Poly[4];
	Poly[0] = ToLight - AxisX * Extent.X - AxisY * Extent.Y;
	Poly[1] = ToLight + AxisX * Extent.X - AxisY * Extent.Y;
	Poly[2] = ToLight + AxisX * Extent.X + AxisY * Extent.Y;
	Poly[3] = ToLight - AxisX * Extent.X + AxisY * Extent.Y;

	return PolygonIrradiance( Poly );
}

/** Generates a sample on the light's surface. */
void FRectLight::SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const
{
	Sample.DiskPosition = FVector2f(0, 0);

	FVector3f AxisY = GetLightTangent();
	FVector3f AxisZ = -Direction;
	FVector3f AxisX = AxisY ^ AxisZ;
	FVector2f Extent(
		LightSourceRadius,
		LightSourceLength
	);

	float UnitX = RandomStream.GetFraction() * 2.0f - 1.0f;
	float UnitY = RandomStream.GetFraction() * 2.0f - 1.0f;

	Sample.Position = Position + AxisX * Extent.X * UnitX + AxisY * Extent.Y * UnitY;
	Sample.Normal = -AxisZ;
	// Probability of generating this surface position is 1 / SurfaceArea
	Sample.PDF = 1.0f / (4.0f * Extent.X * Extent.Y);
}

/** Generates a direction sample from the light's domain */
void FRectLight::SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const
{
	FVector3f AxisY = GetLightTangent();
	FVector3f AxisZ = -Direction;
	FVector3f AxisX = AxisY ^ AxisZ;
	FVector2f Extent(
		LightSourceRadius,
		LightSourceLength
	);

	float UnitX = RandomStream.GetFraction() * 2.0f - 1.0f;
	float UnitY = RandomStream.GetFraction() * 2.0f - 1.0f;

	FVector4f SamplePosition = Position + AxisX * Extent.X * UnitX + AxisY * Extent.Y * UnitY;

	FVector4f SampleDirection = GetCosineHemisphereVector( RandomStream );
	RayPDF = SampleDirection.Z / PI;

	SampleDirection =
		SampleDirection.X * AxisX +
		SampleDirection.Y * AxisY +
		SampleDirection.Z * AxisZ;

	SampleRay = FLightRay(
		SamplePosition,
		SamplePosition - SampleDirection * Radius,
		NULL,
		this
		);

	LightSourceNormal = -AxisZ;

	Power = SourceTextureAvgColor * IndirectColor * Brightness;
}

/** Generates a direction sample from the light based on the given rays */
void FRectLight::SampleDirection(
	const TArray<FIndirectPathRay>& IndirectPathRays, 
	FLMRandomStream& RandomStream, 
	FLightRay& SampleRay, 
	float& RayPDF, 
	FLinearColor& Power) const
{
	FVector4f Unused;
	FVector2f Unused2;
	FRectLight::SampleDirection(RandomStream, SampleRay, Unused, Unused2, RayPDF, Power);
}


//----------------------------------------------------------------------------
//	Sky light class
//----------------------------------------------------------------------------
void FSkyLight::Import( FLightmassImporter& Importer )
{
	FLight::Import( Importer );

	Importer.ImportData( (FSkyLightData*)this );

	TArray<FFloat16Color> RadianceEnvironmentMap;
	Importer.ImportArray(RadianceEnvironmentMap, RadianceEnvironmentMapDataSize);

	CubemapSize = FMath::Sqrt(RadianceEnvironmentMapDataSize / 6.f);
	NumMips = FMath::CeilLogTwo(CubemapSize) + 1;

	check(FMath::IsPowerOfTwo(CubemapSize));
	check(NumMips > 0);
	check(RadianceEnvironmentMapDataSize == CubemapSize * CubemapSize * 6);

	if (bUseFilteredCubemap && CubemapSize > 0)
	{
		const double StartTime = FPlatformTime::Seconds();

		PrefilteredRadiance.Empty(NumMips);
		PrefilteredRadiance.AddZeroed(NumMips);

		PrefilteredRadiance[0].Empty(CubemapSize * CubemapSize * 6);
		PrefilteredRadiance[0].AddZeroed(CubemapSize * CubemapSize * 6);

		for (int32 TexelIndex = 0; TexelIndex < CubemapSize * CubemapSize * 6; TexelIndex++)
		{
			FLinearColor Lighting = FLinearColor(RadianceEnvironmentMap[TexelIndex]);
			PrefilteredRadiance[0][TexelIndex] = Lighting;
		}

		FIntPoint SubCellOffsets[4] =
		{
			FIntPoint(0, 0),
			FIntPoint(1, 0),
			FIntPoint(0, 1),
			FIntPoint(1, 1)
		};

		const float SubCellWeight = 1.0f / (float)UE_ARRAY_COUNT(SubCellOffsets);

		for (int32 MipIndex = 1; MipIndex < NumMips; MipIndex++)
		{
			const int32 MipSize = 1 << (NumMips - MipIndex - 1);
			const int32 ParentMipSize = MipSize * 2;
			const int32 CubeFaceSize = MipSize * MipSize;

			PrefilteredRadiance[MipIndex].Empty(CubeFaceSize * 6);
			PrefilteredRadiance[MipIndex].AddZeroed(CubeFaceSize * 6);

			for (int32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
			{
				for (int32 Y = 0; Y < MipSize; Y++)
				{
					for (int32 X = 0; X < MipSize; X++)
					{
						FLinearColor FilteredValue(0, 0, 0, 0);

						for (int32 OffsetIndex = 0; OffsetIndex < UE_ARRAY_COUNT(SubCellOffsets); OffsetIndex++)
						{
							FIntPoint ParentOffset = FIntPoint(X, Y) * 2 + SubCellOffsets[OffsetIndex];
							int32 ParentTexelIndex = FaceIndex * ParentMipSize * ParentMipSize + ParentOffset.Y * ParentMipSize + ParentOffset.X;
							FLinearColor ParentLighting = PrefilteredRadiance[MipIndex - 1][ParentTexelIndex];
							FilteredValue += ParentLighting;
						}
					
						FilteredValue *= SubCellWeight;

						PrefilteredRadiance[MipIndex][FaceIndex * CubeFaceSize + Y * MipSize + X] = FilteredValue;
					}
				}
			}
		}

		ComputePrefilteredVariance();

		const double EndTime = FPlatformTime::Seconds();
		UE_LOGF(LogLightmass, Log, "Skylight import processing %.3fs with CubemapSize %u", (float)(EndTime - StartTime), CubemapSize);
	}
}

void FSkyLight::ComputePrefilteredVariance()
{
	PrefilteredVariance.Empty(NumMips);
	PrefilteredVariance.AddZeroed(NumMips);
	
	TArray<float> TempMaxVariance;
	TempMaxVariance.Empty(NumMips);
	TempMaxVariance.AddZeroed(NumMips);

	for (int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
	{
		const int32 MipSize = 1 << (NumMips - MipIndex - 1);
		const int32 CubeFaceSize = MipSize * MipSize;
		const int32 BaseMipTexelSize = CubemapSize / MipSize;
		const float NormalizeFactor = 1.0f / FMath::Max(BaseMipTexelSize * BaseMipTexelSize - 1, 1);

		PrefilteredVariance[MipIndex].Empty(CubeFaceSize * 6);
		PrefilteredVariance[MipIndex].AddZeroed(CubeFaceSize * 6);

		for (int32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
		{
			for (int32 Y = 0; Y < MipSize; Y++)
			{
				for (int32 X = 0; X < MipSize; X++)
				{
					int32 TexelIndex = FaceIndex * CubeFaceSize + Y * MipSize + X;

					float Mean = PrefilteredRadiance[MipIndex][TexelIndex].GetLuminance();

					int32 BaseTexelOffset = FaceIndex * CubemapSize * CubemapSize + X * BaseMipTexelSize + Y * BaseMipTexelSize * CubemapSize;
					float SumOfSquares = 0;

					//@todo - implement in terms of the previous mip level, not the bottom mip level
					for (int32 BaseY = 0; BaseY < BaseMipTexelSize; BaseY++)
					{
						for (int32 BaseX = 0; BaseX < BaseMipTexelSize; BaseX++)
						{
							int32 BaseTexelIndex = BaseTexelOffset + BaseY * CubemapSize + BaseX;
							float BaseValue = PrefilteredRadiance[0][BaseTexelIndex].GetLuminance();

							SumOfSquares += (BaseValue - Mean) * (BaseValue - Mean);
						}
					}

					PrefilteredVariance[MipIndex][TexelIndex] = SumOfSquares * NormalizeFactor;
					TempMaxVariance[MipIndex] = FMath::Max(TempMaxVariance[MipIndex], SumOfSquares * NormalizeFactor);
				}
			}
		}
	}
}

FLinearColor FSkyLight::SampleRadianceCubemap(float Mip, int32 CubeFaceIndex, FVector2f FaceUV) const
{
	checkSlow(bUseFilteredCubemap);
	FLinearColor HighMipRadiance;
	{
		const int32 MipIndex = FMath::CeilToInt(Mip);
		const int32 MipSize = 1 << (NumMips - MipIndex - 1);
		const int32 CubeFaceSize = MipSize * MipSize;
		FIntPoint FaceCoordinate(FaceUV.X * MipSize, FaceUV.Y * MipSize);
		check(FaceCoordinate.X >= 0 && FaceCoordinate.X < MipSize);
		check(FaceCoordinate.Y >= 0 && FaceCoordinate.Y < MipSize);
		HighMipRadiance = PrefilteredRadiance[MipIndex][CubeFaceIndex * CubeFaceSize + FaceCoordinate.Y * MipSize + FaceCoordinate.X];
	}

	FLinearColor LowMipRadiance;
	{
		const int32 MipIndex = FMath::FloorToInt(Mip);
		const int32 MipSize = 1 << (NumMips - MipIndex - 1);
		const int32 CubeFaceSize = MipSize * MipSize;
		FIntPoint FaceCoordinate(FaceUV.X * MipSize, FaceUV.Y * MipSize);
		check(FaceCoordinate.X >= 0 && FaceCoordinate.X < MipSize);
		check(FaceCoordinate.Y >= 0 && FaceCoordinate.Y < MipSize);
		LowMipRadiance = PrefilteredRadiance[MipIndex][CubeFaceIndex * CubeFaceSize + FaceCoordinate.Y * MipSize + FaceCoordinate.X];
	}

	return FMath::Lerp(LowMipRadiance, HighMipRadiance, FMath::Fractional(Mip));
}

float FSkyLight::SampleVarianceCubemap(float Mip, int32 CubeFaceIndex, FVector2f FaceUV) const
{
	checkSlow(bUseFilteredCubemap);
	float HighMipVariance;
	{
		const int32 MipIndex = FMath::CeilToInt(Mip);
		const int32 MipSize = 1 << (NumMips - MipIndex - 1);
		const int32 CubeFaceSize = MipSize * MipSize;
		FIntPoint FaceCoordinate(FaceUV.X * MipSize, FaceUV.Y * MipSize);
		check(FaceCoordinate.X >= 0 && FaceCoordinate.X < MipSize);
		check(FaceCoordinate.Y >= 0 && FaceCoordinate.Y < MipSize);
		HighMipVariance = PrefilteredVariance[MipIndex][CubeFaceIndex * CubeFaceSize + FaceCoordinate.Y * MipSize + FaceCoordinate.X];
	}

	float LowMipVariance;

	{
		const int32 MipIndex = FMath::FloorToInt(Mip);
		const int32 MipSize = 1 << (NumMips - MipIndex - 1);
		const int32 CubeFaceSize = MipSize * MipSize;
		FIntPoint FaceCoordinate(FaceUV.X * MipSize, FaceUV.Y * MipSize);
		check(FaceCoordinate.X >= 0 && FaceCoordinate.X < MipSize);
		check(FaceCoordinate.Y >= 0 && FaceCoordinate.Y < MipSize);
		LowMipVariance = PrefilteredVariance[MipIndex][CubeFaceIndex * CubeFaceSize + FaceCoordinate.Y * MipSize + FaceCoordinate.X];
	}

	return FMath::Lerp(LowMipVariance, HighMipVariance, FMath::Fractional(Mip));
}

void GetCubeFaceAndUVFromDirection(const FVector4f& IncomingDirection, int32& CubeFaceIndex, FVector2f& FaceUVs)
{
	FVector3f AbsIncomingDirection(FMath::Abs(IncomingDirection.X), FMath::Abs(IncomingDirection.Y), FMath::Abs(IncomingDirection.Z));

	int32 LargestChannelIndex = 0;

	if (AbsIncomingDirection.Y > AbsIncomingDirection.X)
	{
		LargestChannelIndex = 1;
	}

	if (AbsIncomingDirection.Z > AbsIncomingDirection.Y && AbsIncomingDirection.Z > AbsIncomingDirection.X)
	{
		LargestChannelIndex = 2;
	}

	CubeFaceIndex = LargestChannelIndex * 2 + (IncomingDirection[LargestChannelIndex] < 0 ? 1 : 0);

	if (CubeFaceIndex == 0)
	{
		FaceUVs = FVector2f(-IncomingDirection.Z, -IncomingDirection.Y);
		//CubeCoordinates = float3(1, -ScaledUVs.y, -ScaledUVs.x);
	}
	else if (CubeFaceIndex == 1)
	{
		FaceUVs = FVector2f(IncomingDirection.Z, -IncomingDirection.Y);
		//CubeCoordinates = float3(-1, -ScaledUVs.y, ScaledUVs.x);
	}
	else if (CubeFaceIndex == 2)
	{
		FaceUVs = FVector2f(IncomingDirection.X, IncomingDirection.Z);
		//CubeCoordinates = float3(ScaledUVs.x, 1, ScaledUVs.y);
	}
	else if (CubeFaceIndex == 3)
	{
		FaceUVs = FVector2f(IncomingDirection.X, -IncomingDirection.Z);
		//CubeCoordinates = float3(ScaledUVs.x, -1, -ScaledUVs.y);
	}
	else if (CubeFaceIndex == 4)
	{
		FaceUVs = FVector2f(IncomingDirection.X, -IncomingDirection.Y);
		//CubeCoordinates = float3(ScaledUVs.x, -ScaledUVs.y, 1);
	}
	else
	{
		FaceUVs = FVector2f(-IncomingDirection.X, -IncomingDirection.Y);
		//CubeCoordinates = float3(-ScaledUVs.x, -ScaledUVs.y, -1);
	}

	FaceUVs = FaceUVs / AbsIncomingDirection[LargestChannelIndex] * .5f + .5f;

	// When exactly on the edge of two faces, snap to the nearest addressable texel
	FaceUVs.X = FMath::Min(FaceUVs.X, .999f);
	FaceUVs.Y = FMath::Min(FaceUVs.Y, .999f);
}

float FSkyLight::GetMipIndexForSolidAngle(float SolidAngle) const
{
	//@todo - corners of the cube should use a different mip
	const float AverageTexelSolidAngle = 4 * PI / (6 * CubemapSize * CubemapSize) * 2;
	float Mip = 0.5 * FMath::Log2(SolidAngle / AverageTexelSolidAngle);
	return FMath::Clamp<float>(Mip, 0.0f, NumMips - 1);
}

FLinearColor FSkyLight::GetPathLighting(const FVector4f& IncomingDirection, float PathSolidAngle, bool bCalculateForIndirectLighting) const
{
	if (CubemapSize == 0)
	{
		return FLinearColor::Black;
	}

	FLinearColor Lighting = FLinearColor::Black;

	if (bUseFilteredCubemap)
	{
		int32 CubeFaceIndex;
		FVector2f FaceUVs;
		GetCubeFaceAndUVFromDirection(IncomingDirection, CubeFaceIndex, FaceUVs);

		const float MipIndex = GetMipIndexForSolidAngle(PathSolidAngle);

		Lighting = SampleRadianceCubemap(MipIndex, CubeFaceIndex, FaceUVs);
	}
	else
	{
		FSHVector3 SH = FSHVector3::SHBasisFunction(FVector4(IncomingDirection));
		Lighting = Dot(IrradianceEnvironmentMap, SH);
	}

	const float LightingScale = bCalculateForIndirectLighting ? IndirectLightingScale : 1.0f;
	Lighting = (Lighting * Brightness * LightingScale) * FLinearColor(Color);

	Lighting.R = FMath::Max(Lighting.R, 0.0f);
	Lighting.G = FMath::Max(Lighting.G, 0.0f);
	Lighting.B = FMath::Max(Lighting.B, 0.0f);

	return Lighting;
}

float FSkyLight::GetPathVariance(const FVector4f& IncomingDirection, float PathSolidAngle) const
{
	if (CubemapSize == 0 || !bUseFilteredCubemap)
	{
		return 0;
	}

	int32 CubeFaceIndex;
	FVector2f FaceUVs;
	GetCubeFaceAndUVFromDirection(IncomingDirection, CubeFaceIndex, FaceUVs);

	const float MipIndex = GetMipIndexForSolidAngle(PathSolidAngle);
	return SampleVarianceCubemap(MipIndex, CubeFaceIndex, FaceUVs);
}

void FMeshLightPrimitive::AddSubPrimitive(const FTexelToCorners& TexelToCorners, const FIntPoint& Coordinates, const FLinearColor& InTexelPower, float NormalOffset)
{
	const FVector4f FirstTriangleNormal = (TexelToCorners.Corners[0].WorldPosition - TexelToCorners.Corners[1].WorldPosition) ^ (TexelToCorners.Corners[2].WorldPosition - TexelToCorners.Corners[1].WorldPosition);
	const float FirstTriangleArea = .5f * FirstTriangleNormal.Size3();
	const FVector4f SecondTriangleNormal = (TexelToCorners.Corners[2].WorldPosition - TexelToCorners.Corners[1].WorldPosition) ^ (TexelToCorners.Corners[2].WorldPosition - TexelToCorners.Corners[3].WorldPosition);
	const float SecondTriangleArea = .5f * SecondTriangleNormal.Size3();
	const float SubPrimitiveSurfaceArea = FirstTriangleArea + SecondTriangleArea;
	// Convert power per texel into power per texel surface area
	const FLinearColor SubPrimitivePower = InTexelPower * SubPrimitiveSurfaceArea;
	
	// If this is the first sub primitive, initialize
	if (NumSubPrimitives == 0)
	{
		SurfaceNormal = TexelToCorners.WorldTangentZ;
		const FVector4f OffsetAmount = NormalOffset * TexelToCorners.WorldTangentZ;
		for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
		{
			Corners[CornerIndex].WorldPosition = TexelToCorners.Corners[CornerIndex].WorldPosition + OffsetAmount;
			Corners[CornerIndex].FurthestCoordinates = Coordinates;
		}

		SurfaceArea = SubPrimitiveSurfaceArea;
		Power = SubPrimitivePower;
	}
	else
	{
		// Average sub primitive normals
		SurfaceNormal += TexelToCorners.WorldTangentZ;

		// Directions corresponding to CornerOffsets in FStaticLightingSystem::CalculateTexelCorners
		static const FIntPoint CornerDirections[NumTexelCorners] = 
		{
			FIntPoint(-1, -1),
			FIntPoint(1, -1),
			FIntPoint(-1, 1),
			FIntPoint(1, 1)
		};

		const FVector4f OffsetAmount = NormalOffset * TexelToCorners.WorldTangentZ;
		for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
		{
			const FIntPoint& ExistingFurthestCoordinates = Corners[CornerIndex].FurthestCoordinates;
			// Store the new position if this coordinate is greater or equal to the previous coordinate for this corner in texture space, in the direction of the corner.
			if (CornerDirections[CornerIndex].X * (Coordinates.X - ExistingFurthestCoordinates.X) >=0 
				&& CornerDirections[CornerIndex].Y * (Coordinates.Y - ExistingFurthestCoordinates.Y) >=0)
			{
				Corners[CornerIndex].WorldPosition = TexelToCorners.Corners[CornerIndex].WorldPosition + OffsetAmount;
				Corners[CornerIndex].FurthestCoordinates = Coordinates;
			}
		}

		// Accumulate the area and power that this simplified primitive represents
		SurfaceArea += SubPrimitiveSurfaceArea;
		Power += SubPrimitivePower;
	}
	NumSubPrimitives++;
}

void FMeshLightPrimitive::Finalize()
{
	SurfaceNormal = SurfaceNormal.SizeSquared3() > SMALL_NUMBER ? SurfaceNormal.GetUnsafeNormal3() : FVector4f(0, 0, 1);
}

//----------------------------------------------------------------------------
//	Mesh Area Light class
//----------------------------------------------------------------------------

void FMeshAreaLight::Initialize(float InIndirectPhotonEmitConeAngle, const FBoxSphereBounds3f& InImportanceBounds)
{
	CosIndirectPhotonEmitConeAngle = FMath::Cos(InIndirectPhotonEmitConeAngle);
	ImportanceBounds = InImportanceBounds;
}

/** Returns the number of direct photons to gather required by this light. */
int32 FMeshAreaLight::GetNumDirectPhotons(float DirectPhotonDensity) const
{
	// Gather enough photons to meet DirectPhotonDensity at the influence radius of the mesh area light.
	// Clamp the influence radius to the importance or scene radius for the purposes of emitting photons
	// This prevents huge mesh area lights from emitting more photons than are needed
	const float InfluenceSphereSurfaceAreaMillions = 4.0f * (float)PI * FMath::Square(FMath::Min(ImportanceBounds.SphereRadius, InfluenceRadius)) / 1000000.0f;
	const int32 NumDirectPhotons = FMath::TruncToInt(InfluenceSphereSurfaceAreaMillions * DirectPhotonDensity);
	return NumDirectPhotons == appTruncErrorCode ? INT_MAX : NumDirectPhotons;
}

/** Initializes the mesh area light with primitives */
void FMeshAreaLight::SetPrimitives(
	const TArray<FMeshLightPrimitive>& InPrimitives, 
	float EmissiveLightFalloffExponent, 
	float EmissiveLightExplicitInfluenceRadius,
	int32 InMeshAreaLightGridSize,
	FGuid InLevelGuid)
{
	check(InPrimitives.Num() > 0);
	Primitives = InPrimitives;
	MeshAreaLightGridSize = InMeshAreaLightGridSize;
	LevelGuid = InLevelGuid;
	TotalSurfaceArea = 0.0f;
	TotalPower = FLinearColor::Black;
	Position = FVector4f(0,0,0);
	FBox3f Bounds(ForceInit);

	CachedPrimitiveNormals.Empty(MeshAreaLightGridSize * MeshAreaLightGridSize);
	CachedPrimitiveNormals.AddZeroed(MeshAreaLightGridSize * MeshAreaLightGridSize);
	PrimitivePDFs.Empty(Primitives.Num());
	for (int32 PrimitiveIndex = 0; PrimitiveIndex < Primitives.Num(); PrimitiveIndex++)
	{
		const FMeshLightPrimitive& CurrentPrimitive = Primitives[PrimitiveIndex];
		TotalSurfaceArea += CurrentPrimitive.SurfaceArea;
		TotalPower += CurrentPrimitive.Power;
		PrimitivePDFs.Add(CurrentPrimitive.SurfaceArea);
		for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
		{
			Bounds += CurrentPrimitive.Corners[CornerIndex].WorldPosition;
		}
		const FVector2f SphericalCoordinates = FVector3f(CurrentPrimitive.SurfaceNormal).UnitCartesianToSpherical();
		// Determine grid cell the primitive's normal falls into based on spherical coordinates
		const int32 CacheX = FMath::Clamp(FMath::TruncToInt(SphericalCoordinates.X / (float)PI * MeshAreaLightGridSize), 0, MeshAreaLightGridSize - 1);
		const int32 CacheY = FMath::Clamp(FMath::TruncToInt((SphericalCoordinates.Y + (float)PI) / (2 * (float)PI) * MeshAreaLightGridSize), 0, MeshAreaLightGridSize - 1);
		CachedPrimitiveNormals[CacheY * MeshAreaLightGridSize + CacheX].Add(CurrentPrimitive.SurfaceNormal);
	}

	for (int32 PhiStep = 0; PhiStep < MeshAreaLightGridSize; PhiStep++)
	{
		for (int32 ThetaStep = 0; ThetaStep < MeshAreaLightGridSize; ThetaStep++)
		{
			const TArray<FVector4f>& CurrentCachedNormals = CachedPrimitiveNormals[PhiStep * MeshAreaLightGridSize + ThetaStep];
			if (CurrentCachedNormals.Num() > 0)
			{
				OccupiedCachedPrimitiveNormalCells.Add(FIntPoint(ThetaStep, PhiStep));
			}
		}
	}
	
	// Compute the Cumulative Distribution Function for our step function of primitive surface areas
	CalculateStep1dCDF(PrimitivePDFs, PrimitiveCDFs, UnnormalizedIntegral);

	SourceBounds = FBoxSphereBounds3f(Bounds);
	Position = SourceBounds.Origin;
	Position.W = 1.0f;
	check(TotalSurfaceArea > 0.0f);
	check(TotalPower.R > 0.0f || TotalPower.G > 0.0f || TotalPower.B > 0.0f);
	// The irradiance value at which to place the light's influence radius
	const float IrradianceCutoff = .002f;
	// If EmissiveLightExplicitInfluenceRadius is 0, automatically generate the influence radius based on the light's power
	// Solve Irradiance = Power / Distance ^2 for Radius
	//@todo - should the SourceBounds also factor into the InfluenceRadius calculation?
	InfluenceRadius = EmissiveLightExplicitInfluenceRadius > DELTA ? EmissiveLightExplicitInfluenceRadius : FMath::Sqrt(FLinearColorUtils::LinearRGBToXYZ(TotalPower).G / IrradianceCutoff);
	FalloffExponent = EmissiveLightFalloffExponent;
	// Using the default for point lights
	ShadowExponent = 2.0f;
}

/**
 * Tests whether the light affects the given bounding volume.
 * @param Bounds - The bounding volume to test.
 * @return True if the light affects the bounding volume
 */
bool FMeshAreaLight::AffectsBounds(const FBoxSphereBounds3f& Bounds) const
{
	if((Bounds.Origin - Position).SizeSquared() > FMath::Square(InfluenceRadius + Bounds.SphereRadius + SourceBounds.SphereRadius))
	{
		return false;
	}

	if(!FLight::AffectsBounds(Bounds))
	{
		return false;
	}

	return true;
}

/**
 * Computes the intensity of the direct lighting from this light on a specific point.
 */
FLinearColor FMeshAreaLight::GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const
{
	FLinearColor AccumulatedPower(ForceInit);
	float AccumulatedSurfaceArea = 0.0f;
	for (int32 PrimitiveIndex = 0; PrimitiveIndex < Primitives.Num(); PrimitiveIndex++)
	{
		const FMeshLightPrimitive& CurrentPrimitive = Primitives[PrimitiveIndex];
		FVector4f PrimitiveCenter(0,0,0);
		for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
		{
			PrimitiveCenter += CurrentPrimitive.Corners[CornerIndex].WorldPosition / 4.0f;
		}
		const FVector4f LightVector = (Point - PrimitiveCenter).GetSafeNormal();
		const float NDotL = Dot3(LightVector, CurrentPrimitive.SurfaceNormal);
		if (NDotL >= 0)
		{
			// Using standard Unreal attenuation for point lights for each primitive
			const float RadialAttenuation = FMath::Pow(FMath::Max(1.0f - ((PrimitiveCenter - Point) / InfluenceRadius).SizeSquared3(), 0.0f), (FVector4f::FReal)FalloffExponent);
			// Weight exitant power by the distance attenuation to this primitive and the light's cosine distribution around the primitive's normal
			//@todo - photon emitting does not take the cosine distribution into account
			AccumulatedPower += CurrentPrimitive.Power * RadialAttenuation * NDotL;
		}
	}
	return AccumulatedPower / TotalSurfaceArea * (bCalculateForIndirectLighting ? IndirectLightingScale : 1.0f);
}

/** Returns an intensity scale based on the receiving point. */
float FMeshAreaLight::CustomAttenuation(const FVector4f& Point, FLMRandomStream& RandomStream, bool bMaintainEvenDensity) const
{
	const float FullProbabilityDistance = .5f * InfluenceRadius;
	float PowerWeightedAttenuation = 0.0f;
	float PowerWeightedPhysicalAttenuation = 0.0f;
	float DepositProbability = 0.0f;
	for (int32 PrimitiveIndex = 0; PrimitiveIndex < Primitives.Num(); PrimitiveIndex++)
	{
		const FMeshLightPrimitive& CurrentPrimitive = Primitives[PrimitiveIndex];
		FVector4f PrimitiveCenter(0,0,0);
		for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
		{
			PrimitiveCenter += CurrentPrimitive.Corners[CornerIndex].WorldPosition / 4.0f;
		}
		const float NDotL = Dot3((Point - PrimitiveCenter), CurrentPrimitive.SurfaceNormal);
		if (NDotL >= 0)
		{
			const float RadialAttenuation = FMath::Pow(FMath::Max(1.0f - ((PrimitiveCenter - Point) / InfluenceRadius).SizeSquared3(), 0.0f), (FVector4f::FReal)FalloffExponent);
			const float PowerWeight = FLinearColorUtils::LinearRGBToXYZ(CurrentPrimitive.Power).G;
			// Weight the attenuation factors by how much power this primitive emits, and its distance attenuation
			PowerWeightedAttenuation += PowerWeight * RadialAttenuation;
			// Also accumulate physical attenuation
			const float DistanceSquared = (PrimitiveCenter - Point).SizeSquared3();
			PowerWeightedPhysicalAttenuation += PowerWeight / DistanceSquared;
			DepositProbability += CurrentPrimitive.SurfaceArea / TotalSurfaceArea * FMath::Min(DistanceSquared / (FullProbabilityDistance * FullProbabilityDistance), 1.0f);
		}
	}

	DepositProbability = FMath::Clamp(DepositProbability, 0.0f, 1.0f);

	if (!bMaintainEvenDensity)
	{
		DepositProbability = 1.0f;
	}

	// Thin out photons near the light source.
	// This is partly an optimization since the photon density near light sources doesn't need to be high, and the natural 1 / R^2 density is overkill, 
	// But this also improves quality since we are doing a nearest N photon neighbor search when calculating irradiance.  
	// If the photon map has a high density of low power photons near light sources,
	// Combined with sparse, high power photons from other light sources (directional lights for example), the result will be very splotchy.
	if (RandomStream.GetFraction() < DepositProbability)
	{
		// Remove physical attenuation, apply standard Unreal point light attenuation from each primitive
		return PowerWeightedAttenuation / (PowerWeightedPhysicalAttenuation * DepositProbability);
	}
	else
	{
		return 0.0f;
	}
}

// Fudge factor to get mesh area light photon intensities to match direct lighting more closely.
static const float MeshAreaLightIntensityScale = 2.5f; 

/** Generates a direction sample from the light's domain */
void FMeshAreaLight::SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const
{
	FLightSurfaceSample SurfaceSample;
	FMeshAreaLight::SampleLightSurface(RandomStream, SurfaceSample);

	const float DistanceFromCenter = (SurfaceSample.Position - Position).Size3();

	// Generate a sample direction from a distribution that is uniform over all directions
	FVector4f SampleDir;
	do 
	{
		SampleDir = GetUnitVector(RandomStream);
	} 
	// Regenerate the direction vector until it is less than .1 of a degree from perpendicular to the light's surface normal
	// This prevents generating directions that are deemed outside of the light source primitive's hemisphere by later calculations due to fp imprecision
	while(FMath::Abs(Dot3(SampleDir, SurfaceSample.Normal)) < .0017);
	
	if (Dot3(SampleDir, SurfaceSample.Normal) < 0.0f)
	{
		// Reflect the sample direction across the origin so that it lies in the same hemisphere as the primitive normal
		SampleDir *= -1.0f;
	}

	SampleRay = FLightRay(
		SurfaceSample.Position,
		SurfaceSample.Position + SampleDir * FMath::Max(InfluenceRadius - DistanceFromCenter, 0.0f),
		NULL,
		this
		);

	LightSourceNormal = SurfaceSample.Normal;

	// The probability of selecting any direction in a hemisphere defined by each primitive normal
	const float HemispherePDF = 1.0f / (2.0f * (float)PI);
	RayPDF = 0.0f;

	const FIntPoint Corners[] = 
	{
		FIntPoint(0,0),
		FIntPoint(0,1),
		FIntPoint(1,0),
		FIntPoint(1,1)
	};

	// Use a grid which contains cached primitive normals to accelerate PDF calculation
	// This prevents the need to iterate over all of the mesh area light's primitives, of which there may be thousands
	for (int32 OccupiedCellIndex = 0; OccupiedCellIndex < OccupiedCachedPrimitiveNormalCells.Num(); OccupiedCellIndex++)
	{
		const int32 ThetaStep = OccupiedCachedPrimitiveNormalCells[OccupiedCellIndex].X;
		const int32 PhiStep = OccupiedCachedPrimitiveNormalCells[OccupiedCellIndex].Y;
		const TArray<FVector4f>& CurrentCachedNormals = CachedPrimitiveNormals[PhiStep * MeshAreaLightGridSize + ThetaStep];
		if (CurrentCachedNormals.Num() > 0)
		{
			bool bAllCornersInSameHemisphere = true;
			bool bAllCornersInOppositeHemisphere = true;
			// Determine whether the cell is completely in the same hemisphere as the sample direction, completely on the other side or spanning the terminator
			// This is done by checking each cell's corners
			for (int32 CornerIndex = 0; CornerIndex < UE_ARRAY_COUNT(Corners); CornerIndex++)
			{
				const float Theta = (ThetaStep + Corners[CornerIndex].X) / (float)MeshAreaLightGridSize * (float)PI;
				const float Phi = (PhiStep + Corners[CornerIndex].Y) / (float)MeshAreaLightGridSize * 2 * (float)PI - (float)PI;
				// Calculate the cartesian unit direction corresponding to this corner
				const FVector4f CurrentCornerDirection = FVector2f(Theta, Phi).SphericalToUnitCartesian();
				bAllCornersInSameHemisphere = bAllCornersInSameHemisphere && Dot3(CurrentCornerDirection, SampleDir) > 0.0f;
				bAllCornersInOppositeHemisphere = bAllCornersInOppositeHemisphere && Dot3(CurrentCornerDirection, SampleDir) < 0.0f;
			}

			if (bAllCornersInSameHemisphere)
			{
				// If the entire cell is in the same hemisphere as the sample direction, the sample could have been generated from any of them
				RayPDF += CurrentCachedNormals.Num() * HemispherePDF;
			}
			else if (!bAllCornersInOppositeHemisphere)
			{
				// If the cell spans both hemispheres, we have to test each normal individually
				for (int32 CachedNormalIndex = 0; CachedNormalIndex < CurrentCachedNormals.Num(); CachedNormalIndex++)
				{
					if (Dot3(CurrentCachedNormals[CachedNormalIndex], SampleDir) > 0.0f)
					{
						// Accumulate the probability that this direction was generated by each primitive
						RayPDF += HemispherePDF;
					}
				}
			}
		}
	}

	RayPDF /= Primitives.Num();
	
	checkSlow(RayPDF > 0.0f);

	Power = TotalPower / TotalSurfaceArea * MeshAreaLightIntensityScale;
}

/** Generates a direction sample from the light based on the given rays */
void FMeshAreaLight::SampleDirection(
	const TArray<FIndirectPathRay>& IndirectPathRays, 
	FLMRandomStream& RandomStream, 
	FLightRay& SampleRay, 
	float& RayPDF, 
	FLinearColor& Power) const
{
	checkSlow(IndirectPathRays.Num() > 0);
	// Pick an indirect path ray with uniform probability
	const int32 RayIndex = FMath::TruncToInt(RandomStream.GetFraction() * IndirectPathRays.Num());
	checkSlow(RayIndex >= 0 && RayIndex < IndirectPathRays.Num());
	const FIndirectPathRay& ChosenPathRay = IndirectPathRays[RayIndex];
	const FVector4f PathRayDirection = ChosenPathRay.UnitDirection;

	FVector4f XAxis(0,0,0);
	FVector4f YAxis(0,0,0);
	GenerateCoordinateSystem(PathRayDirection, XAxis, YAxis);

	// Calculate Cos of the angle between the direction and the light source normal.
	// This is also the Sin of the angle between the direction and the plane perpendicular to the normal.
	const float DirectionDotLightNormal = Dot3(PathRayDirection, ChosenPathRay.LightSourceNormal);
	checkSlow(DirectionDotLightNormal > 0.0f);
	// Calculate Cos of the angle between the direction and the plane perpendicular to the normal using cos^2 + sin^2 = 1
	const float CosDirectionNormalPlaneAngle = FMath::Sqrt(1.0f - DirectionDotLightNormal * DirectionDotLightNormal);

	// Clamp the cone angle to CosDirectionNormalPlaneAngle so that any direction generated from the cone lies in the same hemisphere 
	// As the light source normal that was used to generate that direction.
	// This is necessary to make sure we only generate directions that the light actually emits in.
	// Within the range [0, PI / 2], smaller angles have a larger cosine
	// The DELTA bias is to avoid generating directions that are so close to being perpendicular to the normal that their dot product is negative due to fp imprecision.
	const float CosEmitConeAngle = FMath::Max(CosIndirectPhotonEmitConeAngle, FMath::Min(CosDirectionNormalPlaneAngle + DELTA, 1.0f));

	// Generate a sample direction within a cone about the indirect path
	const FVector4f ConeSampleDirection = UniformSampleCone(RandomStream, CosEmitConeAngle, XAxis, YAxis, PathRayDirection);

	FLightSurfaceSample SurfaceSample;
	float NormalDotSampleDirection = 0.0f;
	do 
	{
		// Generate a surface sample
		FMeshAreaLight::SampleLightSurface(RandomStream, SurfaceSample);
		NormalDotSampleDirection = Dot3(SurfaceSample.Normal, ConeSampleDirection);
	} 
	// Use rejection sampling to find a surface position that is valid for ConeSampleDirection
	while(NormalDotSampleDirection < 0.0f);

	const float DistanceFromCenter = (SurfaceSample.Position - Position).Size3();

	SampleRay = FLightRay(
		SurfaceSample.Position,
		SurfaceSample.Position + ConeSampleDirection * FMath::Max(InfluenceRadius - DistanceFromCenter, 0.0f),
		NULL,
		this
		);

	const float ConePDF = UniformConePDF(CosEmitConeAngle);
	RayPDF = 0.0f;
	// Calculate the probability that this direction was chosen
	for (int32 OtherRayIndex = 0; OtherRayIndex < IndirectPathRays.Num(); OtherRayIndex++)
	{
		// Accumulate the cone probability for all the cones which contain the sample position
		if (Dot3(IndirectPathRays[OtherRayIndex].UnitDirection, ConeSampleDirection) > (1.0f - DELTA) * CosEmitConeAngle)
		{
			RayPDF += ConePDF;
		}
	}
	RayPDF /= IndirectPathRays.Num();
	checkSlow(RayPDF > 0);
	Power = TotalPower / TotalSurfaceArea * MeshAreaLightIntensityScale;
}

/** Validates a surface sample given the position that sample is affecting. */
void FMeshAreaLight::ValidateSurfaceSample(const FVector4f& Point, FLightSurfaceSample& Sample) const
{
}

/** Returns the light's radiant power. */
float FMeshAreaLight::Power() const
{
	const FLinearColor LightPower = TotalPower / TotalSurfaceArea * 2.0f * (float)PI * InfluenceRadius * InfluenceRadius;
	return FLinearColorUtils::LinearRGBToXYZ(LightPower).G;
}

/** Generates a sample on the light's surface. */
void FMeshAreaLight::SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const
{
	float PrimitivePDF;
	float FloatPrimitiveIndex;
	// Pick a primitive with probability proportional to the primitive's fraction of the light's total surface area
	Sample1dCDF(PrimitivePDFs, PrimitiveCDFs, UnnormalizedIntegral, RandomStream, PrimitivePDF, FloatPrimitiveIndex);
	const int32 PrimitiveIndex = FMath::TruncToInt(FloatPrimitiveIndex * Primitives.Num());
	check(PrimitiveIndex >= 0 && PrimitiveIndex < Primitives.Num());

	const FMeshLightPrimitive& SelectedPrimitive = Primitives[PrimitiveIndex];
	// Approximate the primitive as a coplanar square, and sample uniformly by area
	const float Alpha1 = RandomStream.GetFraction();
	const FVector4f InterpolatedPosition1 = FMath::Lerp(SelectedPrimitive.Corners[0].WorldPosition, SelectedPrimitive.Corners[1].WorldPosition, Alpha1);
	const FVector4f InterpolatedPosition2 = FMath::Lerp(SelectedPrimitive.Corners[2].WorldPosition, SelectedPrimitive.Corners[3].WorldPosition, Alpha1);
	const float Alpha2 = RandomStream.GetFraction();
	const FVector4f SamplePosition = FMath::Lerp(InterpolatedPosition1, InterpolatedPosition2, Alpha2);
	const float SamplePDF = PrimitivePDF / SelectedPrimitive.SurfaceArea;
	Sample = FLightSurfaceSample(SamplePosition, SelectedPrimitive.SurfaceNormal, FVector2f(0,0), SamplePDF);
}

/** Returns true if all parts of the light are behind the surface being tested. */
bool FMeshAreaLight::BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const
{
	const float NormalDotLight = Dot3(TriangleNormal, FMeshAreaLight::GetDirectLightingDirection(TrianglePoint, TriangleNormal));
	return NormalDotLight < 0.0f;
}

/** Gets a single direction to use for direct lighting that is representative of the whole area light. */
FVector4f FMeshAreaLight::GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const
{
	// The position on a sphere approximating the area light surface that will first be visible to a triangle rotating toward the light
	const FVector4f FirstVisibleLightPoint = Position + PointNormal * SourceBounds.SphereRadius;
	return FirstVisibleLightPoint - Point;
}

}
