// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneExport.h"
#include "Math/LMMath.h"
#include "Lighting.h"

namespace Lightmass
{

class FDirectionalLight;
class FPointLight;
class FSpotLight;
class FSkyLight;
class FStaticMesh;
class FStaticMeshStaticLightingMesh;
class FFluidSurfaceStaticLightingMesh;
class FLandscapeStaticLightingMesh;
class FStaticMeshStaticLightingTextureMapping;
class FBSPSurfaceStaticLighting;
class FFluidSurfaceStaticLightingTextureMapping;
class FLandscapeStaticLightingTextureMapping;
class FStaticLightingGlobalVolumeMapping;
class FLandscapeStaticLightingGlobalVolumeMapping;

/** A sample of a light's surface. */
class FLightSurfaceSample
{
public:
	/** World space position */
	FVector4f Position;
    /** Normal of the light's surface at the sample point */
	FVector4f Normal;
	/** Position on the disk for lights modelled by a disk */
	FVector2f DiskPosition;
	/** The probability that a sample with this position was generated */
	float PDF;

	FLightSurfaceSample() {}

	FLightSurfaceSample(const FVector4f& InPosition, const FVector4f& InNormal, const FVector2f& InDiskPosition, float InPDF) :
		Position(InPosition),
		Normal(InNormal),
		DiskPosition(InDiskPosition),
		PDF(InPDF)
	{}
};

/** A path that was found to result in at least one indirect photon being deposited. */
class FIndirectPathRay
{
public:
	FVector4f Start;
	FVector4f UnitDirection;
	FVector4f LightSourceNormal;
	FVector2f LightSurfacePosition;
	float Length;

	FIndirectPathRay(const FVector4f& InStart, const FVector4f& InUnitDirection, const FVector4f& InLightSourceNormal, const FVector2f& InLightSurfacePosition, const float& InLength) :
		Start(InStart),
		UnitDirection(InUnitDirection),
		LightSourceNormal(InLightSourceNormal),
		LightSurfacePosition(InLightSurfacePosition),
		Length(InLength)
	{}
};

class FIrradiancePhotonData
{
protected:
	/** XYZ stores world space position, W stores 1 if the photon has contribution from direct photons, and 0 otherwise. */
	FVector4f PositionAndDirectContribution;

	/** 
	* XYZ stores the world space normal of the receiving surface.
	* The irradiance photon caching pass stores 1 in W if the photon is actually used. 
	* This is overwritten with the photon's irradiance in RGBE in the irradiance photon calculating pass.
	*/
	FVector4f SurfaceNormalAndIrradiance;
};

//----------------------------------------------------------------------------
//	Light base class
//----------------------------------------------------------------------------
class FLight : public FLightData
{
public:
	virtual ~FLight() { }

	virtual void			Import( class FLightmassImporter& Importer );

	/**
	 * @return 'this' if the light is a skylight, NULL otherwise 
	 */
	virtual const class FSkyLight* GetSkyLight() const
	{
		return NULL;
	}
	virtual class FSkyLight* GetSkyLight()
	{
		return NULL;
	}

	virtual class FDirectionalLight* GetDirectionalLight() 
	{
		return NULL;
	}

	virtual const class FDirectionalLight* GetDirectionalLight() const
	{
		return NULL;
	}

	virtual class FPointLight* GetPointLight() 
	{
		return NULL;
	}

	virtual const class FPointLight* GetPointLight() const
	{
		return NULL;
	}

	virtual class FSpotLight* GetSpotLight() 
	{
		return NULL;
	}

	virtual const class FSpotLight* GetSpotLight() const
	{
		return NULL;
	}

	virtual const class FMeshAreaLight* GetMeshAreaLight() const
	{
		return NULL;
	}

	/** Returns the number of direct photons to gather required by this light. */
	virtual int32 GetNumDirectPhotons(float DirectPhotonDensity) const = 0;

	/**
	 * Tests whether the light affects the given bounding volume.
	 * @param Bounds - The bounding volume to test.
	 * @return True if the light affects the bounding volume
	 */
	virtual bool AffectsBounds(const FBoxSphereBounds3f& Bounds) const;

	virtual FSphere3f GetBoundingSphere() const;

	/**
	 * Computes the intensity of the direct lighting from this light on a specific point.
	 */
	virtual FLinearColor GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const;

	/** Returns an intensity scale based on the receiving point. */
	virtual float CustomAttenuation(const FVector4f& Point, FLMRandomStream& RandomStream, bool bMaintainEvenDensity) const { return 1.0f; }

	/** Generates a direction sample from the light's domain */
	virtual void SampleDirection(FLMRandomStream& RandomStream, class FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const = 0;

	/** Gives the light an opportunity to precalculate information about the indirect path rays that will be used to generate new directions. */
	virtual void CachePathRays(const TArray<FIndirectPathRay>& IndirectPathRays) {}

	/** Generates a direction sample from the light based on the given rays */
	virtual void SampleDirection(
		const TArray<FIndirectPathRay>& IndirectPathRays, 
		FLMRandomStream& RandomStream, 
		FLightRay& SampleRay, 
		float& RayPDF,
		FLinearColor& Power) const = 0;

	/** Returns the light's radiant power. */
	virtual float Power() const = 0;

	/** Generates and caches samples on the light's surface. */
	virtual void CacheSurfaceSamples(int32 BounceIndex, int32 NumSamples, int32 NumPenumbraSamples, FLMRandomStream& RandomStream);

	/** Retrieves the array of cached light surface samples. */
	const TArray<FLightSurfaceSample>& GetCachedSurfaceSamples(int32 BounceIndex, bool bPenumbra) const;

	/** Validates a surface sample given the position that sample is affecting.  The sample is unaffected by default. */
	virtual void ValidateSurfaceSample(const FVector4f& Point, FLightSurfaceSample& Sample) const {}

	/** Gets a single position which represents the center of the area light source from the ReceivingPosition's point of view. */
	virtual FVector4f LightCenterPosition(const FVector4f& ReceivingPosition, const FVector4f& ReceivingNormal) const { return Position; }

	/** Returns true if all parts of the light are behind the surface being tested. */
	virtual bool BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const = 0;

	/** Gets a single direction to use for direct lighting that is representative of the whole area light. */
	virtual FVector4f GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const = 0;

	/**
	 * Returns whether static lighting, aka lightmaps, is being used for primitive/ light interaction.
	 *
	 * @return true if lightmaps/ static lighting is being used, false otherwise
	 */
	bool UseStaticLighting() const
	{
		return (LightFlags & GI_LIGHT_HASSTATICLIGHTING) != 0;
	}

	virtual FVector3f GetLightTangent() const { return Direction; }

protected:
	/** Cached samples of the light's surface, indexed first by bounce number, then by whether the shadow ray is a penumbra ray, then by sample index. */
	TArray<TArray<TArray<FLightSurfaceSample> > > CachedLightSurfaceSamples;

	/** Cached calculation of the light's indirect color, which is the base Color adjusted by the IndirectLightingScale and IndirectLightingSaturation */
	FLinearColor IndirectColor;

	/** Generates a sample on the light's surface. */
	virtual void SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const = 0;

	TArray< uint8 > LightTextureProfileData;
};


//----------------------------------------------------------------------------
//	Directional light class
//----------------------------------------------------------------------------
class FDirectionalLight : public FLight, public FDirectionalLightData
{
public:

	float IndirectDiskRadius;

	virtual void			Import( class FLightmassImporter& Importer );

	virtual class FDirectionalLight* GetDirectionalLight() 
	{
		return this;
	}

	virtual const class FDirectionalLight* GetDirectionalLight() const
	{
		return this;
	}

	void Initialize(
		const FBoxSphereBounds3f& InSceneBounds, 
		bool bInEmitPhotonsOutsideImportanceVolume,
		const FBoxSphereBounds3f& InImportanceBounds, 
		float InIndirectDiskRadius, 
		int32 InGridSize, 
		float InDirectPhotonDensity,
		float InOutsideImportanceVolumeDensity);

	/** Returns the number of direct photons to gather required by this light. */
	virtual int32 GetNumDirectPhotons(float DirectPhotonDensity) const;

	/** Generates a direction sample from the light's domain */
	virtual void SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const;

	/** Gives the light an opportunity to precalculate information about the indirect path rays that will be used to generate new directions. */
	virtual void CachePathRays(const TArray<FIndirectPathRay>& IndirectPathRays);

	/** Generates a direction sample from the light based on the given rays */
	virtual void SampleDirection(
		const TArray<FIndirectPathRay>& IndirectPathRays, 
		FLMRandomStream& RandomStream, 
		FLightRay& SampleRay, 
		float& RayPDF,
		FLinearColor& Power) const;

	/** Returns the light's radiant power. */
	virtual float Power() const;

	/** Validates a surface sample given the position that sample is affecting. */
	virtual void ValidateSurfaceSample(const FVector4f& Point, FLightSurfaceSample& Sample) const;

	/** Gets a single position which represents the center of the area light source from the ReceivingPosition's point of view. */
	virtual FVector4f LightCenterPosition(const FVector4f& ReceivingPosition, const FVector4f& ReceivingNormal) const;

	/** Returns true if all parts of the light are behind the surface being tested. */
	virtual bool BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const;

	/** Gets a single direction to use for direct lighting that is representative of the whole area light. */
	virtual FVector4f GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const;

protected:

	/** Extent of PathRayGrid in the [-1, 1] space of the directional light's disk. */
	float GridExtent;

	/** Center of PathRayGrid in the [-1, 1] space of the directional light's disk. */
	FVector2f GridCenter;

	/** Size of PathRayGrid in each dimension. */
	int32 GridSize;

	/** Grid of indices into IndirectPathRays that affect each cell. */
	TArray<TArray<int32>> PathRayGrid;

	/** Bounds of the scene that the directional light is affecting. */
	FBoxSphereBounds3f SceneBounds;

	bool bEmitPhotonsOutsideImportanceVolume;

	/** Bounds of the importance volume in the scene.  If the radius is 0, there was no importance volume. */
	FBoxSphereBounds3f ImportanceBounds;

	/** Center of the importance volume in the [-1,1] space of the directional light's disk */
	FVector2f ImportanceDiskOrigin;

	/** Radius of the importance volume in the [-1,1] space of the directional light's disk */
	float LightSpaceImportanceDiskRadius;

	/** Density of photons to gather outside of the importance volume. */
	float OutsideImportanceVolumeDensity;

	/** Probability of generating a sample inside the importance volume. */
	float ImportanceBoundsSampleProbability;

	/** X axis of the directional light, which is unit length and orthogonal to the direction and YAxis. */
	FVector4f XAxis;

	/** Y axis of the directional light, which is unit length and orthogonal to the direction and XAxis. */
	FVector4f YAxis;

	/** Generates a sample on the light's surface. */
	virtual void SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const;
};


//----------------------------------------------------------------------------
//	Point light class
//----------------------------------------------------------------------------
class FPointLight : public FLight, public FPointLightData
{
public:
	virtual void			Import( class FLightmassImporter& Importer );

	virtual class FPointLight* GetPointLight() 
	{
		return this;
	}

	virtual const class FPointLight* GetPointLight() const
	{
		return this;
	}

	void Initialize(float InIndirectPhotonEmitConeAngle);

	/** Returns the number of direct photons to gather required by this light. */
	virtual int32 GetNumDirectPhotons(float DirectPhotonDensity) const;

	/**
	 * Tests whether the light affects the given bounding volume.
	 * @param Bounds - The bounding volume to test.
	 * @return True if the light affects the bounding volume
	 */
	virtual bool AffectsBounds(const FBoxSphereBounds3f& Bounds) const;

	virtual FSphere3f GetBoundingSphere() const
	{
		return FSphere3f(Position, Radius);
	}

	/**
	 * Computes the intensity of the direct lighting from this light on a specific point.
	 */
	virtual FLinearColor GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const;

	/** Returns an intensity scale based on the receiving point. */
	virtual float CustomAttenuation(const FVector4f& Point, FLMRandomStream& RandomStream, bool bMaintainEvenDensity) const;

	/** Generates a direction sample from the light's domain */
	virtual void SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const;

	/** Generates a direction sample from the light based on the given rays */
	virtual void SampleDirection(
		const TArray<FIndirectPathRay>& IndirectPathRays, 
		FLMRandomStream& RandomStream, 
		FLightRay& SampleRay, 
		float& RayPDF,
		FLinearColor& Power) const;

	/** Validates a surface sample given the position that sample is affecting. */
	virtual void ValidateSurfaceSample(const FVector4f& Point, FLightSurfaceSample& Sample) const;

	/** Returns the light's radiant power. */
	virtual float Power() const;

	virtual FVector4f LightCenterPosition(const FVector4f& ReceivingPosition, const FVector4f& ReceivingNormal) const;

	/** Returns true if all parts of the light are behind the surface being tested. */
	virtual bool BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const;

	/** Gets a single direction to use for direct lighting that is representative of the whole area light. */
	virtual FVector4f GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const;

	virtual FVector3f GetLightTangent() const override;

protected:

	float CosIndirectPhotonEmitConeAngle;

	/** Generates a sample on the light's surface. */
	virtual void SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const;
};


//----------------------------------------------------------------------------
//	Spot light class
//----------------------------------------------------------------------------
class FSpotLight : public FPointLight, public FSpotLightData
{
public:

	virtual class FSpotLight* GetSpotLight() 
	{
		return this;
	}

	virtual const class FSpotLight* GetSpotLight() const
	{
		return this;
	}

	virtual void Import( class FLightmassImporter& Importer );

	void Initialize(float InIndirectPhotonEmitConeAngle);

	/**
	 * Tests whether the light affects the given bounding volume.
	 * @param Bounds - The bounding volume to test.
	 * @return True if the light affects the bounding volume
	 */
	virtual bool AffectsBounds(const FBoxSphereBounds3f& Bounds) const;

	virtual FSphere3f GetBoundingSphere() const;

	/**
	 * Computes the intensity of the direct lighting from this light on a specific point.
	 */
	virtual FLinearColor GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const;

	/** Returns the number of direct photons to gather required by this light. */
	virtual int32 GetNumDirectPhotons(float DirectPhotonDensity) const;

	/** Generates a direction sample from the light's domain */
	virtual void SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const;

	/** Generates a direction sample from the light based on the given rays */
	virtual void SampleDirection(
		const TArray<FIndirectPathRay>& IndirectPathRays, 
		FLMRandomStream& RandomStream, 
		FLightRay& SampleRay, 
		float& RayPDF,
		FLinearColor& Power) const;

protected:
	float SinOuterConeAngle;
	float CosOuterConeAngle;
	float CosInnerConeAngle;
};


//----------------------------------------------------------------------------
//	Rect light class
//----------------------------------------------------------------------------
class FRectLight : public FPointLight, public FRectLightData
{
public:
	virtual void			Import( class FLightmassImporter& Importer );

	virtual FLinearColor	GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const override;
	virtual int32			GetNumDirectPhotons(float DirectPhotonDensity) const override;
	virtual void			ValidateSurfaceSample(const FVector4f& Point, FLightSurfaceSample& Sample) const override;
	virtual FVector4f		LightCenterPosition(const FVector4f& ReceivingPosition, const FVector4f& ReceivingNormal) const override;
	virtual bool			AffectsBounds(const FBoxSphereBounds3f& Bounds) const override;
	virtual bool			BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const override;
	virtual FVector4f		GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const override;

	virtual void			SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const override;
	virtual void			SampleDirection(const TArray<FIndirectPathRay>& IndirectPathRays, FLMRandomStream& RandomStream, FLightRay& SampleRay, float& RayPDF, FLinearColor& Power) const override;

protected:
	virtual void			SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const override;

	TArray< FFloat16Color >	SourceTexture;
};


//----------------------------------------------------------------------------
//	Sky light class
//----------------------------------------------------------------------------
class FSkyLight : public FLight, public FSkyLightData
{
public:
	virtual void			Import( class FLightmassImporter& Importer );

	/**
	 * @return 'this' if the light is a skylight, NULL otherwise 
	 */
	virtual const class FSkyLight* GetSkyLight() const
	{
		return this;
	}
	virtual class FSkyLight* GetSkyLight()
	{
		return this;
	}

	/** Returns the number of direct photons to gather required by this light. */
	virtual int32 GetNumDirectPhotons(float DirectPhotonDensity) const
	{ checkf(0, TEXT("GetNumDirectPhotons is not supported for skylights")); return 0; }

	/** Generates a direction sample from the light's domain */
	virtual void SampleDirection(FLMRandomStream& RandomStream, class FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const
	{ checkf(0, TEXT("SampleDirection is not supported for skylights")); }

	/** Generates a direction sample from the light based on the given rays */
	virtual void SampleDirection(
		const TArray<FIndirectPathRay>& IndirectPathRays, 
		FLMRandomStream& RandomStream, 
		FLightRay& SampleRay, 
		float& RayPDF,
		FLinearColor& Power) const
	{ checkf(0, TEXT("SampleDirection is not supported for skylights")); }

	/** Returns the light's radiant power. */
	virtual float Power() const
	{ checkf(0, TEXT("Power is not supported for skylights")); return 0; }

	/** Returns true if all parts of the light are behind the surface being tested. */
	virtual bool BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const { return false; }

	/** Gets a single direction to use for direct lighting that is representative of the whole area light. */
	virtual FVector4f GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const 
	{ checkf(0, TEXT("GetDirectLightingDirection is not supported for skylights")); return FVector4f(); }

	FLinearColor GetPathLighting(const FVector4f& IncomingDirection, float PathSolidAngle, bool bCalculateForIndirectLighting) const;

	float GetPathVariance(const FVector4f& IncomingDirection, float PathSolidAngle) const;

protected:

	/** Generates a sample on the light's surface. */
	virtual void SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const
	{ checkf(0, TEXT("SampleLightSurface is not supported for skylights")); }

	float GetMipIndexForSolidAngle(float SolidAngle) const;
	FLinearColor SampleRadianceCubemap(float MipIndex, int32 CubeFaceIndex, FVector2f FaceUV) const;
	float SampleVarianceCubemap(float MipIndex, int32 CubeFaceIndex, FVector2f FaceUV) const;

	void ComputePrefilteredVariance();

	int32 CubemapSize;
	int32 NumMips;
	TArray<TArray<FLinearColor>> PrefilteredRadiance;
	TArray<TArray<float>> PrefilteredVariance;
};

class FMeshLightPrimitiveCorner
{
public:
	/** World space corner position, not necessarily coplanar with the other corners. */
	FVector4f WorldPosition;
	/** Coordinate in texture space corresponding to the position this corner is storing. */
	FIntPoint FurthestCoordinates;
};

/** The atomic shape used to represent an area light's shape. */
class FMeshLightPrimitive
{
public:
	FMeshLightPrimitiveCorner Corners[NumTexelCorners];
	/** Average normal of the sub primitives making up this simplified primitive. */
	FVector4f SurfaceNormal;
	/** Radiant flux of this primitive */
	FLinearColor Power;
	/** Surface area of this primitive */
	float SurfaceArea;
	/** Number of original primitives that were combined into this simplified primitive. */
	int32 NumSubPrimitives;

	void AddSubPrimitive(const struct FTexelToCorners& TexelToCorners, const FIntPoint& Coordinates, const FLinearColor& InTexelPower, float NormalOffset);
	void Finalize();
};

//----------------------------------------------------------------------------
//	Mesh Area Light class
//----------------------------------------------------------------------------
class FMeshAreaLight : public FLight
{
public:

	FMeshAreaLight(EForceInit)
	{
		// Initialize base members since the default constructor does nothing
		// All the other light types are always serialized in so this is only needed for FMeshAreaLight
		// Note: this will stomp on derived class data members, but it's assumed that 0 is a valid default.
		FMemory::Memzero((FLightData*)this, sizeof(FLightData));
	}

	virtual const class FMeshAreaLight* GetMeshAreaLight() const
	{
		return this;
	}

	void Initialize(float InIndirectPhotonEmitConeAngle, const FBoxSphereBounds3f& InImportanceBounds);

	/** Returns the number of direct photons to gather required by this light. */
	virtual int32 GetNumDirectPhotons(float DirectPhotonDensity) const;

	/** Initializes the mesh area light with primitives */
	void SetPrimitives(
		const TArray<FMeshLightPrimitive>& InPrimitives, 
		float EmissiveLightFalloffExponent, 
		float EmissiveLightExplicitInfluenceRadius,
		int32 InMeshAreaLightGridSize,
		FGuid InLevelGuid);

	int32 GetNumPrimitives() const 
	{ 
		int32 NumTotalPrimitives = 0;
		for (int32 PrimitiveIndex = 0; PrimitiveIndex < Primitives.Num(); PrimitiveIndex++)
		{
			NumTotalPrimitives += Primitives[PrimitiveIndex].NumSubPrimitives;
		}
		return NumTotalPrimitives; 
	}

	int32 GetNumSimplifiedPrimitives() const { return Primitives.Num(); }

	/**
	* Tests whether the light affects the given bounding volume.
	* @param Bounds - The bounding volume to test.
	* @return True if the light affects the bounding volume
	*/
	virtual bool AffectsBounds(const FBoxSphereBounds3f& Bounds) const;

	/**
	 * Computes the intensity of the direct lighting from this light on a specific point.
	 */
	virtual FLinearColor GetDirectIntensity(const FVector4f& Point, bool bCalculateForIndirectLighting) const;

	/** Returns an intensity scale based on the receiving point. */
	virtual float CustomAttenuation(const FVector4f& Point, FLMRandomStream& RandomStream, bool bMaintainEvenDensity) const;

	/** Generates a direction sample from the light's domain */
	virtual void SampleDirection(FLMRandomStream& RandomStream, FLightRay& SampleRay, FVector4f& LightSourceNormal, FVector2f& LightSurfacePosition, float& RayPDF, FLinearColor& Power) const;

	/** Generates a direction sample from the light based on the given rays */
	virtual void SampleDirection(
		const TArray<FIndirectPathRay>& IndirectPathRays, 
		FLMRandomStream& RandomStream, 
		FLightRay& SampleRay, 
		float& RayPDF,
		FLinearColor& Power) const;

	/** Validates a surface sample given the position that sample is affecting. */
	void ValidateSurfaceSample(const FVector4f& Point, FLightSurfaceSample& Sample) const;

	/** Returns the light's radiant power. */
	virtual float Power() const;

	/** Returns true if all parts of the light are behind the surface being tested. */
	virtual bool BehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal) const;

	/** Gets a single direction to use for direct lighting that is representative of the whole area light. */
	virtual FVector4f GetDirectLightingDirection(const FVector4f& Point, const FVector4f& PointNormal) const;

protected:
	/** Radiant flux of all the primitives */
	FLinearColor TotalPower;

	/** Accumulated surface area of all primitives */
	float TotalSurfaceArea;

	/** Generated radius of the light's influence */
	float InfluenceRadius;

	/** Bounds of the light's primitives */
	FBoxSphereBounds3f SourceBounds;

	/** Falloff of the attenuation function */
	float FalloffExponent;

	/** Cosine of the angle about an indirect path in which to emit indirect photons */
	float CosIndirectPhotonEmitConeAngle;

	/** All the primitives that define this light's shape */
	TArray<FMeshLightPrimitive> Primitives;

	/** Size of the data stored in CachedPrimitiveNormals in each dimension. */
	int32 MeshAreaLightGridSize;

	FGuid LevelGuid;

	/** Grid of arrays of primitive normals, used to accelerate PDF calculation once a sample is generated. */
	TArray<TArray<FVector4f> > CachedPrimitiveNormals;

	/** Entries in CachedPrimitiveNormals that have one or more cached normals. */
	TArray<FIntPoint> OccupiedCachedPrimitiveNormalCells;

	/** Bounds of the importance volume in the scene.  If the radius is 0, there was no importance volume. */
	FBoxSphereBounds3f ImportanceBounds;

	/** Probability of selecting each primitive when sampling. */
	TArray<float> PrimitivePDFs;

	/** Stores the cumulative distribution function of PrimitivePDFs */
	TArray<float> PrimitiveCDFs;

	/** Stores the integral of PrimitivePDFs */
	float UnnormalizedIntegral;

	/** Generates a sample on the light's surface. */
	virtual void SampleLightSurface(FLMRandomStream& RandomStream, FLightSurfaceSample& Sample) const;

	friend class FLightmassSolverExporter;
};

/** Volume that determines where to place visibility cells. */
class FPrecomputedVisibilityVolume
{
public:
	FBox3f Bounds;
	TArray<FPlane4f> Planes;
};

/** Volume that overrides visibility for a set of Ids. */
class FPrecomputedVisibilityOverrideVolume
{
public:
	FBox3f Bounds;
	TArray<int32> OverrideVisibilityIds;
	TArray<int32> OverrideInvisibilityIds;
};

struct FVolumetricLightmapDensityVolume : public FVolumetricLightmapDensityVolumeData
{
	TArray<FPlane4f> Planes;
};

//----------------------------------------------------------------------------
//	Scene class
//----------------------------------------------------------------------------
class FScene : public FSceneFileHeader
{
public:
	FScene();
	virtual ~FScene();

	virtual void			Import( class FLightmassImporter& Importer );
	FBoxSphereBounds3f		GetImportanceBounds() const;

	FBox3f ImportanceBoundingBox;
	TArray<FBox3f> ImportanceVolumes;
	TArray<FBox3f> CharacterIndirectDetailVolumes;
	TArray<FVolumetricLightmapDensityVolume> VolumetricLightmapDensityVolumes;
	TArray<FSphere3f> Portals;
	TArray<FPrecomputedVisibilityVolume> PrecomputedVisibilityVolumes;
	TArray<FPrecomputedVisibilityOverrideVolume> PrecomputedVisibilityOverrideVolumes;
	TArray<FVector4f> CameraTrackPositions;

	TArray<FDirectionalLight>	DirectionalLights;
	TArray<FPointLight>			PointLights;
	TArray<FSpotLight>			SpotLights;
	TArray<FRectLight>			RectLights;
	TArray<FSkyLight>			SkyLights;

	TArray<FStaticMeshStaticLightingMesh>				StaticMeshInstances;
	TArray<FFluidSurfaceStaticLightingMesh>				FluidMeshInstances;
	TArray<FLandscapeStaticLightingMesh>				LandscapeMeshInstances;
	TArray<FBSPSurfaceStaticLighting>					BspMappings;
	TArray<FStaticMeshStaticLightingTextureMapping>		TextureLightingMappings;
	TArray<FFluidSurfaceStaticLightingTextureMapping>	FluidMappings;
	TArray<FLandscapeStaticLightingTextureMapping>		LandscapeMappings;
	TArray<FStaticLightingGlobalVolumeMapping>			VolumeMappings;
	TArray<FLandscapeStaticLightingGlobalVolumeMapping> LandscapeVolumeMappings;

	TArray<FGuid> VisibilityBucketGuids;
	TArray<FGuid> VolumetricLightmapTaskGuids;

	RTCDevice EmbreeDevice;
	bool bVerifyEmbree;

	/** The mapping whose texel is selected in Unreal and is being debugged. */
	const class FStaticLightingMapping* DebugMapping;

	const FLight* FindLightByGuid(const FGuid& Guid) const;

	/** Returns true if the specified position is inside any of the importance volumes. */
	bool IsPointInImportanceVolume(const FVector4f& Position, float Tolerance) const;

	bool IsBoxInImportanceVolume(const FBox3f& QueryBox) const;

	/** Returns true if the specified position is inside any of the visibility volumes. */
	bool IsPointInVisibilityVolume(const FVector4f& Position) const;

	bool DoesBoxIntersectVisibilityVolume(const FBox3f& TestBounds) const;

	/** Returns accumulated bounds from all the visibility volumes. */
	FBox3f GetVisibilityVolumeBounds() const;

	bool GetVolumetricLightmapAllowedMipRange(const FVector4f& Position, FIntPoint& OutRange) const;

private:
	/** Searches through all mapping arrays for the mapping matching FindGuid. */
	const FStaticLightingMapping* FindMappingByGuid(FGuid FindGuid) const;
	
	/** Applies GeneralSettings.StaticLightingLevelScale to all scale dependent settings. */
	void ApplyStaticLightingScale();
};


} // namespace Lightmass
