// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmassSwarm.h"
#include "CPUSolver.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/FileManager.h"
#include "UnrealLightmass.h"

namespace Lightmass
{

/** The first NUM_TASKTIMINGS roundtrip timings. */
FTiming FTiming::GTaskTimings[NUM_TASKTIMINGS];
/** Number of tasks requested so far. */
volatile int32 FTiming::GTaskRequestCounter = 0;
/** Number of tasks received so far. */
volatile int32 FTiming::GTaskReceiveCounter = 0;

static int32 SwarmConnectionDroppedExitCode = 2;

/**
 * Constructs the Swarm wrapper used by Lightmass.
 * @param SwarmInterface	The global SwarmInterface to use
 * @param JobGuid			Guid that identifies the job we're working on
 * @param TaskQueueSize		Number of tasks we should try to keep in the queue
 */
FLightmassSwarm::FLightmassSwarm( NSwarm::FSwarmInterface& SwarmInterface, const FGuid& InJobGuid, int32 TaskQueueSize )
:	API( SwarmInterface )
,	JobGuid(InJobGuid)
,	bIsDone(false)
,	QuitRequest(false)
,	TaskQueue(TaskQueueSize)
,	NumRequestedTasks(0)
,	TotalBytesRead(0)
,	TotalBytesWritten(0)
,	TotalSecondsRead(0.0)
,	TotalSecondsWritten(0.0)
,	TotalNumReads(0)
,	TotalNumWrites(0)
{
	API.SetJobGuid( JobGuid );

	NSwarm::TLogFlags ConnectionLogFlags = NSwarm::SWARM_LOG_NONE;
	if (GReportDetailedStats)
	{
		ConnectionLogFlags = NSwarm::SWARM_LOG_ALL;
	}
	FString OptionsFolder = FPaths::Combine(*FPaths::GameAgnosticSavedDir(), TEXT("Swarm"));
	OptionsFolder = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*OptionsFolder);
	bool bConnectionEstablished = API.OpenConnection(SwarmCallback, this, ConnectionLogFlags, *OptionsFolder) >= 0;
	checkf(bConnectionEstablished, TEXT("Tried to open a connection to Swarm, but failed"));
}

FLightmassSwarm::~FLightmassSwarm()
{
	API.CloseConnection();
}

/** 
 * Opens a new channel alongside the root channel owned by the LightmassSwarm object
 * 
 * @param ChannelName Name of the channel
 * @param ChannelFlags Flags (read, write, etc) for the channel
 * @param bPushChannel If true, this new channel will be auto-pushed onto the stack
 *
 * @return The channel that was opened, without disturbing the root channel
 */
int32 FLightmassSwarm::OpenChannel( const TCHAR* ChannelName, int32 ChannelFlags, bool bPushChannel )
{
	int32 NewChannel = API.OpenChannel( ChannelName, ( NSwarm::TChannelFlags )ChannelFlags );
	if ((NewChannel == NSwarm::SWARM_ERROR_CONNECTION_NOT_FOUND) ||
		(NewChannel == NSwarm::SWARM_ERROR_CONNECTION_DISCONNECTED))
	{
		// The connection has dropped, exit with a special code
		exit(SwarmConnectionDroppedExitCode);
	}

	if (bPushChannel)
	{
		PushChannel(NewChannel);
	}

	return NewChannel;
}

/** 
 * Closes a channel previously opened with OpenChannel
 * 
 * @param Channel The channel to close
 */
void FLightmassSwarm::CloseChannel( int32 Channel )
{
	int32 ReturnCode = API.CloseChannel(Channel);
	if ((ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_NOT_FOUND) ||
		(ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_DISCONNECTED))
	{
		// The connection has dropped, exit with a special code
		exit(SwarmConnectionDroppedExitCode);
	}
}

/**
 * Pushes a new channel on the stack as the current channel to read from 
 * 
 * @param Channel New channel to read from
 */
void FLightmassSwarm::PushChannel(int32 Channel)
{
	FScopeLock Lock(&SwarmAccess);
	ChannelStack.Push(Channel);
}

/**
 * Pops the top channel
 *
 * @param bCloseChannel If true, the channel will be closed when it is popped off
 */
void FLightmassSwarm::PopChannel(bool bCloseChannel)
{
	int32 PoppedChannel;
	{
		FScopeLock Lock(&SwarmAccess);
		PoppedChannel = ChannelStack.Pop();
	}

	if (bCloseChannel)
	{
		int32 ReturnCode = API.CloseChannel(PoppedChannel);
		if ((ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_NOT_FOUND) ||
			(ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_DISCONNECTED))
		{
			// The connection has dropped, exit with a special code
			exit(SwarmConnectionDroppedExitCode);
		}
	}
}

/**
 * Prefetch tasks into our work queue, which is where RequestTask gets
 * its work items from.
 */
void FLightmassSwarm::PrefetchTasks()
{
	if ( GDebugMode == false )
	{
		// Request the initial TaskQueueSize tasks from Swarm.
		for ( int32 RequestIndex=0; RequestIndex < TaskQueue.GetMaxNumElements(); ++RequestIndex )
		{
			FTiming::NotifyTaskRequested();
			FPlatformAtomics::InterlockedIncrement( &NumRequestedTasks );
			SendMessage( NSwarm::FMessage( NSwarm::MESSAGE_TASK_REQUEST ) );
		}
	}
}

/**
 * Thread-safe blocking call to request a new task from the local task queue.
 * It will block until there's a task in the local queue, or the timeout period has passed.
 * If a task is returned, it will asynchronously request a new task from Swarm to keep the queue full.
 * You must call AcceptTask() or RejectTask() if a task is returned.
 *
 * @param TaskGuid	[out] When successful, contains the FGuid for the new task
 * @param WaitTime	Timeout period in milliseconds, or -1 for INFINITE
 * @return			true if a Task Guid is returned, false if the timeout period has passed or IsDone()/ReceivedQuitRequest() is true
 */
bool FLightmassSwarm::RequestTask( FGuid& TaskGuid, uint32 WaitTime )
{
	// If we've gotten all tasks there is, don't wait for more.
	if ( bIsDone )
	{
		WaitTime = 0;
	}

	if ( !QuitRequest && TaskQueue.Pop( TaskGuid, WaitTime ) )
	{
		if ( !bIsDone )
		{
			if (GReportDetailedStats)
			{
				// Notify Swarm that we're starting work on this Task
	 			SendMessage( NSwarm::FTaskState( TaskGuid, NSwarm::TJobTaskState::JOB_TASK_STATE_RUNNING ) );
			}

			// Request a new task from Swarm.
			FTiming::NotifyTaskRequested();
			FPlatformAtomics::InterlockedIncrement( &NumRequestedTasks );
			SendMessage( NSwarm::FMessage( NSwarm::MESSAGE_TASK_REQUEST ) );
		}
		return true;
	}
	return false;
}

/**
 * Accepts a requested task. This will also notify Unreal.
 * @param TaskGuid	The task that is being accepted
 */
void FLightmassSwarm::AcceptTask( const FGuid& TaskGuid )
{
	if (GReportDetailedStats)
	{
	 	SendMessage( NSwarm::FTaskState(TaskGuid, NSwarm::JOB_TASK_STATE_ACCEPTED) );
	}
}

/**
 * Rejects a requested task. This will also notify Unreal.
 * @param TaskGuid	The task that is being rejected
 */
void FLightmassSwarm::RejectTask( const FGuid& TaskGuid )
{
	SendMessage( NSwarm::FTaskState(TaskGuid, NSwarm::JOB_TASK_STATE_REJECTED) );
}

/**
 * Tells Swarm that the task is completed and all results have been fully exported. This will also notify Unreal.
 * @param TaskGuid	A guid that identifies the task that has been completed
 */
void FLightmassSwarm::TaskCompleted( const FGuid& TaskGuid )
{
	SendMessage( NSwarm::FTaskState(TaskGuid, NSwarm::JOB_TASK_STATE_COMPLETE_SUCCESS) );
}

/**
 * Tells Swarm that the task has failed. This will also notify Unreal.
 * @param TaskGuid	A guid that identifies the task that has failed
 */
void FLightmassSwarm::TaskFailed( const FGuid& TaskGuid )
{
	SendMessage( NSwarm::FTaskState(TaskGuid, NSwarm::JOB_TASK_STATE_COMPLETE_FAILURE) );
}

/**
 * Sends a message to Swarm. Thread-safe access.
 * @param Message	Swarm message to send.
 */
void FLightmassSwarm::SendMessage( const NSwarm::FMessage& Message )
{
	double StartTime = FPlatformTime::Seconds();
	int32 ReturnCode = API.SendMessage( Message );
	if ((ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_NOT_FOUND) ||
		(ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_DISCONNECTED))
	{
		// The connection has dropped, exit with a special code
		exit(SwarmConnectionDroppedExitCode);
	}
	GStatistics.SendMessageTime += FPlatformTime::Seconds() - StartTime;
}

/**
 *	Sends an alert message to Swarm. Thread-safe access.
 *	@param	AlertType		The type of alert.
 *	@param	ObjectGuid		The GUID of the object associated with the alert.
 *	@param	TypeId			The type of object.
 *	@param	MessageText		The text of the message.
 */
void FLightmassSwarm::SendAlertMessage(	NSwarm::TAlertLevel AlertLevel, 
	const FGuid& ObjectGuid, const int32 TypeId, const TCHAR* MessageText)
{
	double StartTime = FPlatformTime::Seconds();

	NSwarm::FAlertMessage AlertMessage(JobGuid, AlertLevel, ObjectGuid, TypeId, MessageText);
	int32 ReturnCode = API.SendMessage( AlertMessage );
	if ((ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_NOT_FOUND) ||
		(ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_DISCONNECTED))
	{
		// The connection has dropped, exit with a special code
		exit(SwarmConnectionDroppedExitCode);
	}
	
	GStatistics.SendMessageTime += FPlatformTime::Seconds() - StartTime;
}


/**
 * Sends text information to Swarm.
 */
void FLightmassSwarm::SendTextMessageImpl(const TCHAR* Buffer)
{
	// Log it locally.
	GLog->Log( Buffer );

	// Send to Swarm.
	SendMessage( NSwarm::FInfoMessage( Buffer ) );
}

/**
 * Report to Swarm by sending back a file.
 * @param Filename	File to send back
 */
void FLightmassSwarm::ReportFile( const TCHAR* Filename )
{
	//@TODO: Send back the log file to Instigator
//	int32 ReturnCode = API.AddChannel( Filename, Filename );
// 	if ((ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_NOT_FOUND) ||
// 		(ReturnCode == NSwarm::SWARM_ERROR_CONNECTION_DISCONNECTED))
// 	{
// 		// The connection has dropped, exit with a special code
// 		exit(SwarmConnectionDroppedExitCode);
// 	}
}

/**
 * Triggers the task queue enough times to release all blocked threads.
 */
void FLightmassSwarm::TriggerAllThreads()
{
	TaskQueue.TriggerAll();
}

/**
 * The callback function used by Swarm to communicate to Lightmass.
 * @param CallbackMessage	Message sent from Swarm
 * @param UserParam			User-defined parameter (specified in OpenConnection). Type-casted FLightmassSwarm pointer.
 */
void FLightmassSwarm::SwarmCallback( NSwarm::FMessage* CallbackMessage, void* UserParam )
{
	FLightmassSwarm* This = (FLightmassSwarm*) UserParam;

	// Always handle QUIT messages.
	if ( CallbackMessage->Type == NSwarm::MESSAGE_QUIT )
	{
		This->QuitRequest = true;
		This->TriggerAllThreads();
		return;
	}

	// Are we requesting a task?
	if ( This->NumRequestedTasks > 0 )
	{
		switch ( CallbackMessage->Type )
		{
			case NSwarm::MESSAGE_TASK_REQUEST_RESPONSE:
			{
				NSwarm::FTaskRequestResponse* Response = (NSwarm::FTaskRequestResponse*) CallbackMessage;
				if ( Response->ResponseType == NSwarm::RESPONSE_TYPE_RELEASE )
				{
					// No more tasks to give us.
					This->bIsDone = true;
					// Decrement our request status and notify a thread.
					FPlatformAtomics::InterlockedDecrement( &This->NumRequestedTasks );
					// Notify all waiting threads (which will make them shut down).
					This->TriggerAllThreads();
				}
				else if ( Response->ResponseType == NSwarm::RESPONSE_TYPE_SPECIFICATION )
				{
					FTiming::NotifyTaskReceived();
					NSwarm::FTaskSpecification* TaskSpec = (NSwarm::FTaskSpecification*) Response;
					if ( This->QuitRequest || This->TaskQueue.Push(TaskSpec->TaskGuid) == false )
					{
						UE_LOGF(LogLightmass, Log, "SwarmCallback - Rejecting task! (Already shutting down, or task queue overflow.)");
						// We got a task but we've already been told to quit, or our queue is erroneously full!
						// Reject the task.
						This->SendMessage( NSwarm::FTaskState(TaskSpec->TaskGuid, NSwarm::JOB_TASK_STATE_REJECTED) );
					}

					// Decrement our request status and notify one thread.
					FPlatformAtomics::InterlockedDecrement( &This->NumRequestedTasks );
				}
				break;
			}
		}
	}
	// Default response behavior.
	else
	{
		switch ( CallbackMessage->Type )
		{
			case NSwarm::MESSAGE_TASK_REQUEST_RESPONSE:
			{
				NSwarm::FTaskRequestResponse* Response = (NSwarm::FTaskRequestResponse*) CallbackMessage;
				if ( Response->ResponseType == NSwarm::RESPONSE_TYPE_SPECIFICATION )
				{
					UE_LOGF(LogLightmass, Log, "SwarmCallback - Rejecting task! (it wasn't requested.)");
					// We're not accepting any tasks at this time.
					NSwarm::FTaskSpecification* TaskSpec = (NSwarm::FTaskSpecification*) Response;
					This->SendMessage( NSwarm::FTaskState(TaskSpec->TaskGuid, NSwarm::JOB_TASK_STATE_REJECTED) );
					break;
				}
			}
		}
	}
}

}	//Lightmass
