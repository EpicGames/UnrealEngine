// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef SWARMINTERFACE_API
#define SWARMINTERFACE_API
#endif

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"
#include "LMThreading.h"
#include "Editor/SwarmInterface/Public/SwarmDefines.h"
#include "Editor/SwarmInterface/Public/SwarmInterface.h"
#include "Misc/StringBuilder.h"
#include "Templates/IsValidVariadicFunctionArg.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "ImportExport.h"

namespace Lightmass
{

/** Whether to enable channel I/O via Swarm - disable for performance debugging */
#define SWARM_ENABLE_CHANNEL_READS	1
#define SWARM_ENABLE_CHANNEL_WRITES	1

/** Flags to use when opening the different kinds of output channels */
/** MUST PAIR APPROPRIATELY WITH THE SAME FLAGS IN UE5 */
static const int32 LM_TEXTUREMAPPING_CHANNEL_FLAGS	= NSwarm::SWARM_JOB_CHANNEL_WRITE;
static const int32 LM_VOLUMESAMPLES_CHANNEL_FLAGS		= NSwarm::SWARM_JOB_CHANNEL_WRITE;
static const int32 LM_PRECOMPUTEDVISIBILITY_CHANNEL_FLAGS	= NSwarm::SWARM_JOB_CHANNEL_WRITE;
static const int32 LM_VOLUMEDEBUGOUTPUT_CHANNEL_FLAGS	= NSwarm::SWARM_JOB_CHANNEL_WRITE;
static const int32 LM_DOMINANTSHADOW_CHANNEL_FLAGS	= NSwarm::SWARM_JOB_CHANNEL_WRITE;
static const int32 LM_MESHAREALIGHT_CHANNEL_FLAGS		= NSwarm::SWARM_JOB_CHANNEL_WRITE;
static const int32 LM_DEBUGOUTPUT_CHANNEL_FLAGS		= NSwarm::SWARM_JOB_CHANNEL_WRITE;

/** Flags to use when opening the different kinds of input channels */
/** MUST PAIR APPROPRIATELY WITH THE SAME FLAGS IN UE5 */
#if LM_COMPRESS_INPUT_DATA
	static const int32 LM_SCENE_CHANNEL_FLAGS			= NSwarm::SWARM_JOB_CHANNEL_READ | NSwarm::SWARM_CHANNEL_MISC_ENABLE_COMPRESSION;
	static const int32 LM_STATICMESH_CHANNEL_FLAGS	= NSwarm::SWARM_CHANNEL_READ | NSwarm::SWARM_CHANNEL_MISC_ENABLE_COMPRESSION;
	static const int32 LM_MATERIAL_CHANNEL_FLAGS		= NSwarm::SWARM_CHANNEL_READ | NSwarm::SWARM_CHANNEL_MISC_ENABLE_COMPRESSION;
#else
	static const int32 LM_SCENE_CHANNEL_FLAGS			= NSwarm::SWARM_JOB_CHANNEL_READ;
	static const int32 LM_STATICMESH_CHANNEL_FLAGS	= NSwarm::SWARM_CHANNEL_READ;
	static const int32 LM_MATERIAL_CHANNEL_FLAGS		= NSwarm::SWARM_CHANNEL_READ;
#endif

class FLightmassSwarm
{
public:
	/**
	 * Constructs the Swarm wrapper used by Lightmass.
	 * @param SwarmInterface	The global SwarmInterface to use
	 * @param JobGuid			Guid that identifies the job we're working on
	 * @param TaskQueueSize		Number of tasks we should try to keep in the queue
	 */
	FLightmassSwarm( NSwarm::FSwarmInterface& SwarmInterface, const FGuid& JobGuid, int32 TaskQueueSize );

	/** Destructor */
	~FLightmassSwarm();

	/**
	 * @retrurn the currently active channel for reading
	 */
	int32 GetChannel()
	{
		checkf(ChannelStack.Num() > 0, TEXT("Tried to get a channel, but none exists"));
		return ChannelStack[ChannelStack.Num() - 1];
	}

	/**
	 * Returns the current job guid.
	 * @return	Current Swarm job guid
	 */
	const FGuid& GetJobGuid()
	{
		return JobGuid;
	}

	/** 
	 * Opens a new channel and optionally pushes it on to the channel stack
	 * 
	 * @param ChannelName Name of the channel
	 * @param ChannelFlags Flags (read, write, etc) for the channel
	 * @param bPushChannel If true, this new channel will be auto-pushed onto the stack
	 *
	 * @return The channel that was opened
	 */
	int32 OpenChannel( const TCHAR* ChannelName, int32 ChannelFlags, bool bPushChannel );

	/** 
	 * Closes a channel previously opened with OpenSideChannel
	 * 
	 * @param Channel The channel to close
	 */
	void CloseChannel( int32 Channel );

	/**
	 * Pushes a new channel on the stack as the current channel to read from 
	 * 
	 * @param Channel New channel to read from
	 */
	void PushChannel(int32 Channel);

	/**
	 * Pops the top channel
	 *
	 * @param bCloseChannel If true, the channel will be closed when it is popped off
	 */
	void PopChannel(bool bCloseChannel);

	/**
	 * Function that closes and pops current channel
	 */
	void CloseCurrentChannel()
	{
		PopChannel(true);
	}

	/**
	 * Reads data from the current channel
	 *
	 * @param Data Data to read from the channel
	 * @param Size Size of Data
	 *
	 * @return Amount read
	 */
	int32 Read(void* Data, int32 Size)
	{
		TotalNumReads++;
		TotalSecondsRead -= FPlatformTime::Seconds();
#if SWARM_ENABLE_CHANNEL_READS
		int32 NumRead = API.ReadChannel(GetChannel(), Data, Size);
#else
 		int32 NumRead = 0;
#endif
		TotalBytesRead += NumRead;
		TotalSecondsRead += FPlatformTime::Seconds();
		return NumRead;
	}

	/**
	 * Writes data to the current channel
	 *
	 * @param Data Data to write over the channel
	 * @param Size Size of Data
	 *
	 * @return Amount written
	 */
	int32 Write(const void* Data, int32 Size)
	{
		TotalNumWrites++;
		TotalSecondsWritten -= FPlatformTime::Seconds();
#if SWARM_ENABLE_CHANNEL_WRITES
		int32 NumWritten = API.WriteChannel(GetChannel(), Data, Size);
#else
		int32 NumWritten = 0;
#endif
		TotalBytesWritten += NumWritten;
		TotalSecondsWritten += FPlatformTime::Seconds();
		return NumWritten;
	}

	/**
	 * The callback function used by Swarm to communicate to Lightmass.
	 * @param CallbackMessage	Message sent from Swarm
	 * @param UserParam			User-defined parameter (specified in OpenConnection). Type-casted FLightmassSwarm pointer.
	 */
	static void SwarmCallback( NSwarm::FMessage* CallbackMessage, void* CallbackData );

	/**
	 * Whether Swarm wants us to quit.
	 * @return	true if Swarm has told us to quit.
	 */
	bool ReceivedQuitRequest() const
	{
		return QuitRequest;
	}

	/**
	 * Whether we've received all tasks already and there are no more tasks to work on.
	 * @return	true if there are no more tasks in the job to work on
	 */
	bool IsDone() const
	{
		return bIsDone;
	}

	/**
	 * Prefetch tasks into our work queue, which is where RequestTask gets
	 * its work items from.
	 */
	void	PrefetchTasks();

	/**
	 * Thread-safe blocking call to request a new task from the local task queue.
	 * It will block until there's a task in the local queue, or the timeout period has passed.
	 * If a task is returned, it will asynchronously request a new task from Swarm to keep the queue full.
	 * You must call AcceptTask() or RejectTask() if a task is returned.
	 *
	 * @param TaskGuid	[out] When successful, contains the FGuid for the new task
	 * @param WaitTime	Timeout period in milliseconds, or -1 for INFINITE
	 * @return			true if a Task Guid is returned, false if the timeout period has passed or IsDone()/ReceivedQuitRequest() is true
	 */
	bool	RequestTask( FGuid& TaskGuid, uint32 WaitTime = uint32(-1) );

	/**
	 * Accepts a requested task. This will also notify UE5.
	 * @param TaskGuid	The task that is being accepted
	 */
	void	AcceptTask( const FGuid& TaskGuid );

	/**
	 * Rejects a requested task. This will also notify UE5.
	 * @param TaskGuid	The task that is being rejected
	 */
	void	RejectTask( const FGuid& TaskGuid );

	/**
	 * Tells Swarm that the task is completed and all results have been fully exported. This will also notify UE5.
	 * @param TaskGuid	A guid that identifies the task that has been completed
	 */
	void	TaskCompleted( const FGuid& TaskGuid );

	/**
	 * Tells Swarm that the task has failed. This will also notify UE5.
	 * @param TaskGuid	A guid that identifies the task that has failed
	 */
	void	TaskFailed( const FGuid& TaskGuid );

	/**
	 * Sends text information to Swarm, using printf-like parameters.
	 */
	template <typename FmtType, typename... Types>
	void SendTextMessage(const FmtType& Fmt, Types... Args)
	{
		SendTextMessageImpl(TStringBuilder<1024>().Appendf(Fmt, Args...).ToString());
	}

private:
	void SendTextMessageImpl(const TCHAR* Buffer);
public:

	/**
	 * Report to Swarm by sending back a file.
	 * @param Filename	File to send back
	 */
	void	ReportFile( const TCHAR* Filename );

	/**
	 * Sends a message to Swarm. Thread-safe access.
	 * @param Message	Swarm message to send.
	 */
	void	SendMessage( const NSwarm::FMessage& Message );

	/**
	 *	Sends an alert message to Swarm. Thread-safe access.
	 *	@param	AlertType		The type of alert.
	 *	@param	ObjectGuid		The GUID of the object associated with the alert.
	 *	@param	TypeId			The type of object.
	 *	@param	MessageText		The text of the message.
	 */
	void	SendAlertMessage(	NSwarm::TAlertLevel AlertLevel, 
								const FGuid& ObjectGuid,
								const int32 TypeId,
								const TCHAR* MessageText);

	/**
	 * @return the total number of bytes that have been read from Swarm
	 */
	uint64 GetTotalBytesRead()
	{
		return TotalBytesRead;
	}

	/**
	 * @return the total number of bytes that have been read from Swarm
	 */
	uint64 GetTotalBytesWritten()
	{
		return TotalBytesWritten;
	}

	/**
	 * @return the total number of seconds spent reading from Swarm
	 */
	double GetTotalSecondsRead()
	{
		return TotalSecondsRead;
	}

	/**
	 * @return the total number of seconds spent writing to Swarm
	 */
	double GetTotalSecondsWritten()
	{
		return TotalSecondsWritten;
	}

	/**
	 * @return the total number of reads from Swarm
	 */
	uint32 GetTotalNumReads()
	{
		return TotalNumReads;
	}

	/**
	 * @return the total number of writes to Swarm
	 */
	uint32 GetTotalNumWrites()
	{
		return TotalNumWrites;
	}

private:

	/**
	 * Triggers the task queue enough times to release all blocked threads.
	 */
	void	TriggerAllThreads();

	/** The Swarm interface. */
	NSwarm::FSwarmInterface&	API;

	/** The job guid (the same as the scene guid) */
	FGuid						JobGuid;

	/** Critical section to synchronize any access to the Swarm API (sending messages, etc). */
	FCriticalSection			SwarmAccess;

	/** true if there are no more tasks in the job. */
	bool						bIsDone;

	/** Set to true when a QUIT message is received from Swarm. */
	bool						QuitRequest;

	/** Tasks that have been received from Swarm but not yet handed out to a worker thread. */
	TProducerConsumerQueue<FGuid>	TaskQueue;

	/** Number of outstanding task requests. */
	volatile int32				NumRequestedTasks;

	/** Stack of used channels */
	TArray<int32>					ChannelStack;

	/** Total bytes read/written */
	uint64						TotalBytesRead;
	uint64						TotalBytesWritten;

	/** Total number of seconds spent reading from Swarm */
	double						TotalSecondsRead;
	/** Total number of seconds spent writing to Swarm */
	double						TotalSecondsWritten;
	/** Total number of reads to Swarm */
	uint32						TotalNumReads;
	/** Total number of writes to Swarm */
	uint32						TotalNumWrites;
};

// Number of task-requests to skip before timing a roundtrip
#define TASKTIMING_FREQ		50

// Number of roundtrip timings to capture
#define NUM_TASKTIMINGS		100

/** Helper struct for request-task/receive-task roundtrip timings */
struct FTiming
{
	/** Constructor that clears the timing. */
	FTiming()
	{
		Clear();
	}
	/** Clear the timing. */
	void		Clear()
	{
		StartTime = 0.0;
		Duration = 0.0;
	}
	/** Start the timing. */
	void		Start( )
	{
		StartTime = FPlatformTime::Seconds();
	}
	/** Stop the timing. */
	void		Stop()
	{
		Duration = FPlatformTime::Seconds() - StartTime;
	}
	/** Start time, as measured by an FPlatformTime::Seconds() timestamp. */
	double		StartTime;
	/** Duration of the timing, in seconds. */
	double		Duration;

	/** Call when requesting a task. */
	static FORCEINLINE void NotifyTaskRequested()
	{
		int32 CurrCounter = FPlatformAtomics::InterlockedIncrement(&GTaskRequestCounter);
		if ( (CurrCounter % TASKTIMING_FREQ) == 0 )
		{
			int32 CurrTiming = CurrCounter / TASKTIMING_FREQ;
			if ( CurrTiming < NUM_TASKTIMINGS )
			{
				GTaskTimings[CurrTiming].Start();
			}
		}
	}

	/** Call when receiving a task. */
	static FORCEINLINE void NotifyTaskReceived()
	{
		int32 CurrCounter = FPlatformAtomics::InterlockedIncrement(&GTaskReceiveCounter);
		if ( (CurrCounter % TASKTIMING_FREQ) == 0 )
		{
			int32 CurrTiming = CurrCounter / TASKTIMING_FREQ;
			if ( CurrTiming < NUM_TASKTIMINGS )
			{
				GTaskTimings[CurrTiming].Stop();
			}
		}
	}

	/** Returns the average roundtrip time for the timings captured so far, in seconds. */
	static double GetAverageTiming()
	{
		int32 NumTimings = FMath::Min<int32>(GTaskRequestCounter, GTaskReceiveCounter);
		NumTimings = FMath::Min<int32>(NumTimings, NUM_TASKTIMINGS);
		double TotalDuration = 0.0;
		for ( int32 TimingIndex=1; TimingIndex < NumTimings; ++TimingIndex )
		{
			TotalDuration += GTaskTimings[TimingIndex].Duration;
		}
		return TotalDuration / NumTimings;
	}

	/** The first NUM_TASKTIMINGS roundtrip timings. */
	static FTiming GTaskTimings[NUM_TASKTIMINGS];
	/** Number of tasks requested so far. */
	static volatile int32 GTaskRequestCounter;
	/** Number of tasks received so far. */
	static volatile int32 GTaskReceiveCounter;
};


}	//Lightmass
