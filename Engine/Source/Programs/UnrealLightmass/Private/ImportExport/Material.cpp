// Copyright Epic Games, Inc. All Rights Reserved.

#include "Material.h"
#include "Importer.h"

namespace Lightmass
{
	// The logic of these function needs to match SubstrateMaterial.cpp
	bool IsOpaqueOrMaskedBlendMode(EBlendMode BlendMode)	{ return BlendMode == BLEND_Opaque || BlendMode == BLEND_Masked; }
	bool IsTranslucentOnlyBlendMode(EBlendMode BlendMode)	{ return BlendMode == BLEND_Translucent || BlendMode == BLEND_TranslucentColoredTransmittance || BlendMode == BLEND_TranslucentGreyTransmittance; }
	bool IsAlphaHoldoutBlendMode(EBlendMode BlendMode)		{ return BlendMode == BLEND_AlphaHoldout; }
	bool IsModulateBlendMode(EBlendMode BlendMode)			{ return BlendMode == BLEND_Modulate || BlendMode == BLEND_ColoredTransmittanceOnly; }
	bool IsAdditiveBlendMode(EBlendMode BlendMode)			{ return BlendMode == BLEND_Additive; }
	bool IsAlphaCompositeBlendMode(EBlendMode BlendMode)	{ return BlendMode == BLEND_AlphaComposite; }

	//----------------------------------------------------------------------------
	//	Material base class
	//----------------------------------------------------------------------------
	void FBaseMaterial::Import( class FLightmassImporter& Importer )
	{
		verify(Importer.ImportData((FBaseMaterialData*)this));
	}

	//----------------------------------------------------------------------------
	//	Material class
	//----------------------------------------------------------------------------
	void FMaterial::Import( class FLightmassImporter& Importer )
	{
		// import super class
		FBaseMaterial::Import(Importer);

		// import the shared data structure
		verify(Importer.ImportData((FMaterialData*)this));

		// import the actual material samples...
		int32 ReadSize;

		// Emissive
		ReadSize = EmissiveSize * EmissiveSize;
		checkf(ReadSize > 0, TEXT("Failed to import emissive data!"));
		MaterialEmissive.Init(TF_ARGB16F, EmissiveSize, EmissiveSize);
		verify(Importer.Read(MaterialEmissive.GetData(), ReadSize * sizeof(FFloat16Color)));

		// Diffuse
		ReadSize = DiffuseSize * DiffuseSize;
		if (ReadSize > 0)
		{
			MaterialDiffuse.Init(TF_ARGB16F, DiffuseSize, DiffuseSize);
			verify(Importer.Read(MaterialDiffuse.GetData(), ReadSize * sizeof(FFloat16Color)));
		}
		else
		{
			// Opaque materials should always import diffuse
			check(!IsOpaqueOrMaskedBlendMode(BlendMode));
		}

		// Transmission
		ReadSize = TransmissionSize * TransmissionSize;
		if (ReadSize > 0)
		{
			MaterialTransmission.Init(TF_ARGB16F, TransmissionSize, TransmissionSize);
			verify(Importer.Read(MaterialTransmission.GetData(), ReadSize * sizeof(FFloat16Color)));
		}
		else
		{
			// Materials with a translucent blend mode should always import transmission
			check(!IsTranslucentOnlyBlendMode(BlendMode) && !IsAdditiveBlendMode(BlendMode) && !IsModulateBlendMode(BlendMode) && !IsAlphaCompositeBlendMode(BlendMode) && !IsAlphaHoldoutBlendMode(BlendMode))

		}

		// Normal
		ReadSize = NormalSize * NormalSize;
		if( ReadSize > 0 )
		{
			MaterialNormal.Init(TF_ARGB16F, NormalSize, NormalSize);
			verify(Importer.Read(MaterialNormal.GetData(), ReadSize * sizeof(FFloat16Color)));
		}
	}

}	// namespace Lightmass


