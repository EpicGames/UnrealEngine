// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
// @todo UE5: This seems wrong to me to be needed (scene uses mesh, not the other way around), and is only needed for the MAX_TEXCOORDS
#include "MaterialExport.h"
#include "SceneExport.h"
#include "Texture.h"

namespace Lightmass
{
	//----------------------------------------------------------------------------
	//	Material base class
	//----------------------------------------------------------------------------

	class FBaseMaterial : public FBaseMaterialData
	{
	public:
		virtual ~FBaseMaterial() { }
		virtual void	Import( class FLightmassImporter& Importer );
	};


	//----------------------------------------------------------------------------
	//	Material class
	//----------------------------------------------------------------------------
	class FMaterial : public FBaseMaterial, public FMaterialData
	{
	public:
		virtual void	Import( class FLightmassImporter& Importer );

		/** Sample the various properties of this material at the given UV */
		inline void SampleEmissive(const FVector2f& UV, FLinearColor& Emissive, float& OutEmissiveBoost) const
		{
			Emissive = MaterialEmissive.Sample(UV);
			OutEmissiveBoost = EmissiveBoost;
		}
		inline void SampleDiffuse(const FVector2f& UV, FLinearColor& Diffuse, float& OutDiffuseBoost) const
		{
			Diffuse = MaterialDiffuse.Sample(UV);
			OutDiffuseBoost = DiffuseBoost;
		}
		inline FLinearColor SampleTransmission(const FVector2f& UV) const
		{
			return MaterialTransmission.Sample(UV);
		}
		inline void SampleNormal(const FVector2f& UV, FVector4f& Normal) const
		{
			Normal = MaterialNormal.SampleNormal(UV);
			Normal.W = 0.0f;
			Normal = Normal.GetSafeNormal();
			if( Normal.SizeSquared3() < KINDA_SMALL_NUMBER )
			{
				Normal.Set( 0.0f, 0.0f, 1.0f, 0.0f );
			}
		}

	protected:
		FTexture2D MaterialEmissive;
		FTexture2D MaterialDiffuse;
		FTexture2D MaterialTransmission;
		FTexture2D MaterialNormal;
	};

}	// namespace Lightmass


