// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh.h"
#include "Importer.h"
#include "UnrealLightmass.h"

namespace Lightmass
{

//----------------------------------------------------------------------------
//	Mesh base class
//----------------------------------------------------------------------------

void FBaseMesh::Import( FLightmassImporter& Importer )
{
	Importer.ImportData( (FBaseMeshData*)this );
}

//----------------------------------------------------------------------------
//	Static mesh class
//----------------------------------------------------------------------------

void FStaticMesh::Import( FLightmassImporter& Importer )
{
	// import super class
	FBaseMesh::Import(Importer);

	// import the shared data structure
	Importer.ImportData( (FStaticMeshData*)this );

	UE_LOGF(LogLightmass, Verbose, "Importing a static mesh with %d LODs [%ls]", NumLODs, *Guid.ToString());
	checkf(NumLODs > 0, TEXT("Imported a static mesh with 0 LODs. Uhoh"));

	// create the LODs
	LODs.Empty(NumLODs);

	// import each of the LODs
	for (uint32 LODIndex = 0; LODIndex < NumLODs; LODIndex++)
	{
		FStaticMeshLOD* LOD = new(LODs) FStaticMeshLOD;

		// import each LOD separately
		LOD->Import(Importer);
	}
}

//----------------------------------------------------------------------------
//	Static mesh LOD class
//----------------------------------------------------------------------------

void FStaticMeshLOD::Import( class FLightmassImporter& Importer )
{
	// import the shared data structure
	Importer.ImportData((FStaticMeshLODData*)this);

	Importer.ImportArray(Elements, NumElements);

	for (uint32 MeshElementIndex = 0; MeshElementIndex < NumElements; MeshElementIndex++)
	{
		// Only triangle lists are supported for now
		check(Elements[MeshElementIndex].FirstIndex % 3 == 0);
	}
	Importer.ImportArray(Indices, NumIndices);
	Importer.ImportArray(Vertices, NumVertices);
}


}
