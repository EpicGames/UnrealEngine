// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
// @todo UE5: This seems wrong to me to be needed (scene uses mesh, not the other way around), and is only needed for the MAX_TEXCOORDS
#include "MeshExport.h"
#include "SceneExport.h"


namespace Lightmass
{

class FStaticMeshLOD;
class FStaticMeshElement;


//----------------------------------------------------------------------------
//	Mesh base class
//----------------------------------------------------------------------------

class FBaseMesh : public FBaseMeshData
{
public:
	virtual ~FBaseMesh() { }
	virtual void			Import( class FLightmassImporter& Importer );
};


//----------------------------------------------------------------------------
//	Static mesh class
//----------------------------------------------------------------------------

class FStaticMesh : public FBaseMesh, public FStaticMeshData
{
public:
	virtual void			Import( class FLightmassImporter& Importer );

	/**
	 * @return the given LOD by index
	 */
	inline const FStaticMeshLOD&	GetLOD(int32 Index) const
	{
		return LODs[Index];
	}

	class FDefaultAggregateMesh* VoxelizationMesh = nullptr;

protected:
	/** array of LODs (same number as FStaticMeshData.NumLODs) */
	TArray<FStaticMeshLOD>		LODs;
};


//----------------------------------------------------------------------------
//	Static mesh LOD class
//----------------------------------------------------------------------------

class FStaticMeshLOD : public FStaticMeshLODData
{
public:
	virtual ~FStaticMeshLOD() { }
	virtual void			Import( class FLightmassImporter& Importer );

	/**
	 * @return the given element by index
	 */
	inline const FStaticMeshElement&	GetElement(int32 Index) const
	{
		return Elements[Index];
	}

	/**
	 * @return the given vertex index by index
	 */
	inline const uint32					GetIndex(int32 Index) const
	{
		return Indices[Index];
	}

	/**
	 * @return the given vertex by index
	 */
	inline const FStaticMeshVertex&		GetVertex(int32 Index) const
	{
		return Vertices[Index];
	}
protected:
	/** array of Elements for this LOD (same number as FStaticMeshLODData.NumElements) */
	TArray<FStaticMeshElement>	Elements;

	/** array of Indices for this LOD (same number as FStaticMeshLODData.NumIndices) */
	TArray<uint32>				Indices;

	/** array of vertices for this LOD (same number as FStaticMeshLODData.NumVertices) */
	TArray<FStaticMeshVertex>	Vertices;
};


//----------------------------------------------------------------------------
//	Static mesh element class
//----------------------------------------------------------------------------

class FStaticMeshElement : public FStaticMeshElementData
{
public:
protected:
};

}	// namespace Lightmass


