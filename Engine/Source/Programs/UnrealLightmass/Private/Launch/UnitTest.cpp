// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTest.h"
#include "LMkDOP.h"
#include "Math/LMOctree.h"
#include "UnrealLightmass.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Guid.h"
#include "LightmassScene.h"

namespace Lightmass
{

/** Defines how the light is stored in the scene's light octree. */
struct FTestOctreeSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };
	enum { LoosenessDenominator = 16 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	FORCEINLINE static FBoxCenterAndExtent GetBoundingBox(const float& Element)
	{
		return FBoxCenterAndExtent(FVector4f(0), FVector4f(Element));
	}

	FORCEINLINE static bool AreElementsEqual(const float& A,const float& B)
	{
		return A == B;
	}
};


class FTestCollisionDataProvider
{
	TkDOPTree<FTestCollisionDataProvider, uint16>& kDOP;
	FVector4f Vertex;
public:

	FTestCollisionDataProvider(TkDOPTree<FTestCollisionDataProvider, uint16>& InkDOP)
		: kDOP(InkDOP)
		, Vertex(0)
	{
	}

	/**
	 * Given an index, returns the position of the vertex
	 *
	 * @param Index the index into the vertices array
	 */
	FORCEINLINE const FVector4f& GetVertex(uint16 Index) const
	{
		return Vertex;
	}

	/** Returns additional information. */
	FORCEINLINE int32 GetItemIndex(uint16 MaterialIndex) const
	{
		return 0;
	}

	/**
	 * Returns the kDOPTree for this mesh
	 */
	FORCEINLINE const TkDOPTree<FTestCollisionDataProvider,uint16>& GetkDOPTree(void) const
	{
		return kDOP;
	}
	/**
	 * Returns the local to world for the component
	 */
	FORCEINLINE const FMatrix44f& GetLocalToWorld(void) const
	{
		return FMatrix44f::Identity;
	}

	/**
	 * Returns the world to local for the component
	 */
	FORCEINLINE const FMatrix44f GetWorldToLocal(void) const
	{
		return FMatrix44f::Identity;
	}

	/**
	 * Returns the local to world transpose adjoint for the component
	 */
	FORCEINLINE FMatrix44f GetLocalToWorldTransposeAdjoint(void) const
	{
		return FMatrix44f::Identity;
	}

	/**
	 * Returns the determinant for the component
	 */
	FORCEINLINE float GetDeterminant(void) const
	{
		return 0.0f;
	}
};

class FTestRunnable : public FRunnable
{
	bool bStop;
public:
	virtual bool Init(void) 
	{
		bStop = false;
		return true;
	}

	virtual uint32 Run(void)
	{
		for (int32 i = 0; i < 10 && !bStop; i++)
		{
			UE_LOGF(LogLightmass, Display, "Thread counter %d", i);
			FPlatformProcess::Sleep(1.0f);
		}
		UE_LOGF(LogLightmass, Display, "Thread done!!");

		return 0;
	}

	virtual void Stop(void)
	{
		bStop = true;
	}

	virtual void Exit(void)
	{
	}

};

void TestLightmass()
{
	UE_LOGF(LogLightmass, Display, "\n\n");
	UE_LOGF(LogLightmass, Display, "===============================================================================================");
	UE_LOGF(LogLightmass, Display, "Running \"unit test\". This will take several seconds, and will end with an assertion.");
	UE_LOGF(LogLightmass, Display, "This is on purpose, as it's testing the callstack gathering...");
	UE_LOGF(LogLightmass, Display, "===============================================================================================");
	UE_LOGF(LogLightmass, Display, "\n\n");

	void* Buf = FMemory::Malloc(1024);

	TArray<int32> TestArray;

	TestArray.Add(5);

	TArray<int32> ArrayCopy = TestArray;
	
	FVector4f TestVectorA(1, 0, 0, 1);
	FVector4f TestVectorB(1, 1, 1, 1);
	FVector4f TestVector = TestVectorA + TestVectorB;

	FString TestString = FString::Printf(TEXT("Copy has %d, Vector is [%.2f, %.2f, %.2f, %.2f]\n"), ArrayCopy[0], TestVector.X, TestVector.Y, TestVector.Z, TestVector.W);

	printf("%s", TCHAR_TO_UTF8(*TestString));

	FMemory::Free(Buf);

	struct FAlignTester
	{
		uint8 A;
		FMatrix44f M1;
		uint8 B;
		FMatrix44f M2;
		uint8 C;
		FVector4f V;
	};
	FAlignTester AlignTest;
	checkf(((PTRINT)(&FMatrix44f::Identity) & 15) == 0, TEXT("Identity matrix unaligned"));
	checkf(((PTRINT)(&AlignTest.M1) & 15) == 0, TEXT("First matrix unaligned"));
	checkf(((PTRINT)(&AlignTest.M2) & 15) == 0, TEXT("Second matrix unaligned"));
	checkf(((PTRINT)(&AlignTest.V) & 15) == 0, TEXT("Vector unaligned"));

	FGuid Guid(1, 2, 3, 4);
	UE_LOGF(LogLightmass, Display, "Guid is %ls", *Guid.ToString());

	TMap<FString, int32> TestMap;
	TestMap.Add(FString(TEXT("Five")), 5);
	TestMap.Add(TEXT("Ten"), 10);

	UE_LOGF(LogLightmass, Display, "Map[Five] = %d, Map[Ten] = %d", TestMap.FindRef(TEXT("Five")), TestMap.FindRef(FString(TEXT("Ten"))));

	FMatrix44f TestMatrix(FVector3f(0, 0, 0.1f), FVector3f(0, 1.0f, 0), FVector3f(0.9f, 0, 0), FVector3f(0, 0, 0));

	UE_LOGF(LogLightmass, Display, "Mat=\n  [%0.2f, %0.2f, %0.2f, %0.2f]\n  [%0.2f, %0.2f, %0.2f, %0.2f]\n  [%0.2f, %0.2f, %0.2f, %0.2f]\n  [%0.2f, %0.2f, %0.2f, %0.2f]", 
		TestMatrix.M[0][0], TestMatrix.M[0][1], TestMatrix.M[0][2], TestMatrix.M[0][3], 
		TestMatrix.M[1][0], TestMatrix.M[1][1], TestMatrix.M[1][2], TestMatrix.M[1][3], 
		TestMatrix.M[2][0], TestMatrix.M[2][1], TestMatrix.M[2][2], TestMatrix.M[2][3], 
		TestMatrix.M[3][0], TestMatrix.M[3][1], TestMatrix.M[3][2], TestMatrix.M[3][3]
		);

	TestMatrix = TestMatrix.GetTransposed();

	UE_LOGF(LogLightmass, Display, "Transposed Mat=\n  [%0.2f, %0.2f, %0.2f, %0.2f]\n  [%0.2f, %0.2f, %0.2f, %0.2f]\n  [%0.2f, %0.2f, %0.2f, %0.2f]\n  [%0.2f, %0.2f, %0.2f, %0.2f]", 
		TestMatrix.M[0][0], TestMatrix.M[0][1], TestMatrix.M[0][2], TestMatrix.M[0][3], 
		TestMatrix.M[1][0], TestMatrix.M[1][1], TestMatrix.M[1][2], TestMatrix.M[1][3], 
		TestMatrix.M[2][0], TestMatrix.M[2][1], TestMatrix.M[2][2], TestMatrix.M[2][3], 
		TestMatrix.M[3][0], TestMatrix.M[3][1], TestMatrix.M[3][2], TestMatrix.M[3][3]
		);

	TestMatrix = TestMatrix.GetTransposed().InverseFast();

	UE_LOGF(LogLightmass, Display, "Inverted Mat=\n  [%0.2f, %0.2f, %0.2f, %0.2f]\n  [%0.2f, %0.2f, %0.2f, %0.2f]\n  [%0.2f, %0.2f, %0.2f, %0.2f]\n  [%0.2f, %0.2f, %0.2f, %0.2f]", 
		TestMatrix.M[0][0], TestMatrix.M[0][1], TestMatrix.M[0][2], TestMatrix.M[0][3], 
		TestMatrix.M[1][0], TestMatrix.M[1][1], TestMatrix.M[1][2], TestMatrix.M[1][3], 
		TestMatrix.M[2][0], TestMatrix.M[2][1], TestMatrix.M[2][2], TestMatrix.M[2][3], 
		TestMatrix.M[3][0], TestMatrix.M[3][1], TestMatrix.M[3][2], TestMatrix.M[3][3]
		);

	UE_LOGF(LogLightmass, Display, "sizeof FDirectionalLight = %zu, FLight = %zu, FDirectionalLightData = %zu", sizeof(FDirectionalLight), sizeof(FLight), sizeof(FDirectionalLightData));

	TOctree<float, FTestOctreeSemantics> TestOctree(FVector4f(0), 10.0f);
	TestOctree.AddElement(5);


	// kDOP test
	TkDOPTree<FTestCollisionDataProvider, uint16> TestkDOP;
	FTestCollisionDataProvider TestDataProvider(TestkDOP);
	FHitResult TestResult;
	
	FkDOPBuildCollisionTriangle<uint16> TestTri(0, FVector4f(0,0,0,0), FVector4f(1,1,1,0), FVector4f(2,2,2,0), INDEX_NONE, INDEX_NONE, INDEX_NONE, false, true);
	TArray<FkDOPBuildCollisionTriangle<uint16> > TestTriangles;
	TestTriangles.Add(TestTri);

	TestkDOP.Build(TestTriangles);
	
	UE_LOGF(LogLightmass, Display, "\nStarting a thread");
	FTestRunnable* TestRunnable = new FTestRunnable;
	// create a thread with the test runnable, and let it auto-delete itself
	FRunnableThread* TestThread = FRunnableThread::Create(TestRunnable, TEXT("TestRunnable"));


	double Start = FPlatformTime::Seconds();
	UE_LOGF(LogLightmass, Display, "\nWaiting 4 seconds");
	FPlatformProcess::Sleep(4.0f);
	UE_LOGF(LogLightmass, Display, "%.2f seconds have passed, killing thread", FPlatformTime::Seconds() - Start);
	
	// wait for thread to end
	double KillStart = FPlatformTime::Seconds();
	TestRunnable->Stop();
	TestThread->WaitForCompletion();
		
	delete TestThread;
	delete TestRunnable;
	
	UE_LOGF(LogLightmass, Display, "It took %.2f seconds to kill the thread [should be < 1 second]", FPlatformTime::Seconds() - KillStart);


	UE_LOGF(LogLightmass, Display, "\n\n");

	checkf(5 == 2, TEXT("And boom goes the dynamite\n"));
}

} // namespace
