// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Lightmass
{
	/**
	 * Run "unit tests" making use of LightmassCore and other functionality
	 * worth testing
	 */
	void TestLightmass();
}


