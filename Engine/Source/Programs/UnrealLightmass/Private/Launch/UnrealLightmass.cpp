// Copyright Epic Games, Inc. All Rights Reserved.

// UnrealLightmass.cpp : Defines the entry point for the console application.
//

#include "UnrealLightmass.h"
#include "CPUSolver.h"
#include "UnitTest.h"
#include "LightmassSwarm.h"
#include "HAL/ExceptionHandling.h"
#include "RequiredProgramMainCPPInclude.h"
#include "LMDebug.h"
#include "LMHelpers.h"
#include "ImportExport.h"
#include "HAL/PlatformApplicationMisc.h"
#if PLATFORM_LINUX
#include "HAL/PlatformStackWalk.h"
#include "Unix/UnixPlatformCrashContext.h"
#endif

#if USE_LOCAL_SWARM_INTERFACE
#include "IMessagingModule.h"
#endif

DEFINE_LOG_CATEGORY(LogLightmass);

IMPLEMENT_APPLICATION(UnrealLightmass, "UnrealLightmass");

namespace Lightmass
{

/**
 * Compare the output results from 2 lighting results
 *
 * @param Dir1 First directory of mapping file dumps to compare
 * @param Dir2 Seconds directory of mapping file dumps to compare
 */
void CompareLightingResults(const TCHAR* Dir1, const TCHAR* Dir2, float ErrorThreshold);

double GStartupTime = 0.0f;

/**
 * Initialize FCommandLine with C style command line params.
 */
void InitCommandLine(int ArgC, ANSICHAR* ArgV[])
{
	FString CmdLine;

	// loop over the parameters, skipping the first one (which is the executable name)
	for (int32 Arg = 1; Arg < ArgC; Arg++)
	{
		CmdLine += ArgV[Arg];
		// put a space between each argument (not needed after the end)
		if (Arg + 1 < ArgC)
		{
			CmdLine += TEXT(" ");
		}
	}

	FCommandLine::Set(*CmdLine);
}


int LightmassMain(int argc, ANSICHAR* argv[])
{
	GStartupTime = FPlatformTime::Seconds();

	// Create lightmass log file
	GLog->AddOutputDevice( FLightmassLog::Get() );

	// Initialize FCommandLine
	InitCommandLine(argc, argv);

	// Output devices.
	GError = FPlatformApplicationMisc::GetErrorOutputDevice(); 
	GWarn = FPlatformApplicationMisc::GetFeedbackContext();

#if USE_LOCAL_SWARM_INTERFACE
	bool bMessagingMode = false;
	FString CommandLine = FCommandLine::Get();
	if (!FParse::Param(*CommandLine, TEXT("-Messaging")))
	{
		bMessagingMode = true;
		CommandLine += TEXT(" -Messaging");
	}

	// Don't spew all log messages to console (unless user has specified "-stdout" on command line)
	if (!FParse::Param(*CommandLine, TEXT("-stdout")))
	{
		CommandLine += TEXT(" -nostdout");
	}

	GEngineLoop.PreInit(*CommandLine);

	// Tell the module manager is may now process newly-loaded UObjects when new C++ modules are loaded
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	FModuleManager::LoadModuleChecked<IMessagingModule>("Messaging");
	FModuleManager::Get().LoadModule(TEXT("Settings"));
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreDefault);
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostDefault);

	ON_SCOPE_EXIT
	{
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();

		// Similar to CL 16324633. If we shut down FTaskGraphInterface right here, in AppExit
		//   FThreadStats::StopThread could get called and that will try and execute some stats
		//   commands which use FTaskGraphInterface and --> crashes. FEngineLoop::AppExit() calls
		//   FTaskGraphInterface::Shutdown() after calling FThreadStats::StopThread() if needed.
		FEngineLoop::AppExit();
	};
#endif // USE_LOCAL_SWARM_INTERFACE

	UE_LOGF(LogLightmass, Display,  "Lightmass %ls started on: %ls. Command-line: %ls", FPlatformMisc::GetUBTPlatform(), FPlatformProcess::ComputerName(), FCommandLine::Get() );

	// parse commandline options
	bool bRunUnitTest = false;
	bool bDumpTextures = false;
	FGuid SceneGuid(0x0123, 0x4567, 0x89AB, 0xCDEF); // default scene guid if none specified
	int32 NumThreads = FPlatformMisc::NumberOfCoresIncludingHyperthreads(); // default to the number of processors
#if PLATFORM_WINDOWS
	NumThreads = 0;
	int NumProcessorGroups = GetActiveProcessorGroupCount();
	for (int GroupIndex = 0; GroupIndex < NumProcessorGroups; GroupIndex++)
	{
		int NumProcessorsThisGroup = GetActiveProcessorCount(GroupIndex);
		NumThreads += NumProcessorsThisGroup;
	}
#endif
	bool bCompareFiles = false;
	FString File1;
	FString File2;
	float ErrorThreshold = 0.000001f; // default error tolerance to allow in lighting comparisons

	// Override 'NumThreads' with the environment variable, if it's set.
	{
		FString SwarmMaxCoresVariable = FPlatformMisc::GetEnvironmentVariable( TEXT("Swarm_MaxCores") );
		int32 SwarmMaxCores = FCString::Atoi( *SwarmMaxCoresVariable );
		if ( SwarmMaxCores >= 1 && SwarmMaxCores < 128 )
		{
			NumThreads = SwarmMaxCores;
		}
	}

	for (int32 ArgIndex = 1; ArgIndex < argc; ArgIndex++)
	{
		if ((FCStringAnsi::Stricmp(argv[ArgIndex], "-help") == 0) || (FCStringAnsi::Stricmp(argv[ArgIndex], "-?") == 0))
		{
			UE_LOGF(LogLightmass, Display, "Usage:\n  UnrealLightmass\n\t[SceneGuid]\n\t[-debug]\n\t[-unittest]\n\t[-dumptex]\n\t[-numthreads N]\n\t[-compare Dir1 Dir2 [-error N]]");
			UE_LOGF(LogLightmass, Display, "");
			UE_LOGF(LogLightmass, Display, "  SceneGuid : Guid of a scene file. 0x0000012300004567000089AB0000CDEF is the default");
			UE_LOGF(LogLightmass, Display, "  -debug : Processes all mappings in the scene, instead of getting tasks from Swarm Coordinator");
			UE_LOGF(LogLightmass, Display, "  -unittest : Runs a series of validations, then quits");
			UE_LOGF(LogLightmass, Display, "  -dumptex : Outputs .bmp files to the current directory of 2D lightmap/shadowmap results");
			UE_LOGF(LogLightmass, Display, "  -compare : Compares the binary dumps created by UnrealEd to compare Unreal vs LM lighting runs");
			UE_LOGF(LogLightmass, Display, "  -error : Controls the threshold that an error is counted when comparing with -compare");
			return 0;
		}
		else if (FCStringAnsi::Stricmp(argv[ArgIndex], "-unittest") == 0)
		{
			bRunUnitTest = true;
		}
		else if (FCStringAnsi::Stricmp(argv[ArgIndex], "-dumptex") == 0)
		{
			bDumpTextures = true;
		}
		else if (FCStringAnsi::Stricmp(argv[ArgIndex], "-usedebug") == 0)
		{
			// Warning!  This will only process mapping tasks and will skip other types of tasks.
			GDebugMode = true;
		}
		else if (FCStringAnsi::Stricmp(argv[ArgIndex], "-stats") == 0)
		{
			GReportDetailedStats = true;
		}
		else if (FCStringAnsi::Stricmp(argv[ArgIndex], "-numthreads") == 0)
		{
			// use the next parameter as the number of threads (it must exist, or we fail)
			NumThreads = 0;
			if (ArgIndex < argc - 1)
			{
				NumThreads = FCString::Atoi(*FString(argv[++ArgIndex]));
			}

			// validate it
			if (NumThreads == 0)
			{
				UE_LOGF(LogLightmass, Display, "The number of threads was not specified properly, use \"-numthreads N\"");
				return 1;
			}
		}
		else if (FCStringAnsi::Stricmp(argv[ArgIndex], "-compare") == 0)
		{
			bCompareFiles = true;

			if (ArgIndex >= argc - 2)
			{
				UE_LOGF(LogLightmass, Display, "-compare requires two directories to compare (-compare Dir1 Dir2)");
				return 1;
			}
			// cache the files to compare
			File1 = *FString(argv[++ArgIndex]);
			File2 = *FString(argv[++ArgIndex]);
		}
		else if (FCStringAnsi::Stricmp(argv[ArgIndex], "-error") == 0)
		{
			// use the next parameter as the number of threads (it must exist, or we fail)
			if (ArgIndex >= argc - 1)
			{
				UE_LOGF(LogLightmass, Display, "-error requires an error value following (-error N)");
				return 1;
			}

			ErrorThreshold = FCString::Atof(*FString(argv[++ArgIndex]));
		}
		// look for just a Guid on the commandline
		else if (FCStringAnsi::Strlen(argv[ArgIndex]) == 32)
		{
			// break up the string into 4 components
			FString Arg(argv[ArgIndex]);

			// we use _tcstoul to import base 16
#if PLATFORM_USES_MICROSOFT_LIBC_FUNCTIONS
			SceneGuid.A = _tcstoul(*Arg.Mid(0, 8), NULL, 16);
			SceneGuid.B = _tcstoul(*Arg.Mid(8, 8), NULL, 16);
			SceneGuid.C = _tcstoul(*Arg.Mid(16, 8), NULL, 16);
			SceneGuid.D = _tcstoul(*Arg.Mid(24, 8), NULL, 16);
#else
			SceneGuid.A = FCString::Strtoui64(*Arg.Mid(0, 8), NULL, 16);
			SceneGuid.B = FCString::Strtoui64(*Arg.Mid(8, 8), NULL, 16);
			SceneGuid.C = FCString::Strtoui64(*Arg.Mid(16, 8), NULL, 16);
			SceneGuid.D = FCString::Strtoui64(*Arg.Mid(24, 8), NULL, 16);
#endif
		}
	}

	// if we want to run the unit test, do that, then nothing else
	if (bRunUnitTest)
	{
		// this is an ongoing compiler/runtime test for all templates and whatnot
		TestLightmass();
		return 0;
	}

	if (bCompareFiles)
	{
		CompareLightingResults(*File1, *File2, ErrorThreshold);
		return 0;
	}

	// Start the static lighting processing
	UE_LOGF(LogLightmass, Display,  "Processing scene GUID: %08X%08X%08X%08X with %d threads", SceneGuid.A, SceneGuid.B, SceneGuid.C, SceneGuid.D, NumThreads );
	BuildStaticLighting(SceneGuid, NumThreads, bDumpTextures);

#if USE_LOCAL_SWARM_INTERFACE
	if (bMessagingMode)
	{
		float StartTime = FPlatformTime::Seconds();

		do
		{
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FPlatformProcess::Sleep(1.0f);
		} while (FPlatformTime::Seconds() - StartTime < 5.0f);
	}
#endif

	return 0;
}

/**
 * Compare the output results from 2 lighting results
 *
 * @param Filename1 First mapping dump to compare
 * @param Filename2 First mapping dump to compare
 * @param ErrorThreshold Any error less than this is ignored
 *
 * @return Output information, or empty if no differences
 */
FString CompareLightingFiles(const TCHAR* Filename1, const TCHAR* Filename2, float ErrorThreshold)
{
	// open the files and verify they exist
	FArchive* File1 = IFileManager::Get().CreateFileReader(Filename1);
	if (File1 == NULL)
	{
		return FString::Printf(TEXT("File '%s' does not exist!"), Filename1);
	}

	FArchive* File2 = IFileManager::Get().CreateFileReader(Filename2);
	if (File2 == NULL)
	{
		return FString::Printf(TEXT("File '%s' does not exist!"), Filename2);
	}

	// get file sizes
	int64 Size1 = File1->TotalSize();
	int64 Size2 = File2->TotalSize();

	// they must match
	if (Size1 != Size2)
	{
		delete File1;
		delete File2;
		return FString::Printf(TEXT("Files are a different size!"));
	}

	// read in the files
	float* Buf1 = (float*)FMemory::Malloc(Size1);
	float* Buf2 = (float*)FMemory::Malloc(Size1);

	File1->Serialize(Buf1, Size1);
	File2->Serialize(Buf2, Size1);

	delete File1;
	delete File2;

	// compute the number of floats in the buffers
	int32 NumFloats = Size1 / sizeof(float);

	double TotalError = 0;
	float BiggestError = 0;
	int32 NumErrors = 0;
	// compute error over all matches
	for (int32 Index = 0; Index < NumFloats; Index++)
	{
		// get diff between 2 lighting values
		float Error = FMath::Abs(Buf1[Index] - Buf2[Index]);

		// does this error pass our threshold?
		if (Error > ErrorThreshold)
		{
			// add it to the running total
			TotalError += Error;
			NumErrors++;

			// look for biggest
			if (Error > BiggestError)
			{
				BiggestError = Error;
			}
		}
	}

	FMemory::Free(Buf1);
	FMemory::Free(Buf2);

	// return the output if we had any errors
	if (NumErrors > 0)
	{
		return FString::Printf(TEXT("    Error: %0.6f / %d samples, %0.6f avg / %d errors, %0.6f biggest"), TotalError, NumFloats, NumErrors ? TotalError / NumErrors : 0, NumErrors, BiggestError);
	}
	// otherwise, just an empty string
	return TEXT("");
}

class FLocalCompareLightingResultsVisitor : public IPlatformFile::FDirectoryVisitor
{
public:

	FLocalCompareLightingResultsVisitor(const TCHAR* InDir1, const TCHAR* InDir2, float InErrorThreshold)
	:	NumDifferentFiles(0)
	,	TotalFiles(0)
	,	Dir1(InDir1)
	,	Dir2(InDir2)
	,	ErrorThreshold(InErrorThreshold)
	{
	}

	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		if (!bIsDirectory)
		{
			FString Filename(FilenameOrDirectory);
			if (FPaths::GetExtension(Filename) == TEXT("bin"))
			{
				// do the comparison
				FString Output = CompareLightingFiles(
													  *FString::Printf(TEXT("%s/%s"), *Dir1, FilenameOrDirectory),
													  *FString::Printf(TEXT("%s/%s"), *Dir2, FilenameOrDirectory),
													  ErrorThreshold);

				TotalFiles++;
				// if there was any interesting output, show it
				if (Output != TEXT(""))
				{
					UE_LOGF(LogLightmass, Display, "\n  %ls:\n%ls", FilenameOrDirectory, *Output);

					NumDifferentFiles++;
				}
			}
		}

		return true;
	}

	int32 NumDifferentFiles;
	int32 TotalFiles;

private:

	FString Dir1;
	FString Dir2;
	float ErrorThreshold;
};

/**
 * Compare the output results from 2 lighting results
 *
 * @param Dir1 First directory of mapping file dumps to compare
 * @param Dir2 Seconds directory of mapping file dumps to compare
 * @param ErrorThreshold Any error less than this is ignored
 */
void CompareLightingResults(const TCHAR* Dir1, const TCHAR* Dir2, float ErrorThreshold)
{
	UE_LOGF(LogLightmass, Display, "");
	UE_LOGF(LogLightmass, Display, "Comparing '%ls' vs '%ls'", Dir1, Dir2);

	FLocalCompareLightingResultsVisitor Visitor(Dir1, Dir2, ErrorThreshold);
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.IterateDirectory(Dir1, Visitor);

	UE_LOGF(LogLightmass, Display, "\nFound %d issues (out of %d mappings)...", Visitor.NumDifferentFiles, Visitor.TotalFiles);
}

void CriticalErrorCallback()
{
	// Try to notify Swarm about the critical error.
	const FString& CrashReporterURL = appGetCrashReporterURL();
	if ( GSwarm )
	{
		GSwarm->SendTextMessage( TEXT("*** CRITICAL ERROR! Machine: %s"), FPlatformProcess::ComputerName() );
		GSwarm->SendTextMessage( TEXT("*** CRITICAL ERROR! Logfile: %s"), FLightmassLog::Get()->GetLogFilename() );
		GSwarm->SendTextMessage( TEXT("*** CRITICAL ERROR! Crash report: %s"), *CrashReporterURL );
		GSwarm->ReportFile( FLightmassLog::Get()->GetLogFilename() );
		delete GSwarm;
		GSwarm = NULL;
	}
	else
	{
		UE_LOGF(LogLightmass, Display, "--- Critical Error! Machine: %ls. Logfile: %ls. Crash report: %ls. ---", FPlatformProcess::ComputerName(), FLightmassLog::Get()->GetLogFilename(), TEXT("")/**CrashReporterURL*/ );
	}
}

#if PLATFORM_WINDOWS

bool VerifyDLL( const TCHAR* DLLFilename )
{
	HMODULE DbgHelpDll = LoadLibrary( DLLFilename );
	if ( DbgHelpDll == NULL )
	{
		UE_LOGF(LogLightmass, Display,  "Failed to load %ls!", DLLFilename );
		return false;
	}
	return true;
}

void SendSwarmCriticalErrorMessage()
{
	FString ErrorLog = (FString(TEXT("=== Lightmass crashed: ===")) + GErrorExceptionDescription + LINE_TERMINATOR) + GErrorHist;

	// For editor log
	GSwarm->SendMessage(NSwarm::FInfoMessage(*ErrorLog));
	// For lighting results dialog.  Can't use critical error here as that will cause the editor to assert.
	GSwarm->SendAlertMessage(NSwarm::ALERT_LEVEL_ERROR, FGuid(), SOURCEOBJECTTYPE_Unknown, *ErrorLog);
}

#elif PLATFORM_LINUX

static void UnrealLightmassUnixCrashHandler(const FGenericCrashContext& GenericContext)
{
	const uint32 DumpCallstackSize = 2047;
	ANSICHAR DumpCallstack[DumpCallstackSize] = { 0 };
	const FUnixCrashContext& Context = static_cast< const FUnixCrashContext& >( GenericContext );

	FPlatformStackWalk::StackWalkAndDump(DumpCallstack, DumpCallstackSize, 2);

	GSwarm->SendTextMessage(TEXT("\n=== Lightmass crashed (Signal=%d): ==="), Context.Signal);
	GSwarm->SendTextMessage(TEXT("Callstack:\n%s"), UTF8_TO_TCHAR(DumpCallstack));
	GSwarm->SendTextMessage(TEXT("*** CRITICAL ERROR! Machine: %s"), FPlatformProcess::ComputerName() );
	GSwarm->SendTextMessage(TEXT("*** CRITICAL ERROR! Logfile: %s"), FLightmassLog::Get()->GetLogFilename() );
	GLog->Flush();

	// remove the handler for this signal and re-raise it (which should generate the proper core dump)
	// print message to stdout directly, it may be too late for the log (doesn't seem to be printed during a crash in the thread) 
	fprintf(stderr, "UnrealLightmass re-raising signal %d for the default handler. Good bye.\n", Context.Signal);

	struct sigaction ResetToDefaultAction;
	FMemory::Memzero(ResetToDefaultAction);
	ResetToDefaultAction.sa_handler = SIG_DFL;
	sigfillset(&ResetToDefaultAction.sa_mask);
	sigaction(Context.Signal, &ResetToDefaultAction, nullptr);

	raise(Context.Signal);
}

#endif // PLATFORM_LINUX

} // namespace Lightmass

int main(int argc, ANSICHAR* argv[])
{
	Lightmass::GStatistics.TotalTimeStart = FPlatformTime::Seconds();

	int32 ErrorLevel = 0;

	GUseCrashReportClient = false;

#if PLATFORM_WINDOWS
	// Set the error mode to avoid popping up dialog boxes on crashes
	SetErrorMode( SEM_NOGPFAULTERRORBOX | SEM_NOGPFAULTERRORBOX );

	// Verify the required DLLs
	if ( !Lightmass::VerifyDLL(TEXT("dbghelp.dll")) )
	{
		return 1;
	}
#endif

#if PLATFORM_MAC
 	if ( true )
#else
	if ( FPlatformMisc::IsDebuggerPresent() )
#endif
	{
		// Don't use exception handling when a debugger is attached to exactly trap the crash.
		ErrorLevel = Lightmass::LightmassMain(argc, argv);
	}
#if PLATFORM_LINUX
	else
	{
		FPlatformMisc::SetCrashHandler(Lightmass::UnrealLightmassUnixCrashHandler);

		GIsGuarded = true;
		// Run the guarded code.
		ErrorLevel = Lightmass::LightmassMain(argc, argv);
		GIsGuarded = false;
	}
#elif PLATFORM_WINDOWS
	else
	{
		// Use structured exception handling to trap any crashes, walk the the stack and display a crash dialog box.
		__try
		{
			__try
			{
				__try
				{
					GIsGuarded = true;
					// Run the guarded code.
					ErrorLevel = Lightmass::LightmassMain(argc, argv);
					GIsGuarded = false;
				}
				__except( ReportCrash( GetExceptionInformation() ) )
				{
					printf( "Exception handled in main, crash report generated, re-throwing exception\n" );

					// With the crash report created, propagate the error. Per the MSDN documentation
					// on GetExceptionInformation(), the flags and parameters are not available outside
					// the exception filter (i.e. ReportCrash) as they were probably stack-pointers.
					RaiseException(GetExceptionCode(), 0, 0, nullptr);
				}
			}
			__except( EXCEPTION_EXECUTE_HANDLER )
			{
				printf( "Exception handled in main, calling appHandleCriticalError\n" );
				// Crashed
				ErrorLevel = 1;
				Lightmass::SendSwarmCriticalErrorMessage();
				Lightmass::appHandleCriticalError();
				Lightmass::CriticalErrorCallback();
			}
		}
		__except( EXCEPTION_EXECUTE_HANDLER )
		{
			// Do nothing except prevent the crash
			printf( "Exception handled in main, attempting to prevent application crash\n" );
		}
	}
#endif

	Lightmass::GStatistics.TotalTimeEnd = FPlatformTime::Seconds();
	return ErrorLevel;
}
