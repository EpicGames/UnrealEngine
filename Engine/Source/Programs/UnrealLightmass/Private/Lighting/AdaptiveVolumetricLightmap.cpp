// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LightingSystem.h"
#include "MonteCarlo.h"
#include "Raster.h"
#include "HAL/PlatformTime.h"

namespace Lightmass
{

struct FTriangle 
{
	FVector3f Vertices[3];
};

struct FOverlapInterval
{
	float Min;
	float Max;
};

FOverlapInterval GetInterval(const FTriangle& Triangle, const FVector3f& Vector) 
{
	FOverlapInterval Result;
	Result.Min = Result.Max = FVector3f::DotProduct(Vector, Triangle.Vertices[0]);

	for (int32 i = 1; i < 3; ++i) 
	{
		float Projection = FVector3f::DotProduct(Vector, Triangle.Vertices[i]);
		Result.Min = FMath::Min(Result.Min, Projection);
		Result.Max = FMath::Max(Result.Max, Projection);
	}

	return Result;
}

FOverlapInterval GetInterval(const FBox3f& Box, const FVector3f& Vector) 
{
	FVector3f BoxVertices[8] = 
	{
		FVector3f(Box.Min.X, Box.Max.Y, Box.Max.Z),
		FVector3f(Box.Min.X, Box.Max.Y, Box.Min.Z),
		FVector3f(Box.Min.X, Box.Min.Y, Box.Max.Z),
		FVector3f(Box.Min.X, Box.Min.Y, Box.Min.Z),
		FVector3f(Box.Max.X, Box.Max.Y, Box.Max.Z),
		FVector3f(Box.Max.X, Box.Max.Y, Box.Min.Z),
		FVector3f(Box.Max.X, Box.Min.Y, Box.Max.Z),
		FVector3f(Box.Max.X, Box.Min.Y, Box.Min.Z)
	};

	FOverlapInterval Result;
	Result.Min = Result.Max = FVector3f::DotProduct(Vector, BoxVertices[0]);

	for (int32 i = 1; i < UE_ARRAY_COUNT(BoxVertices); ++i) 
	{
		float Projection = FVector3f::DotProduct(Vector, BoxVertices[i]);
		Result.Min = FMath::Min(Result.Min, Projection);
		Result.Max = FMath::Max(Result.Max, Projection);
	}

	return Result;
}

bool OverlapOnAxis(const FBox3f& Box, const FTriangle& Triangle, const FVector3f& Vector) 
{
	FOverlapInterval A = GetInterval(Box, Vector);
	FOverlapInterval B = GetInterval(Triangle, Vector);
	return ((B.Min <= A.Max) && (A.Min <= B.Max));
}

bool IntersectTriangleAndAABB(const FTriangle& Triangle, const FBox3f& Box) 
{
	FVector3f TriangleEdge0 = Triangle.Vertices[1] - Triangle.Vertices[0]; 
	FVector3f TriangleEdge1 = Triangle.Vertices[2] - Triangle.Vertices[1]; 
	FVector3f TriangleEdge2 = Triangle.Vertices[0] - Triangle.Vertices[2]; 

	FVector3f BoxNormal0(1.0f, 0.0f, 0.0f);
	FVector3f BoxNormal1(0.0f, 1.0f, 0.0f);
	FVector3f BoxNormal2(0.0f, 0.0f, 1.0f);

	FVector3f TestDirections[13] = 
	{
		// Separating axes from the box normals
		BoxNormal0, 
		BoxNormal1, 
		BoxNormal2,
		// One separating axis for the triangle normal
		FVector3f::CrossProduct(TriangleEdge0, TriangleEdge1),
		// Separating axes for the triangle edges
		FVector3f::CrossProduct(BoxNormal0, TriangleEdge0),
		FVector3f::CrossProduct(BoxNormal0, TriangleEdge1),
		FVector3f::CrossProduct(BoxNormal0, TriangleEdge2),
		FVector3f::CrossProduct(BoxNormal1, TriangleEdge0),
		FVector3f::CrossProduct(BoxNormal1, TriangleEdge1),
		FVector3f::CrossProduct(BoxNormal1, TriangleEdge2),
		FVector3f::CrossProduct(BoxNormal2, TriangleEdge0),
		FVector3f::CrossProduct(BoxNormal2, TriangleEdge1),
		FVector3f::CrossProduct(BoxNormal2, TriangleEdge2)
	};

	for (int i = 0; i < UE_ARRAY_COUNT(TestDirections); ++i) 
	{
		if (!OverlapOnAxis(Box, Triangle, TestDirections[i])) 
		{
			// If we don't overlap on a single axis, the shapes do not intersect
			return false;
		}
	}

	return true;
}

FVector3f GetBaryCentric2D(const FVector3f& Point, const FVector3f& A, const FVector3f& B, const FVector3f& C)
{
	FVector3f::FReal a = ((B.Y-C.Y)*(Point.X-C.X) + (C.X-B.X)*(Point.Y-C.Y)) / ((B.Y-C.Y)*(A.X-C.X) + (C.X-B.X)*(A.Y-C.Y));
	FVector3f::FReal b = ((C.Y-A.Y)*(Point.X-C.X) + (A.X-C.X)*(Point.Y-C.Y)) / ((B.Y-C.Y)*(A.X-C.X) + (C.X-B.X)*(A.Y-C.Y));

	return FVector3f(a, b, 1.0f - a - b);	
}

bool PointUnderTriangle(FVector3f Point, FTriangle Triangle)
{
	const FVector3f BaryCentricCoordinate = GetBaryCentric2D(Point, Triangle.Vertices[0], Triangle.Vertices[1], Triangle.Vertices[2]);
	const float PointOnTriangleZ = Triangle.Vertices[0].Z * BaryCentricCoordinate.X + Triangle.Vertices[1].Z * BaryCentricCoordinate.Y + Triangle.Vertices[2].Z * BaryCentricCoordinate.Z;

	return BaryCentricCoordinate.X >= 0 && BaryCentricCoordinate.Y >= 0 && BaryCentricCoordinate.Z >= 0 && PointOnTriangleZ > Point.Z;
}

bool FStaticLightingSystem::DoesVoxelIntersectSceneGeometry(const FBox3f& CellBounds, FGuid& OutIntersectingLevelGuid) const
{
	const float Child2dTriangleArea = .5f * CellBounds.GetSize().X * CellBounds.GetSize().Y / (VolumetricLightmapSettings.BrickSize * VolumetricLightmapSettings.BrickSize);
	const float SurfaceLightmapDensityThreshold = .5f * VolumetricLightmapSettings.SurfaceLightmapMinTexelsPerVoxelAxis * VolumetricLightmapSettings.SurfaceLightmapMinTexelsPerVoxelAxis / Child2dTriangleArea;

	const FBox3f ExpandedCellBoundsSurfaceGeometry = CellBounds.ExpandBy(CellBounds.GetSize() * VolumetricLightmapSettings.VoxelizationCellExpansionForSurfaceGeometry);
	const FBox3f ExpandedCellBoundsVolumeGeometry = CellBounds.ExpandBy(CellBounds.GetSize() * VolumetricLightmapSettings.VoxelizationCellExpansionForVolumeGeometry);

	if (Scene.GeneralSettings.bUseFastVoxelization)
	{
		const FStaticLightingMesh* Mesh = nullptr;
		Mesh = VoxelizationSurfaceAggregateMesh->IntersectBox(ExpandedCellBoundsSurfaceGeometry);
		if (Mesh != nullptr)
		{
			OutIntersectingLevelGuid = Mesh->LevelGuid;
			return true;
		}

		Mesh = VoxelizationVolumeAggregateMesh->IntersectBox(ExpandedCellBoundsVolumeGeometry);
		if (Mesh != nullptr)
		{
			OutIntersectingLevelGuid = Mesh->LevelGuid;
			return true;
		}

		for (int32 MeshIndex = 0; MeshIndex < Scene.StaticMeshInstances.Num(); MeshIndex++)
		{
			const FStaticMeshStaticLightingMesh* MeshInstance = &Scene.StaticMeshInstances[MeshIndex];

			if (MeshInstance->StaticMesh->VoxelizationMesh != nullptr)
			{
				if (MeshInstance->Mapping->GetVolumeMapping() == nullptr)
				{
					if (MeshInstance->StaticMesh->VoxelizationMesh->IntersectBox(ExpandedCellBoundsSurfaceGeometry.TransformBy(MeshInstance->WorldToLocal)) != nullptr)
					{
						OutIntersectingLevelGuid = MeshInstance->LevelGuid;
						return true;
					}
				}
				else
				{
					if (MeshInstance->StaticMesh->VoxelizationMesh->IntersectBox(ExpandedCellBoundsVolumeGeometry.TransformBy(MeshInstance->WorldToLocal)) != nullptr)
					{
						OutIntersectingLevelGuid = MeshInstance->LevelGuid;
						return true;
					}
				}
			}
		}

		return false;
	}
	else
	{
		for (int32 MappingIndex = 0; MappingIndex < AllMappings.Num(); MappingIndex++)
		{
			const FStaticLightingMapping* CurrentMapping = AllMappings[MappingIndex];
			const FStaticLightingTextureMapping* TextureMapping = CurrentMapping->GetTextureMapping();
			const FStaticLightingMesh* CurrentMesh = CurrentMapping->Mesh;
			const FBox3f& ExpandedCellBounds = CurrentMapping->GetVolumeMapping() ? ExpandedCellBoundsVolumeGeometry : ExpandedCellBoundsSurfaceGeometry;

			const bool bMeshBelongsToLOD0 = CurrentMesh->DoesMeshBelongToLOD0();

			if ((CurrentMesh->LightingFlags & GI_INSTANCE_CASTSHADOW)
				&& CurrentMesh->BoundingBox.Intersect(ExpandedCellBounds)
				&& bMeshBelongsToLOD0)
			{
				for (int32 TriangleIndex = 0; TriangleIndex < CurrentMesh->NumTriangles; TriangleIndex++)
				{
					FStaticLightingVertex Vertices[3];
					int32 ElementIndex;
					CurrentMesh->GetTriangle(TriangleIndex, Vertices[0], Vertices[1], Vertices[2], ElementIndex);

					if (CurrentMesh->IsElementCastingShadow(ElementIndex)
						// Lightmass doesn't handle bounced light from translucency
						&& !CurrentMesh->IsTranslucent(ElementIndex))
					{
						FTriangle Triangle;
						Triangle.Vertices[0] = Vertices[0].WorldPosition;
						Triangle.Vertices[1] = Vertices[1].WorldPosition;
						Triangle.Vertices[2] = Vertices[2].WorldPosition;

						FBox3f TriangleAABB(Triangle.Vertices, UE_ARRAY_COUNT(Triangle.Vertices));

						if (ExpandedCellBounds.Intersect(TriangleAABB))
						{
							const FVector4f TriangleNormal = (Vertices[2].WorldPosition - Vertices[0].WorldPosition) ^ (Vertices[1].WorldPosition - Vertices[0].WorldPosition);
							const float TriangleArea = 0.5f * TriangleNormal.Size3();

							if (TriangleArea > DELTA)
							{
								if (TextureMapping)
								{
									// Triangle vertices in lightmap UV space, scaled by the lightmap resolution
									const FVector2f Vertex0 = Vertices[0].TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->SizeX, TextureMapping->SizeY);
									const FVector2f Vertex1 = Vertices[1].TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->SizeX, TextureMapping->SizeY);
									const FVector2f Vertex2 = Vertices[2].TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->SizeX, TextureMapping->SizeY);

									// Area in lightmap space, or the number of lightmap texels covered by this triangle
									const float LightmapTriangleArea = FMath::Abs(
										Vertex0.X * (Vertex1.Y - Vertex2.Y)
										+ Vertex1.X * (Vertex2.Y - Vertex0.Y)
										+ Vertex2.X * (Vertex0.Y - Vertex1.Y));

									const float TexelDensity = LightmapTriangleArea / TriangleArea;
									// Skip texture lightmapped triangles whose texel density is less than one texel per the area of a right triangle formed by the child voxel.
									// If surface lighting is being calculated at a low resolution, it's unlikely that the volume near that surface needs to have detailed lighting.
									if (TexelDensity < SurfaceLightmapDensityThreshold)
									{
										continue;
									}
								}

								if (IntersectTriangleAndAABB(Triangle, ExpandedCellBounds))
								{
									OutIntersectingLevelGuid = CurrentMesh->LevelGuid;
									return true;
								}
							}
						}
					}
				}
			}
		}

		return false;
	}
}

bool FStaticLightingSystem::ShouldRefineVoxel(int32 TreeDepth, const FBox3f& CellBounds, const TArray<FVector3f>& VoxelTestPositions, bool bDebugThisVoxel, FGuid& OutIntersectingLevelGuid) const
{
	const bool bCellInsideImportanceVolume = Scene.IsBoxInImportanceVolume(CellBounds);

	// The volumetric lightmap bounds are larger than the importance volume bounds, since we force the volumetric lightmap volume to have cube voxels
	if (!bCellInsideImportanceVolume)
	{
		return false;
	}

	FIntPoint VolumeRange(INT_MAX, INT_MAX);
	bool bAnyDensityVolumeFound = false;

	for (int32 SampleIndex = 0; SampleIndex < VoxelTestPositions.Num(); SampleIndex++)
	{
		FVector3f SamplePosition = CellBounds.Min + VoxelTestPositions[SampleIndex] * CellBounds.GetSize();
		FIntPoint SampleAllowedMipRange;
		bool bSampleInDensityVolume = Scene.GetVolumetricLightmapAllowedMipRange(SamplePosition, SampleAllowedMipRange);
		
		if (bSampleInDensityVolume)
		{
			bAnyDensityVolumeFound = true;
			VolumeRange.X = FMath::Min(VolumeRange.X, SampleAllowedMipRange.X);
			VolumeRange.Y = FMath::Min(VolumeRange.Y, SampleAllowedMipRange.Y);
		}
	}

	// Default unrestricted
	FIntPoint AllowedMipRange(0, INT_MAX);

	if (bAnyDensityVolumeFound)
	{
		AllowedMipRange = VolumeRange;
	}

	const int32 CandidateMipLevel = VolumetricLightmapSettings.MaxRefinementLevels - TreeDepth - 1;

	if (CandidateMipLevel < AllowedMipRange.X)
	{
		return false;
	}

	if (CandidateMipLevel >= AllowedMipRange.Y)
	{
		return true;
	}

	bool bVoxelIntersectsScene = DoesVoxelIntersectSceneGeometry(CellBounds, OutIntersectingLevelGuid);
	
	if (!bVoxelIntersectsScene)
	{
		const FBox3f ExpandedCellBounds = CellBounds.ExpandBy(CellBounds.GetExtent() * VolumetricLightmapSettings.VoxelizationCellExpansionForLights);
		FBoxSphereBounds3f ExpandedBoxSphereBounds(ExpandedCellBounds);

		for (int32 LightIndex = 0; LightIndex < Lights.Num() && !bVoxelIntersectsScene; LightIndex++)
		{
			const FLight* Light = Lights[LightIndex];

			if ((Light->GetSpotLight() || Light->GetPointLight()) 
				&& (Light->LightFlags & GI_LIGHT_HASSTATICLIGHTING)
				// Refine around static lights, where lighting is going to be changing rapidly
				&& Light->AffectsBounds(ExpandedBoxSphereBounds))
			{
				const FSphere3f LightBounds = Light->GetBoundingSphere();

				// If the light is smaller than the voxel, subdivide regardless of light brightness, since we will likely undersample it
				if (LightBounds.W < ExpandedBoxSphereBounds.SphereRadius)
				{
					bVoxelIntersectsScene = true;
				}
				else
				{
					for (int32 SampleIndex = 0; SampleIndex < VoxelTestPositions.Num(); SampleIndex++)
					{
						FVector3f SamplePosition = ExpandedCellBounds.Min + VoxelTestPositions[SampleIndex] * ExpandedCellBounds.GetSize();
						FLinearColor DirectLighting = Light->GetDirectIntensity(SamplePosition, false);

						if (DirectLighting.GetLuminance() > VolumetricLightmapSettings.LightBrightnessSubdivideThreshold)
						{
							// Only subdivide if the light has a significant effect on this voxel
							bVoxelIntersectsScene = true;
							break;
						}
					}
				}
			}
		}
	}

	if (bVoxelIntersectsScene 
		&& LandscapeMappings.Num() > 0
		&& VolumetricLightmapSettings.bCullBricksBelowLandscape)
	{
		if (Scene.GeneralSettings.bUseFastVoxelization)
		{
			FBox3f StretchedCellBounds(CellBounds.Min, FVector3f(CellBounds.Max.X, CellBounds.Max.Y, LandscapeCullingVoxelizationAggregateMesh->GetBounds().Max.Z));

			if (LandscapeCullingVoxelizationAggregateMesh->IntersectBox(StretchedCellBounds) && !LandscapeCullingVoxelizationAggregateMesh->IntersectBox(CellBounds))
			{
				return false;
			}
		}
		else
		{
			TArray<FVector3f, TInlineAllocator<100>> TestPositions;
			TArray<bool, TInlineAllocator<100>> PositionUnderLandscape;

			int32 TestResolution = 10;
			TestPositions.Empty(TestResolution * TestResolution);
			PositionUnderLandscape.Empty(TestResolution * TestResolution);
			PositionUnderLandscape.AddZeroed(TestResolution * TestResolution);

			for (int32 Y = 0; Y < TestResolution; Y++)
			{
				for (int32 X = 0; X < TestResolution; X++)
				{
					const FVector3f TestPosition = CellBounds.Min + FVector3f(X / (float)TestResolution, Y / (float)TestResolution, 1.0f) * CellBounds.GetSize();
					TestPositions.Add(TestPosition);
				}
			}

			for (int32 MappingIndex = 0; MappingIndex < LandscapeMappings.Num(); MappingIndex++)
			{
				const FStaticLightingMapping* CurrentMapping = LandscapeMappings[MappingIndex];
				const FStaticLightingMesh* CurrentMesh = CurrentMapping->Mesh;

				if ((CurrentMesh->LightingFlags & GI_INSTANCE_CASTSHADOW)
					&& CurrentMesh->BoundingBox.IntersectXY(CellBounds))
				{
					for (int32 TriangleIndex = 0; TriangleIndex < CurrentMesh->NumTriangles; TriangleIndex++)
					{
						FStaticLightingVertex Vertices[3];
						int32 ElementIndex;
						CurrentMesh->GetTriangle(TriangleIndex, Vertices[0], Vertices[1], Vertices[2], ElementIndex);

						if (CurrentMesh->IsElementCastingShadow(ElementIndex))
						{
							FTriangle Triangle;
							Triangle.Vertices[0] = Vertices[0].WorldPosition;
							Triangle.Vertices[1] = Vertices[1].WorldPosition;
							Triangle.Vertices[2] = Vertices[2].WorldPosition;

							for (int32 PointIndex = 0; PointIndex < PositionUnderLandscape.Num(); PointIndex++)
							{
								if (PointUnderTriangle(TestPositions[PointIndex], Triangle))
								{
									PositionUnderLandscape[PointIndex] = true;
								}
							}
						}
					}
				}
			}

			bool bAllPointsUnderLandscape = true;

			for (int32 PointIndex = 0; PointIndex < PositionUnderLandscape.Num(); PointIndex++)
			{
				bAllPointsUnderLandscape = bAllPointsUnderLandscape && PositionUnderLandscape[PointIndex];
			}

			return !bAllPointsUnderLandscape;
		}
	}

	return bVoxelIntersectsScene;
}

struct FIrradianceBrickBuildData
{
	FGuid IntersectingLevelGuid;
	FIntVector LocalCellCoordinate;
	int32 TreeDepth;
	bool bHasChildren;
	bool bDebugBrick;
};

// Requires texel selecting something to get into debug mode.  
bool bDebugVolumetricLightmapCell = false;
FVector3f DebugWorldPosition(548.092041f, 36.330334f, 832.141907f);

void FStaticLightingSystem::RecursivelyBuildBrickTree(
	int32 StartCellIndex,
	int32 NumCells,
	FIntVector LocalCellCoordinate, 
	int32 TreeDepth, 
	bool bCoveringDebugPosition,
	const FBox3f& TopLevelCellBounds, 
	const TArray<FVector3f>& VoxelTestPositions,
	const FGuid& IntersectingLevelGuid,
	TArray<FIrradianceBrickBuildData>& OutBrickBuildData)
{
	if (StartCellIndex == 0)
	{
		FIrradianceBrickBuildData NewBuildData;
		NewBuildData.IntersectingLevelGuid = IntersectingLevelGuid;
		NewBuildData.LocalCellCoordinate = LocalCellCoordinate;
		NewBuildData.TreeDepth = TreeDepth;
		NewBuildData.bDebugBrick = bCoveringDebugPosition;
		OutBrickBuildData.Add(NewBuildData);
	}

	const int32 BuildDataIndex = OutBrickBuildData.Num() - 1;

	const int32 BrickSizeLog2 = FMath::FloorLog2(VolumetricLightmapSettings.BrickSize);
	const int32 DetailCellsPerTopLevelBrick = 1 << (VolumetricLightmapSettings.MaxRefinementLevels * BrickSizeLog2);
	const int32 DetailCellsPerCurrentLevelBrick = 1 << ((VolumetricLightmapSettings.MaxRefinementLevels - TreeDepth) * BrickSizeLog2);
	const float InvBrickSize = 1.0f / VolumetricLightmapSettings.BrickSize;
	const int32 NumCellsPerBrick = VolumetricLightmapSettings.BrickSize * VolumetricLightmapSettings.BrickSize * VolumetricLightmapSettings.BrickSize;

	// Assume children are present if we are only processing a portion of the brick
	bool bHasChildren = StartCellIndex > 0;

	if (TreeDepth + 1 < VolumetricLightmapSettings.MaxRefinementLevels)
	{
		const int32 DetailCellsPerChildLevelBrick = DetailCellsPerCurrentLevelBrick / VolumetricLightmapSettings.BrickSize;
		const FVector3f BrickNormalizedMin = (FVector3f)LocalCellCoordinate / (float)DetailCellsPerTopLevelBrick;
		const FVector3f WorldBrickMin = TopLevelCellBounds.Min + BrickNormalizedMin * TopLevelCellBounds.GetSize();
		const FVector3f WorldChildCellSize = InvBrickSize * TopLevelCellBounds.GetSize() * DetailCellsPerCurrentLevelBrick / (float)DetailCellsPerTopLevelBrick;

		for (int32 Z = 0; Z < VolumetricLightmapSettings.BrickSize; Z++)
		{
			for (int32 Y = 0; Y < VolumetricLightmapSettings.BrickSize; Y++)
			{
				for (int32 X = 0; X < VolumetricLightmapSettings.BrickSize; X++)
				{
					const int32 CellIndex = (Z * VolumetricLightmapSettings.BrickSize + Y) * VolumetricLightmapSettings.BrickSize + X;

					if (CellIndex >= StartCellIndex && CellIndex < StartCellIndex + NumCells)
					{
						const FVector3f ChildCellPosition = WorldBrickMin + FVector3f(X, Y, Z) * WorldChildCellSize;
						const FBox3f CellBounds(ChildCellPosition, ChildCellPosition + WorldChildCellSize);

						const bool bChildCoveringDebugPosition = bDebugVolumetricLightmapCell && CellBounds.IsInside(DebugWorldPosition);
						FGuid ChildIntersectingLevelGuid;
						const bool bSubdivideCell = ShouldRefineVoxel(TreeDepth + 1, CellBounds, VoxelTestPositions, bChildCoveringDebugPosition, ChildIntersectingLevelGuid);

						if (bSubdivideCell)
						{
							bHasChildren = true;

							const FIntVector LocalChildCellCoordinate(
								X * DetailCellsPerChildLevelBrick, 
								Y * DetailCellsPerChildLevelBrick, 
								Z * DetailCellsPerChildLevelBrick);

							RecursivelyBuildBrickTree(0, NumCellsPerBrick, LocalCellCoordinate + LocalChildCellCoordinate, TreeDepth + 1, bChildCoveringDebugPosition, TopLevelCellBounds, VoxelTestPositions, ChildIntersectingLevelGuid, OutBrickBuildData);
						}
					}
				}
			}
		}
	}

	if (StartCellIndex == 0)
	{
		OutBrickBuildData[BuildDataIndex].bHasChildren = bHasChildren;
	}
}

class FVolumetricLightmapBrickTaskDescription
{
public:
	// Inputs
	FIntVector TaskIndexVector;
	const FIrradianceBrickBuildData& BuildData;
	bool bDebugThisMapping;
	FStaticLightingMappingContext MappingContext;

	// Outputs
	bool bDiscardBrick;
	bool bProcessedOnMainThread;
	volatile int32* NumOutstandingBrickTasks;
	FIrradianceBrickData& BrickData;

	FVolumetricLightmapBrickTaskDescription(
		FIntVector InTaskIndexVector, 
		const FIrradianceBrickBuildData& InBuildData, 
		bool bInDebugThisMapping,
		volatile int32* InNumOutstandingBrickTasks,
		FIrradianceBrickData& InBrickData, 
		FDebugLightingOutput& InDebugOutput,
		class FStaticLightingSystem& InSystem) :
		TaskIndexVector(InTaskIndexVector),
		BuildData(InBuildData),
		bDebugThisMapping(bInDebugThisMapping),
		MappingContext(nullptr, InSystem, &InDebugOutput),
		bDiscardBrick(false),
		bProcessedOnMainThread(false),
		NumOutstandingBrickTasks(InNumOutstandingBrickTasks),
		BrickData(InBrickData)
	{}
};

void FStaticLightingSystem::ProcessVolumetricLightmapBrickTask(FVolumetricLightmapBrickTaskDescription* Task)
{
	const bool bGenerateSkyShadowing = HasSkyShadowing();

	const FIrradianceBrickBuildData& BuildData = Task->BuildData;
	FIrradianceBrickData& BrickData = Task->BrickData;

	const int32 BrickSize = VolumetricLightmapSettings.BrickSize;
	const int32 BrickSizeLog2 = FMath::FloorLog2(BrickSize);
	const int32 DetailCellsPerTopLevelBrick = 1 << (VolumetricLightmapSettings.MaxRefinementLevels * BrickSizeLog2);
	const int32 IndirectionCellsPerTopLevelCell = DetailCellsPerTopLevelBrick / BrickSize;

	const float InvBrickSize = 1.0f / BrickSize;
	const int32 TotalBrickSize = BrickSize * BrickSize * BrickSize;
	const FIntVector IndirectionTextureDimensions = VolumetricLightmapSettings.TopLevelGridSize * IndirectionCellsPerTopLevelCell;

	BrickData.IndirectionTexturePosition = Task->TaskIndexVector * IndirectionCellsPerTopLevelCell + BuildData.LocalCellCoordinate / BrickSize;
	BrickData.TreeDepth = BuildData.TreeDepth;
	BrickData.AmbientVector.Empty(TotalBrickSize);
	BrickData.AmbientVector.AddDefaulted(TotalBrickSize);

	BrickData.VoxelImportProcessingData.Empty(TotalBrickSize);
	BrickData.VoxelImportProcessingData.AddDefaulted(TotalBrickSize);

	if (bGenerateSkyShadowing)
	{
		BrickData.SkyBentNormal.Empty(TotalBrickSize);
		BrickData.SkyBentNormal.AddDefaulted(TotalBrickSize);
	}

	BrickData.DirectionalLightShadowing.Empty(TotalBrickSize);
	BrickData.DirectionalLightShadowing.AddDefaulted(TotalBrickSize);

	for (int32 i = 0; i < UE_ARRAY_COUNT(BrickData.SHCoefficients); i++)
	{
		BrickData.SHCoefficients[i].Empty(TotalBrickSize);
		BrickData.SHCoefficients[i].AddDefaulted(TotalBrickSize);
	}

	const FVector3f TopLevelBrickSize = VolumetricLightmapSettings.VolumeSize / FVector3f(VolumetricLightmapSettings.TopLevelGridSize);
	const FVector3f TopLevelBrickMin = VolumetricLightmapSettings.VolumeMin + FVector3f(Task->TaskIndexVector) * TopLevelBrickSize;

	const FVector3f BrickNormalizedMin = (FVector3f)BuildData.LocalCellCoordinate / (float)DetailCellsPerTopLevelBrick;
	const FVector3f WorldBrickMin = TopLevelBrickMin + BrickNormalizedMin * TopLevelBrickSize;
	const int32 DetailCellsPerCurrentLevelBrick = 1 << ((VolumetricLightmapSettings.MaxRefinementLevels - BuildData.TreeDepth) * BrickSizeLog2);
	const FVector3f WorldChildCellSize = InvBrickSize * TopLevelBrickSize * DetailCellsPerCurrentLevelBrick / (float)DetailCellsPerTopLevelBrick;
	const int32 NumBottomLevelBricks = DetailCellsPerCurrentLevelBrick / BrickSize;
	const float BoundarySize = NumBottomLevelBricks * InvBrickSize;

	FLMRandomStream RandomStream(0);
	float AverageClosestGeometryDistance = 0;
	bool bAllCellsInsideGeometry = true;

#if LIGHTMASS_DO_PROCESSING

	for (int32 Z = 0; Z < BrickSize; Z++)
	{
		for (int32 Y = 0; Y < BrickSize; Y++)
		{
			for (int32 X = 0; X < BrickSize; X++)
			{
				const FVector3f VoxelPosition = WorldBrickMin + FVector3f(X, Y, Z) * WorldChildCellSize;
				
				// Use a radius to avoid shadowing from geometry contained in the cell
				FVolumeLightingSample CurrentSample(FVector4f(VoxelPosition, WorldChildCellSize.GetMax() / 2.0f));

				const FVector3f IndirectionCellPosition = FVector3f(BrickData.IndirectionTexturePosition) + FVector3f(X, Y, Z) * InvBrickSize * NumBottomLevelBricks;
				bool bBorderVoxel = false;

				if (IndirectionCellPosition.X < BoundarySize
					|| IndirectionCellPosition.Y < BoundarySize
					|| IndirectionCellPosition.Z < BoundarySize
					|| IndirectionCellPosition.X > IndirectionTextureDimensions.X - BoundarySize * 1.1f
					|| IndirectionCellPosition.Y > IndirectionTextureDimensions.Y - BoundarySize * 1.1f
					|| IndirectionCellPosition.Z > IndirectionTextureDimensions.Z - BoundarySize * 1.1f)
				{
					bBorderVoxel = true;
					CurrentSample.PositionAndRadius.W = VolumetricLightmapSettings.VolumeSize.GetMax() / 2.0f;
				}

				const bool bDebugSamples = bDebugVolumetricLightmapCell 
					&& BuildData.bDebugBrick
					&& BuildData.TreeDepth == VolumetricLightmapSettings.MaxRefinementLevels - 1
					&& DebugWorldPosition.X >= VoxelPosition.X && DebugWorldPosition.Y >= VoxelPosition.Y && DebugWorldPosition.Z >= VoxelPosition.Z
					&& DebugWorldPosition.X < VoxelPosition.X + WorldChildCellSize.X && DebugWorldPosition.Y < VoxelPosition.Y + WorldChildCellSize.Y && DebugWorldPosition.Z < VoxelPosition.Z + WorldChildCellSize.Z;

				if (bDebugSamples)
				{
					Task->MappingContext.DebugOutput->bValid = true;
				}

				float BackfacingHitsFraction = 0.0f;
				float MinDistanceToSurface = HALF_WORLD_MAX;

				CalculateVolumeSampleIncidentRadiance(
					CachedVolumetricLightmapUniformHemisphereSamples, 
					CachedVolumetricLightmapUniformHemisphereSampleUniforms, 
					CachedVolumetricLightmapMaxUnoccludedLength, 
					CachedVolumetricLightmapVertexOffsets,
					CurrentSample, 
					BackfacingHitsFraction, 
					MinDistanceToSurface, 
					RandomStream, 
					Task->MappingContext, 
					bDebugSamples);

				// Windowing
				if (VolumetricLightmapSettings.WindowingTargetLaplacian > 0.0f)
				{
					FSHVectorRGB3 SHSample;
					CurrentSample.ToSHVector(SHSample);

					FSHVector3 Luminance = SHSample.GetLuminance();
					float WindowingLambda = FSHVector3::FindWindowingLambda(Luminance, VolumetricLightmapSettings.WindowingTargetLaplacian);

					if (WindowingLambda != 0.0f)
					{
						SHSample.ApplyWindowing(WindowingLambda);
					}

					CurrentSample.SetFromSHVector(SHSample);
				}

				const bool bInsideGeometry = BackfacingHitsFraction > .3f;
				bool bDebugInteriorVoxels = false;

				if (bDebugInteriorVoxels && bInsideGeometry)
				{
					CurrentSample.HighQualityCoefficients[0][0] = 10.0f;
				}

				const int32 VoxelIndex = (Z * BrickSize + Y) * BrickSize + X;

				BrickData.SetFromVolumeLightingSample(VoxelIndex, CurrentSample, bInsideGeometry, MinDistanceToSurface, bBorderVoxel);
				Task->MappingContext.Stats.NumVolumetricLightmapSamples++;
				AverageClosestGeometryDistance += MinDistanceToSurface;
				bAllCellsInsideGeometry = bAllCellsInsideGeometry && bInsideGeometry;
			}
		}
	}

#endif

	BrickData.AverageClosestGeometryDistance = AverageClosestGeometryDistance / TotalBrickSize;
	
	const bool bCullBrick = bAllCellsInsideGeometry;

	if (bCullBrick && BuildData.TreeDepth > 0 && !BuildData.bHasChildren)
	{
		Task->bDiscardBrick = true;
	}
}

bool FStaticLightingSystem::ProcessVolumetricLightmapTaskIfAvailable()
{
	bool bAnyTaskProcessedByThisThread = false;

	while (FVolumetricLightmapBrickTaskDescription * NextTask = VolumetricLightmapBrickTasks.Pop())
	{
		//UE_LOGF(LogLightmass, Warning, "Thread picked up volumetric lightmap task");
		ProcessVolumetricLightmapBrickTask(NextTask);
		FPlatformAtomics::InterlockedDecrement(NextTask->NumOutstandingBrickTasks);

		bAnyTaskProcessedByThisThread = true;
	}

	return bAnyTaskProcessedByThisThread;
}

void FStaticLightingSystem::GenerateVoxelTestPositions(TArray<FVector3f>& VoxelTestPositions) const
{
	const int32 BrickSize = VolumetricLightmapSettings.BrickSize;
	const float InvBrickSize = 1.0f / BrickSize;
	const int32 NumSamplesPerCell = 4;

	FLMRandomStream RandomStream(34785);
	VoxelTestPositions.Empty(BrickSize * BrickSize * BrickSize * NumSamplesPerCell);
	VoxelTestPositions.AddDefaulted(BrickSize * BrickSize * BrickSize * NumSamplesPerCell);

	for (int32 Z = 0; Z < BrickSize; Z++)
	{
		for (int32 Y = 0; Y < BrickSize; Y++)
		{
			for (int32 X = 0; X < BrickSize; X++)
			{
				const FVector3f BrickMin = FVector3f(X, Y, Z) * InvBrickSize;
				const int32 CellIndex = (Z * BrickSize + Y) * BrickSize + X;

				for (int32 SampleIndex = 0; SampleIndex < NumSamplesPerCell; SampleIndex++)
				{
					FVector3f RandomOffset = FVector3f(RandomStream.GetFraction(), RandomStream.GetFraction(), RandomStream.GetFraction()) * InvBrickSize;
					VoxelTestPositions[CellIndex * NumSamplesPerCell + SampleIndex] = (BrickMin + RandomOffset);
				}
			}
		}
	}
}

void FStaticLightingSystem::CalculateAdaptiveVolumetricLightmap(int32 TaskIndex)
{
#if !ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	checkf(!bDebugVolumetricLightmapCell, TEXT("enable ALLOW_LIGHTMAP_SAMPLE_DEBUGGING for voxel debugging"));
#endif

	const double StartTime = FPlatformTime::Seconds();

	FPlatformAtomics::InterlockedIncrement(&TasksInProgressThatWillNeedHelp);

	check(TaskIndex >= 0 && TaskIndex < Scene.VolumetricLightmapTaskGuids.Num());
	const int32 NumTopLevelBricks = VolumetricLightmapSettings.TopLevelGridSize.X * VolumetricLightmapSettings.TopLevelGridSize.Y * VolumetricLightmapSettings.TopLevelGridSize.Z;
	const int32 TasksPerTopLevelBrick = Scene.VolumetricLightmapTaskGuids.Num() / NumTopLevelBricks;
	const int32 TopLevelBrickIndex = TaskIndex / TasksPerTopLevelBrick;
	check(TopLevelBrickIndex < NumTopLevelBricks);
	const int32 SubTaskIndex = TaskIndex - TopLevelBrickIndex * TasksPerTopLevelBrick;
	check(SubTaskIndex < TasksPerTopLevelBrick);

	// Create a new link for the output of this task
	TList<FVolumetricLightmapTaskData>* DataLink = new TList<FVolumetricLightmapTaskData>(FVolumetricLightmapTaskData(), NULL);
	DataLink->Element.Guid = Scene.VolumetricLightmapTaskGuids[TaskIndex];

	FStaticLightingMappingContext MappingContext(nullptr, *this);

	const FIntVector TaskIndexVector(
		TopLevelBrickIndex % VolumetricLightmapSettings.TopLevelGridSize.X,
		(TopLevelBrickIndex / VolumetricLightmapSettings.TopLevelGridSize.X) % VolumetricLightmapSettings.TopLevelGridSize.Y,
		TopLevelBrickIndex / (VolumetricLightmapSettings.TopLevelGridSize.X * VolumetricLightmapSettings.TopLevelGridSize.Y));

	const FVector3f TopLevelBrickSize = VolumetricLightmapSettings.VolumeSize / FVector3f(VolumetricLightmapSettings.TopLevelGridSize);
	const FVector3f TopLevelBrickMin = VolumetricLightmapSettings.VolumeMin + FVector3f(TaskIndexVector) * TopLevelBrickSize;

	const int32 BrickSize = VolumetricLightmapSettings.BrickSize;

	const FBox3f TopLevelBounds(TopLevelBrickMin, TopLevelBrickMin + TopLevelBrickSize);
	const bool bCoveringDebugPosition = bDebugVolumetricLightmapCell && TopLevelBounds.IsInside(DebugWorldPosition);

	const int32 NumCellsPerBrick = BrickSize * BrickSize * BrickSize;
	const int32 NumCellsPerTask = NumCellsPerBrick / TasksPerTopLevelBrick;
	const int32 StartCellIndex = SubTaskIndex * NumCellsPerTask;
	int32 NumCells = NumCellsPerTask;

	if (SubTaskIndex == TasksPerTopLevelBrick - 1)
	{
		// Last task should take all remaining cells
		NumCells = NumCellsPerBrick - StartCellIndex;
	}

	check(NumCells > 0);

	TArray<FVector3f> VoxelTestPositions;
	GenerateVoxelTestPositions(VoxelTestPositions);

	TArray<FIrradianceBrickBuildData> BrickBuildData;
	RecursivelyBuildBrickTree(StartCellIndex, NumCells, FIntVector::ZeroValue, 0, bCoveringDebugPosition, TopLevelBounds, VoxelTestPositions, FGuid(), BrickBuildData);

	MappingContext.Stats.VolumetricLightmapVoxelizationTime += FPlatformTime::Seconds() - StartTime;

	if (BrickBuildData.Num() > 0)
	{
		volatile int32 NumOutstandingBrickTasks = 0;

		TArray<FVolumetricLightmapBrickTaskDescription*> BrickTasks;
		BrickTasks.Empty(BrickBuildData.Num());

		DataLink->Element.BrickData.Empty(BrickBuildData.Num());
		DataLink->Element.BrickData.AddDefaulted(BrickBuildData.Num());

		for (int32 BrickIndex = 0; BrickIndex < BrickBuildData.Num(); BrickIndex++)
		{
			DataLink->Element.BrickData[BrickIndex].IntersectingLevelGuid = BrickBuildData[BrickIndex].IntersectingLevelGuid;
		}

		// Calculate lighting for all bricks
		for (int32 BrickIndex = 0; BrickIndex < BrickBuildData.Num(); BrickIndex++)
		{
			FVolumetricLightmapBrickTaskDescription* NewTask = new FVolumetricLightmapBrickTaskDescription(
				TaskIndexVector, 
				BrickBuildData[BrickIndex], 
				bCoveringDebugPosition, 
				&NumOutstandingBrickTasks, 
				DataLink->Element.BrickData[BrickIndex], 
				DataLink->Element.DebugOutput,
				*this);
			
			BrickTasks.Add(NewTask);

			// Add to the queue so other lighting threads can pick up these tasks
			FPlatformAtomics::InterlockedIncrement(&NumOutstandingBrickTasks);
			VolumetricLightmapBrickTasks.Push(NewTask);
		}

		do 
		{
			// Process tasks from any threads until this mapping's tasks are complete
			FVolumetricLightmapBrickTaskDescription* NextTask = VolumetricLightmapBrickTasks.Pop();

			if (NextTask)
			{
				NextTask->bProcessedOnMainThread = true;
				ProcessVolumetricLightmapBrickTask(NextTask);
				FPlatformAtomics::InterlockedDecrement(NextTask->NumOutstandingBrickTasks);
			}
		} 
		while (NumOutstandingBrickTasks > 0);

		int32 BuildBrickIndex = 0;

		for (int32 BrickIndex = 0; BrickIndex < DataLink->Element.BrickData.Num(); BrickIndex++, BuildBrickIndex++)
		{
			if (BrickTasks[BuildBrickIndex]->bDiscardBrick)
			{
				DataLink->Element.BrickData.RemoveAt(BrickIndex);
				BrickIndex--;
			}

			delete BrickTasks[BuildBrickIndex];
		}
	}
	
	FPlatformAtomics::InterlockedDecrement(&TasksInProgressThatWillNeedHelp);
	CompleteVolumetricLightmapTaskList.AddElement(DataLink);

	MappingContext.Stats.TotalVolumetricLightmapLightingThreadTime += FPlatformTime::Seconds() - StartTime;
}

void FIrradianceBrickData::SetFromVolumeLightingSample(int32 Index, const FVolumeLightingSample& Sample, bool bInsideGeometry, float MinDistanceToSurface, bool bBorderVoxel)
{
	static_assert(UE_ARRAY_COUNT(Sample.HighQualityCoefficients) >= UE_ARRAY_COUNT(SHCoefficients) + 1, "Coefficient mismatch");

	AmbientVector[Index] = FFloat3Packed(FLinearColor(Sample.HighQualityCoefficients[0][0], Sample.HighQualityCoefficients[0][1], Sample.HighQualityCoefficients[0][2], 0.0f));

	/*
		SH directional coefficients can be normalized by their ambient term, and then ranges can be derived from SH projection
		This allows packing into an 8 bit format
		[-1, 1] Normalization factors derived from SHBasisFunction

		Result.V0.x = 0.282095f; 
		Result.V0.y = -0.488603f * InputVector.y;
		Result.V0.z = 0.488603f * InputVector.z;
		Result.V0.w = -0.488603f * InputVector.x;

		half3 VectorSquared = InputVector * InputVector;
		Result.V1.x = 1.092548f * InputVector.x * InputVector.y;
		Result.V1.y = -1.092548f * InputVector.y * InputVector.z;
		Result.V1.z = 0.315392f * (3.0f * VectorSquared.z - 1.0f);
		Result.V1.w = -1.092548f * InputVector.x * InputVector.z;
		Result.V2 = 0.546274f * (VectorSquared.x - VectorSquared.y);
	*/

	// Note: encoding behavior has to match CPU decoding in InterpolateVolumetricLightmap and GPU decoding in GetVolumetricLightmapSH3

	FLinearColor CoefficientNormalizationScale0(
		0.282095f / 0.488603f,
		0.282095f / 0.488603f,
		0.282095f / 0.488603f,
		0.282095f / 1.092548f);

	FLinearColor CoefficientNormalizationScale1(
		0.282095f / 1.092548f,
		0.282095f / (4.0f * 0.315392f),
		0.282095f / 1.092548f,
		0.282095f / (2.0f * 0.546274f));

	for (int32 ChannelIndex = 0; ChannelIndex < 3; ChannelIndex++)
	{
		const float InvAmbient = 1.0f / FMath::Max(Sample.HighQualityCoefficients[0][ChannelIndex], .0001f);

		const FLinearColor Vector0Normalized =
			FLinearColor(Sample.HighQualityCoefficients[1][ChannelIndex], Sample.HighQualityCoefficients[2][ChannelIndex], Sample.HighQualityCoefficients[3][ChannelIndex], Sample.HighQualityCoefficients[4][ChannelIndex])
			* CoefficientNormalizationScale0
			* FLinearColor(InvAmbient, InvAmbient, InvAmbient, InvAmbient);

		SHCoefficients[ChannelIndex * 2 + 0][Index] = (Vector0Normalized * FLinearColor(.5f, .5f, .5f, .5f) + FLinearColor(.5f, .5f, .5f, .5f)).QuantizeRound();

		const FLinearColor Vector1Normalized =
			FLinearColor(Sample.HighQualityCoefficients[5][ChannelIndex], Sample.HighQualityCoefficients[6][ChannelIndex], Sample.HighQualityCoefficients[7][ChannelIndex], Sample.HighQualityCoefficients[8][ChannelIndex])
			* CoefficientNormalizationScale1
			* FLinearColor(InvAmbient, InvAmbient, InvAmbient, InvAmbient);

		SHCoefficients[ChannelIndex * 2 + 1][Index] = (Vector1Normalized * FLinearColor(.5f, .5f, .5f, .5f) + FLinearColor(.5f, .5f, .5f, .5f)).QuantizeRound();
	}

	if (SkyBentNormal.Num() > 0)
	{
		SkyBentNormal[Index] = (FLinearColor(Sample.SkyBentNormal) * FLinearColor(.5f, .5f, .5f, .5f) + FLinearColor(.5f, .5f, .5f, .5f)).QuantizeRound();
	}

	DirectionalLightShadowing[Index] = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(Sample.DirectionalLightShadowing * MAX_uint8), 0, MAX_uint8);

	FIrradianceVoxelImportProcessingData NewImportData;
	NewImportData.bInsideGeometry = bInsideGeometry;
	NewImportData.bBorderVoxel = bBorderVoxel;
	NewImportData.ClosestGeometryDistance = MinDistanceToSurface;
	VoxelImportProcessingData[Index] = NewImportData;
}
}
