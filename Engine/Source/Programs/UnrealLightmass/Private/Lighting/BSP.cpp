// Copyright Epic Games, Inc. All Rights Reserved.

#include "BSP.h"
#include "Importer.h"

namespace Lightmass
{
	void FBSPSurfaceStaticLighting::GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const
	{
 		OutV0 = Vertices[TriangleVertexIndices[TriangleIndex * 3 + 0]];
 		OutV1 = Vertices[TriangleVertexIndices[TriangleIndex * 3 + 1]];
 		OutV2 = Vertices[TriangleVertexIndices[TriangleIndex * 3 + 2]];
		ElementIndex = TriangleLightmassSettings[TriangleIndex];
	}

	void FBSPSurfaceStaticLighting::GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const
	{
 		OutI0 = TriangleVertexIndices[TriangleIndex * 3 + 0];
 		OutI1 = TriangleVertexIndices[TriangleIndex * 3 + 1];
 		OutI2 = TriangleVertexIndices[TriangleIndex * 3 + 2];
	}

	void FBSPSurfaceStaticLighting::Import( FLightmassImporter& Importer )
	{
		FStaticLightingMesh::Import( Importer );
		Mapping.Import( Importer );		
		// bsp mapping/mesh are the same
		Mapping.Mesh = this;

		Importer.ImportData( (FBSPSurfaceStaticLightingData*) this );
		Importer.ImportArray( Vertices, NumVertices );
		Importer.ImportArray( TriangleVertexIndices, NumTriangles * 3 );
		Importer.ImportArray( TriangleLightmassSettings, NumTriangles );

		// Ignore invalid BSP lightmap UV's since they are generated and not something that artists have direct control over.
		//@todo - fix the root cause that results in some texels having their centers mapped to multiple texels.
		bColorInvalidTexels = false;
	}

} //namespace Lightmass
