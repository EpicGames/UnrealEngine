// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LightingMesh.h"
#include "Mappings.h"

namespace Lightmass
{

	/** Represents a BSP surface to the static lighting system. */
	class FBSPSurfaceStaticLighting : public FStaticLightingMesh, public FBSPSurfaceStaticLightingData
	{
	public:
	
		FBSPSurfaceStaticLighting()
		: bComplete(false)
		{}

		// FStaticLightingMesh interface.
		virtual void GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const;
		virtual void GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const;

		virtual void Import( class FLightmassImporter& Importer );

		/** true if the surface has complete static lighting. */
		bool bComplete;

		/** Texture mapping for the BSP */
		FStaticLightingTextureMapping Mapping;

		/** The surface's vertices. */
		TArray<FStaticLightingVertex> Vertices;

		/** The vertex indices of the surface's triangles. */
		TArray<int32> TriangleVertexIndices;

		/** Array for each triangle to the lightmass settings (boost, etc) */
		TArray<int32> TriangleLightmassSettings;
	};


} //namespace Lightmass
