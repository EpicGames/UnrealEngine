// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Lightmass
{

enum ELightingSolverType
{
	/** Default direct lighting solver with no global/bounced lighting */
	LightSolver_Direct=0,
	/** GI solver type */
	LightSolver_Bouncy
};

/**
 * A set of parameters specifying how static lighting is rebuilt.
 */
class FLightingBuildOptions
{
public:

	FLightingBuildOptions()
	:	bPerformFullQualityBuild(true)
	,	LightSolverType(LightSolver_Direct)
	{}

	/** Whether to use fast or pretty lighting build.								*/
	bool					bPerformFullQualityBuild;
	/** Current light solver type desired */
	ELightingSolverType		LightSolverType;
};

} //namespace Lightmass
