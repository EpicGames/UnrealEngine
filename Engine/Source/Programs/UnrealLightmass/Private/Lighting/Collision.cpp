// Copyright Epic Games, Inc. All Rights Reserved.

#include "Collision.h"
#include "LightingSystem.h"
#include "UnrealLightmass.h"

namespace Lightmass
{

FStaticLightingAggregateMesh::FStaticLightingAggregateMesh(const FScene& InScene):
	Scene(InScene),
	bHasShadowCastingPrimitives(false),
	SceneBounds(ForceInit),
	SceneSurfaceArea(0),
	SceneSurfaceAreaWithinImportanceVolume(0)
{
}

FBox3f FStaticLightingAggregateMesh::GetBounds() const
{
	// Expand the bounds slightly to avoid having to handle geometry that is exactly on the bounding box,
	// Which happens if you create a new level in Unreal with BSP from the default builder brush.
	return bHasShadowCastingPrimitives ? SceneBounds.ExpandBy(5.0f * Scene.SceneConstants.StaticLightingLevelScale) : FBox3f(FVector4f(0,0,0), FVector4f(0,0,0));
}

FDefaultAggregateMesh::~FDefaultAggregateMesh()
{
	for (int32 i = 0; i < MeshInfos.Num(); i++)
	{
		delete MeshInfos[i];
	}
}

/**
* Merges a mesh into the shadow mesh.
* @param Mesh - The mesh the triangle comes from.
*/
void FDefaultAggregateMesh::AddMesh(const FStaticLightingMesh* Mesh, const FStaticLightingMapping* Mapping)
{
	// Only use shadow casting meshes.
	if( Mesh->LightingFlags&GI_INSTANCE_CASTSHADOW )
	{
		SceneBounds = SceneBounds + Mesh->BoundingBox;

		const FStaticLightingTextureMapping* TextureMapping = Mapping ? Mapping->GetTextureMapping() : NULL;
		const int32 BaseVertexIndex = Vertices.Num();
		MeshInfos.Add(new FStaticLightingMeshInfo(BaseVertexIndex, Mesh));
		Vertices.AddUninitialized(Mesh->NumVertices);
		UVs.AddZeroed(Mesh->NumVertices);
		LightmapUVs.AddZeroed(Mesh->NumVertices);

		const uint32 MeshLODIndices = Mesh->GetLODIndices();
		const uint32 MeshHLODRange = Mesh->GetHLODRange();

		const FBoxSphereBounds3f ImportanceBounds = Scene.GetImportanceBounds();
		for(int32 TriangleIndex = 0;TriangleIndex < Mesh->NumTriangles;TriangleIndex++)
		{
			// Read the triangle from the mesh.
			FStaticLightingVertex V0;
			FStaticLightingVertex V1;
			FStaticLightingVertex V2;
			int32 ElementIndex;
			Mesh->GetTriangle(TriangleIndex,V0,V1,V2,ElementIndex);

			int32 I0 = 0;
			int32 I1 = 0;
			int32 I2 = 0;
			Mesh->GetTriangleIndices(TriangleIndex,I0,I1,I2);

			check(I0 <= Mesh->NumVertices && I1 <= Mesh->NumVertices && I2 <= Mesh->NumVertices);

			const bool bTwoSided = Mesh->IsTwoSided(ElementIndex) || Mesh->IsCastingShadowAsTwoSided();
			const bool bStaticAndOpaque = !Mesh->IsMasked(ElementIndex) && !Mesh->IsTranslucent(ElementIndex) && !Mesh->bMovable;

			Vertices[BaseVertexIndex + I0] = V0.WorldPosition;
			Vertices[BaseVertexIndex + I1] = V1.WorldPosition;
			Vertices[BaseVertexIndex + I2] = V2.WorldPosition;
			UVs[BaseVertexIndex + I0] = V0.TextureCoordinates[Mesh->TextureCoordinateIndex];
			UVs[BaseVertexIndex + I1] = V1.TextureCoordinates[Mesh->TextureCoordinateIndex];
			UVs[BaseVertexIndex + I2] = V2.TextureCoordinates[Mesh->TextureCoordinateIndex];
			if (TextureMapping)
			{
				LightmapUVs[BaseVertexIndex + I0] = V0.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
				LightmapUVs[BaseVertexIndex + I1] = V1.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
				LightmapUVs[BaseVertexIndex + I2] = V2.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
			}

			// Compute the triangle's normal.
			const FVector4f TriangleNormal = (V2.WorldPosition - V0.WorldPosition) ^ (V1.WorldPosition - V0.WorldPosition);

			// Compute the triangle area.
			const float TriangleArea = TriangleNormal.Size3() * 0.5f;

			// Ignore zero area triangles.
			if( TriangleArea > TRIANGLE_AREA_THRESHOLD && Mesh->IsElementCastingShadow(ElementIndex))
			{
				const int32 PayloadIndex = TrianglePayloads.Add(
					FTriangleSOAPayload(MeshInfos.Last(), Mapping, ElementIndex, BaseVertexIndex + I0, BaseVertexIndex + I1, BaseVertexIndex + I2));

				bHasShadowCastingPrimitives = true;

				new(kDOPTriangles) FkDOPBuildCollisionTriangle<uint32>(
					PayloadIndex, // Use the triangle's material index as an index into TrianglePayloads.
					V0.WorldPosition,V1.WorldPosition,V2.WorldPosition,
					Mesh->MeshIndex,
					MeshLODIndices,
					MeshHLODRange,
					bTwoSided,
					bStaticAndOpaque
					);
			}

			// Sum the total triangle area of everything in the aggregate mesh
			SceneSurfaceArea += TriangleArea;

			// Sum the total triangle area of everything in the aggregate mesh within the importance volume,
			// if any vertex is contained or if there is no importance volume at all
			if( ImportanceBounds.SphereRadius < DELTA ||
				ImportanceBounds.GetBox().IsInside( V0.WorldPosition ) ||
				ImportanceBounds.GetBox().IsInside( V1.WorldPosition ) ||
				ImportanceBounds.GetBox().IsInside( V2.WorldPosition ) )
			{
				SceneSurfaceAreaWithinImportanceVolume += TriangleArea;
			}
		}
	}
}

void FDefaultAggregateMesh::AddMeshForVoxelization(const FStaticLightingMesh* Mesh, const FStaticLightingMapping* Mapping, bool bUseForInstancing)
{
	if (bUseForInstancing)
	{
		check(Mesh->GetInstanceableStaticMesh());
	}

	if (Mesh->LightingFlags & GI_INSTANCE_CASTSHADOW && Mesh->DoesMeshBelongToLOD0())
	{
		SceneBounds = SceneBounds + Mesh->BoundingBox;

		const FStaticLightingTextureMapping* TextureMapping = Mapping ? Mapping->GetTextureMapping() : NULL;
		const int32 BaseVertexIndex = Vertices.Num();
		MeshInfos.Add(new FStaticLightingMeshInfo(BaseVertexIndex, Mesh));
		Vertices.AddUninitialized(Mesh->NumVertices);
		UVs.AddZeroed(Mesh->NumVertices);
		LightmapUVs.AddZeroed(Mesh->NumVertices);

		const uint32 MeshLODIndices = Mesh->GetLODIndices();
		const uint32 MeshHLODRange = Mesh->GetHLODRange();

		for (int32 TriangleIndex = 0; TriangleIndex < Mesh->NumTriangles; TriangleIndex++)
		{
			// Read the triangle from the mesh.
			int32 I0 = 0;
			int32 I1 = 0;
			int32 I2 = 0;
			FStaticLightingVertex V0;
			FStaticLightingVertex V1;
			FStaticLightingVertex V2;
			int32 ElementIndex;
			if (bUseForInstancing)
			{
				Mesh->GetInstanceableStaticMesh()->GetNonTransformedTriangleIndices(TriangleIndex, I0, I1, I2);
				Mesh->GetInstanceableStaticMesh()->GetNonTransformedTriangle(TriangleIndex, V0, V1, V2, ElementIndex);
			}
			else
			{
				Mesh->GetTriangleIndices(TriangleIndex, I0, I1, I2);
				Mesh->GetTriangle(TriangleIndex, V0, V1, V2, ElementIndex);
			}

			if (Mesh->IsElementCastingShadow(ElementIndex) && !Mesh->IsTranslucent(ElementIndex))
			{
				check(I0 <= Mesh->NumVertices && I1 <= Mesh->NumVertices && I2 <= Mesh->NumVertices);

				const bool bTwoSided = Mesh->IsTwoSided(ElementIndex) || Mesh->IsCastingShadowAsTwoSided();
				const bool bStaticAndOpaque = !Mesh->IsMasked(ElementIndex) && !Mesh->IsTranslucent(ElementIndex) && !Mesh->bMovable;

				Vertices[BaseVertexIndex + I0] = V0.WorldPosition;
				Vertices[BaseVertexIndex + I1] = V1.WorldPosition;
				Vertices[BaseVertexIndex + I2] = V2.WorldPosition;
				UVs[BaseVertexIndex + I0] = V0.TextureCoordinates[Mesh->TextureCoordinateIndex];
				UVs[BaseVertexIndex + I1] = V1.TextureCoordinates[Mesh->TextureCoordinateIndex];
				UVs[BaseVertexIndex + I2] = V2.TextureCoordinates[Mesh->TextureCoordinateIndex];
				if (TextureMapping)
				{
					LightmapUVs[BaseVertexIndex + I0] = V0.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
					LightmapUVs[BaseVertexIndex + I1] = V1.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
					LightmapUVs[BaseVertexIndex + I2] = V2.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
				}

				// Compute the triangle's normal.
				const FVector4f TriangleNormal = (V2.WorldPosition - V0.WorldPosition) ^ (V1.WorldPosition - V0.WorldPosition);

				// Compute the triangle area.
				const float TriangleArea = TriangleNormal.Size3() * 0.5f;

				// Ignore zero area triangles.
				if (TriangleArea > TRIANGLE_AREA_THRESHOLD)
				{
					const int32 PayloadIndex = TrianglePayloads.Add(
						FTriangleSOAPayload(MeshInfos.Last(), Mapping, ElementIndex, BaseVertexIndex + I0, BaseVertexIndex + I1, BaseVertexIndex + I2));

					new(kDOPTriangles) FkDOPBuildCollisionTriangle<uint32>(
						PayloadIndex, // Use the triangle's material index as an index into TrianglePayloads.
						V0.WorldPosition, V1.WorldPosition, V2.WorldPosition,
						Mesh->MeshIndex,
						MeshLODIndices,
						MeshHLODRange,
						bTwoSided,
						bStaticAndOpaque
						);
				}
			}
		}
	}
}

/**
 * Pre-allocates memory ahead of time, before calling AddMesh() a bunch of times.
 *
 * @param NumMeshes		- Expected number of meshes which will be added
 * @param NumVertices	- Expected number of vertices which will be added
 * @param NumTriangles	- Expected number of triangles which will be added
 */
void FDefaultAggregateMesh::ReserveMemory( int32 NumMeshes, int32 NumVertices, int32 NumTriangles )
{
	UE_LOGF(LogLightmass, Log, "Reserving memory for %d meshes, %d vertices, %d triangles", NumMeshes, NumVertices, NumTriangles);
	MeshInfos.Reserve( NumMeshes );
	Vertices.Reserve( NumVertices );
	UVs.Reserve( NumVertices );
	LightmapUVs.Reserve( NumVertices );
	TrianglePayloads.Reserve( NumTriangles );
	kDOPTriangles.Reserve( NumTriangles );
}

void FDefaultAggregateMesh::PrepareForRaytracing()
{
	// Build the kDOP for simple meshes.
	kDopTree.Build(kDOPTriangles);
	kDOPTriangles.Empty();
	TrianglePayloads.Shrink();
}

void FDefaultAggregateMesh::DumpStats() const
{
	UE_LOGF(LogLightmass, Log, "Preallocated %.1fGb for kDOP nodes and triangles", kDopTree.kDOPPreallocatedMemory / 1024.0f / 1024.0f / 1024.0f);
	UE_LOGF(LogLightmass, Log, "Building kDOP took %5.2f seconds.", kDopTree.kDOPBuildTime);

	// Log information about the aggregate mesh.
	UE_LOGF(LogLightmass, Log, "Static lighting kDOP: %u nodes, %u leaves, %u triangles, %u vertices", kDopTree.GKDOPNodes, kDopTree.GKDOPNumLeaves, kDopTree.GKDOPTriangles, Vertices.Num());
	UE_LOGF(LogLightmass, Log, "Static lighting kDOP: %.3f%% wasted space in leaves", ((kDopTree.GKDOPTriangles - kDOPTriangles.Num()) / (float)kDopTree.GKDOPTriangles) * 100.0f);

	const uint64 kDOPTreeBytes = kDopTree.Nodes.GetAllocatedSize() 
		+ kDopTree.SOATriangles.GetAllocatedSize()
		+ kDOPTriangles.GetAllocatedSize()
		+ TrianglePayloads.GetAllocatedSize()
		+ MeshInfos.GetAllocatedSize()
		+ Vertices.GetAllocatedSize()
		+ UVs.GetAllocatedSize()
		+ LightmapUVs.GetAllocatedSize();

	UE_LOGF(LogLightmass, Log, "kDopTree.Nodes        : %7.1fMb", kDopTree.Nodes.GetAllocatedSize() / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "kDopTree.SOATriangles : %7.1fMb", kDopTree.SOATriangles.GetAllocatedSize() / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "kDOPTriangles         : %7.1fMb", kDOPTriangles.GetAllocatedSize() / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "TrianglePayloads      : %7.1fMb", TrianglePayloads.GetAllocatedSize() / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "MeshInfos             : %7.1fMb", MeshInfos.GetAllocatedSize() / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "Vertices              : %7.1fMb", Vertices.GetAllocatedSize() / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "UVs                   : %7.1fMb", UVs.GetAllocatedSize() / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "LightmapUVs           : %7.1fMb", LightmapUVs.GetAllocatedSize() / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "Static lighting kDOP: %u nodes, %u leaves, %u triangles, %u vertices, %.1f Mb", kDopTree.GKDOPNodes, kDopTree.GKDOPNumLeaves, kDopTree.GKDOPTriangles, Vertices.Num(), kDOPTreeBytes / 1048576.0f);
}

void FDefaultAggregateMesh::DumpPostBuildStats() const
{
	double KDOPTrianglesTraversedRealPercent = (1.0 - ((kDopTree.GKDOPTrianglesTraversed - kDopTree.GKDOPTrianglesTraversedReal) / (kDopTree.GKDOPTrianglesTraversed * 100.0)));
	UE_LOGF(LogLightmass, Log, "kDOP traversals (in millions): %.3g parents, %.3g leaves, %.3g triangles (%.3g, %.3g%%, real triangles).",
		kDopTree.GKDOPParentNodesTraversed / 1000000.0,
		kDopTree.GKDOPLeafNodesTraversed / 1000000.0,
		kDopTree.GKDOPTrianglesTraversed / 1000000.0,
		kDopTree.GKDOPTrianglesTraversedReal / 1000000.0,
		KDOPTrianglesTraversedRealPercent);
}

class FStaticLightingAggregateMeshDataProvider
{
public:

	/** Initialization constructor. */
	FStaticLightingAggregateMeshDataProvider(const FDefaultAggregateMesh* InMesh,const FLightRay& InLightRay):
		Mesh(InMesh),
		LightRay(InLightRay)
	{}

	// kDOP data provider interface.

	FORCEINLINE const FVector4f& GetVertex(uint32 Index) const
	{
		return Mesh->Vertices[Index];
	}

	FORCEINLINE FVector2f GetUV(uint32 Index) const
	{
		return Mesh->UVs[Index];
	}

	FORCEINLINE FVector2f GetLightmapUV(uint32 Index) const
	{
		return Mesh->LightmapUVs[Index];
	}
/*
	FORCEINLINE UMaterialInterface* GetMaterial(uint32 MaterialIndex) const
	{
		return NULL;
	}
*/
	FORCEINLINE int32 GetItemIndex(uint32 MaterialIndex) const
	{
		return MaterialIndex;
	}

	FORCEINLINE const TkDOPTree<const FStaticLightingAggregateMeshDataProvider,uint32>& GetkDOPTree(void) const
	{
		return Mesh->kDopTree;
	}

	FORCEINLINE const FMatrix44f& GetLocalToWorld(void) const
	{
		return FMatrix44f::Identity;
	}

	FORCEINLINE const FMatrix44f& GetWorldToLocal(void) const
	{
		return FMatrix44f::Identity;
	}

	FORCEINLINE FMatrix44f GetLocalToWorldTransposeAdjoint(void) const
	{
		return FMatrix44f::Identity;
	}

	FORCEINLINE float GetDeterminant(void) const
	{
		return 1.0f;
	}

private:

	const FDefaultAggregateMesh* Mesh;
	const FLightRay& LightRay;
};

/**
 * Checks a light ray for intersection with the shadow mesh.
 * @param LightRay - The line segment to check for intersection.
 * @param bFindClosestIntersection - true if the intersection must return the closest intersection.  false if it may return any intersection.
 *		This can be used as an optimization for rays which only need to know if there was an intersection or not, but not any other information about the intersection.
 *		Note: bFindClosestIntersection == false currently does not handle masked materials correctly, it treats them as if they were opaque.
 *		However, bFindClosestIntersection == false does work correctly in conjunction with LIGHTRAY_STATIC_AND_OPAQUEONLY.
 * @param bCalculateTransmission - Whether to keep track of transmission or not.  If this is true, bFindClosestIntersection must also be true.
 * @param bDirectShadowingRay - Whether this ray is being used to calculate direct shadowing.
 * @param CoherentRayCache - The calling thread's collision cache.
 * @param [out] Intersection - The intersection of between the light ray and the mesh.
 * @return true if there is an intersection, false otherwise
 */
bool FDefaultAggregateMesh::IntersectLightRay(
	const FLightRay& LightRay,
	bool bFindClosestIntersection,
	bool bCalculateTransmission,
	bool bDirectShadowingRay,
	FCoherentRayCache& CoherentRayCache,
	FLightRayIntersection& ClosestIntersection) const
{
	LIGHTINGSTAT(FScopedRDTSCTimer RayTraceTimer(bFindClosestIntersection ? CoherentRayCache.FirstHitRayTraceTime : CoherentRayCache.BooleanRayTraceTime);)
	bFindClosestIntersection ? CoherentRayCache.NumFirstHitRaysTraced++ : CoherentRayCache.NumBooleanRaysTraced++;
	// Calculating transmission requires finding the closest intersection for now
	//@todo - allow boolean visibility tests while calculating transmission
	checkSlow(!bCalculateTransmission || bFindClosestIntersection);

	ClosestIntersection.bIntersects = false;
	FLinearColor Transmission(FLinearColor::White);

	//@todo - remove this hack which prevents infinite looping in some levels.
	const int32 MaxNumIterativeIntersections = 20;
	int32 NumIterativeIntersections = 0;
	do 
	{
		FLightRay ClippedLightRay = LightRay;
		if (ClosestIntersection.bIntersects)
		{
			NumIterativeIntersections++;
			// Clip the ray so it is now from the intersection point (plus some amount to avoid intersecting the same triangle again) to the original end of the ray
			ClippedLightRay.ClipAgainstIntersectionFromEnd(ClosestIntersection.IntersectionVertex.WorldPosition + Scene.SceneConstants.VisibilityRayOffsetDistance * ClippedLightRay.Direction.GetUnsafeNormal3());
			
			if ((ClosestIntersection.Mesh == LightRay.Mesh && ((ClosestIntersection.Mesh->LightingFlags & GI_INSTANCE_SELFSHADOWDISABLE) || (LightRay.TraceFlags & LIGHTRAY_SELFSHADOWDISABLE)))
				|| (bDirectShadowingRay && ClosestIntersection.Mesh->IsIndirectlyShadowedOnly(ClosestIntersection.ElementIndex))
				|| (ClosestIntersection.Mesh != LightRay.Mesh && (ClosestIntersection.Mesh->LightingFlags & GI_INSTANCE_SELFSHADOWONLY))
				|| !ClosestIntersection.Mesh->IsSurfaceDomain(ClosestIntersection.ElementIndex))
			{
				// Nothing more to do if we're just avoiding shadowing based on trace flags, just keep going
			}
			else if (ClosestIntersection.Mesh->IsMasked(ClosestIntersection.ElementIndex) || (bDirectShadowingRay && ClosestIntersection.Mesh->IsCastingShadowsAsMasked(ClosestIntersection.ElementIndex)))
			{
				// look to see if we hit a hole or opaque part
 				if (ClosestIntersection.Mesh->EvaluateMaskedCollision(ClosestIntersection.IntersectionVertex.TextureCoordinates[0], ClosestIntersection.ElementIndex))
				{
					// Hit an opaque part of a masked mesh, terminate the ray intersection
					break;
				}
			}
			else if (bCalculateTransmission)
			{
				const FLinearColor NewTransmission = ClosestIntersection.Mesh->EvaluateTransmission(
					ClosestIntersection.IntersectionVertex.TextureCoordinates[0], 
					ClosestIntersection.ElementIndex);

				// Accumulate the total transmission along the ray
				// The result is order independent so the intersections don't have to be strictly front to back
				Transmission *= NewTransmission;
			}
			ClosestIntersection.bIntersects = false;
		}

		// Check the kDOP containing low polygon meshes first.
		FHitResult Result;
		FStaticLightingAggregateMeshDataProvider kDOPDataProvider(this, ClippedLightRay);
		TkDOPLineCollisionCheck<const FStaticLightingAggregateMeshDataProvider,uint32> kDOPCheck(
			ClippedLightRay.Start,
			ClippedLightRay.Start + ClippedLightRay.Direction * ClippedLightRay.Length,
			bFindClosestIntersection,
			(LightRay.TraceFlags & LIGHTRAY_STATIC_AND_OPAQUEONLY) != 0,
			!bDirectShadowingRay,
			(LightRay.TraceFlags & LIGHTRAY_FLIP_SIDEDNESS) != 0,
			kDOPDataProvider,
			LightRay.Mapping ? LightRay.Mapping->Mesh->MeshIndex : INDEX_NONE,
			LightRay.Mapping ? LightRay.Mapping->Mesh->GetLODIndices() : INDEX_NONE,
			LightRay.Mapping ? LightRay.Mapping->Mesh->GetHLODRange() : INDEX_NONE,
			&Result);

		bool bHit = false; 
		if (!bFindClosestIntersection && CoherentRayCache.kDOPNodeIndex != 0xFFFFFFFF)
		{
			TTraversalHistory<uint32> History;
			// Trace against the last hit node if we're doing a boolean visibility check before traversing the whole tree
			// Provides a small speedup with coherent boolean visibility rays (1.1x faster for precomputed visibility)
			bHit = kDopTree.Nodes[CoherentRayCache.kDOPNodeIndex].LineCheck(kDOPCheck, History.AddNode(CoherentRayCache.kDOPNodeIndex));
		}

		if (!bHit)
		{
			bHit = kDopTree.LineCheck(kDOPCheck);
		}

		if (bHit)
		{
			// Setup a vertex to represent the intersection.
			FMinimalStaticLightingVertex IntersectionVertex;
			IntersectionVertex.WorldPosition = ClippedLightRay.Start + ClippedLightRay.Direction * ClippedLightRay.Length * Result.Time;
			IntersectionVertex.WorldTangentZ = kDOPCheck.LocalHitNormal;
			const FTriangleSOAPayload& Payload = TrianglePayloads[ Result.Item ];
			const FVector4f& v1 = kDOPDataProvider.GetVertex(Payload.VertexIndex[0]);
			const FVector4f& v2 = kDOPDataProvider.GetVertex(Payload.VertexIndex[1]);
			const FVector4f& v3 = kDOPDataProvider.GetVertex(Payload.VertexIndex[2]);
			const FVector4f LocalHitPosition = kDOPCheck.LocalStart + kDOPCheck.LocalDir * Result.Time;
			FVector4f BaryCentricWeights;
			//@todo - why is such a huge tolerance needed?  Reuse the barycentric coords calculated by the ray-triangle intersection instead of deriving them from the hit position.
			//@todo - why does this sometimes fail if there was an intersection?
			if (bFindClosestIntersection && GetBarycentricWeights(v1, v2, v3, LocalHitPosition, KINDA_SMALL_NUMBER * 100.0f, BaryCentricWeights))
			{
				const FVector2f& UV1 = kDOPDataProvider.GetUV(Payload.VertexIndex[0]);
				const FVector2f& UV2 = kDOPDataProvider.GetUV(Payload.VertexIndex[1]);
				const FVector2f& UV3 = kDOPDataProvider.GetUV(Payload.VertexIndex[2]);
				// Interpolate the material texture coordinates to the intersection point
				//@todo - only lookup and interpolate UV's if needed
				IntersectionVertex.TextureCoordinates[0] = UV1 * BaryCentricWeights.X + UV2 * BaryCentricWeights.Y + UV3 * BaryCentricWeights.Z;
				const FVector2f& LightmapUV1 = kDOPDataProvider.GetLightmapUV(Payload.VertexIndex[0]);
				const FVector2f& LightmapUV2 = kDOPDataProvider.GetLightmapUV(Payload.VertexIndex[1]);
				const FVector2f& LightmapUV3 = kDOPDataProvider.GetLightmapUV(Payload.VertexIndex[2]);
				// Interpolate the lightmap texture coordinates to the intersection point
				IntersectionVertex.TextureCoordinates[1] = LightmapUV1 * BaryCentricWeights.X + LightmapUV2 * BaryCentricWeights.Y + LightmapUV3 * BaryCentricWeights.Z;
			}
			else
			{
				IntersectionVertex.TextureCoordinates[0] = FVector2f(0,0);
				IntersectionVertex.TextureCoordinates[1] = FVector2f(0,0);
			}

			ClosestIntersection = FLightRayIntersection(true, IntersectionVertex, Payload.MeshInfo->Mesh, Payload.Mapping, Payload.ElementIndex);
			if (bFindClosestIntersection)
			{
				ClippedLightRay.ClipAgainstIntersectionFromStart(ClosestIntersection.IntersectionVertex.WorldPosition);
			}
			else
			{
				// Store off the hit node so future boolean visibility rays can test against that first
				CoherentRayCache.kDOPNodeIndex = kDOPCheck.HitNodeIndex;
				//@todo - handle masked materials correctly with !bFindClosestIntersection
				return true;
			}
		}
	} 
	// Continue tracing as long as we are intersecting meshes that might need to restart the ray
	while (ClosestIntersection.bIntersects 
		&& (ClosestIntersection.Mesh->IsTranslucent(ClosestIntersection.ElementIndex) ||
			ClosestIntersection.Mesh->IsMasked(ClosestIntersection.ElementIndex) ||
			(ClosestIntersection.Mesh == LightRay.Mesh && ((ClosestIntersection.Mesh->LightingFlags & GI_INSTANCE_SELFSHADOWDISABLE) || (LightRay.TraceFlags & LIGHTRAY_SELFSHADOWDISABLE))) ||
			// Continue tracing if we are only allowed to self shadow and intersected a different mesh
			(ClosestIntersection.Mesh != LightRay.Mesh && (ClosestIntersection.Mesh->LightingFlags & GI_INSTANCE_SELFSHADOWONLY)) ||
			(bDirectShadowingRay && ClosestIntersection.Mesh->IsIndirectlyShadowedOnly(ClosestIntersection.ElementIndex)))
		&& NumIterativeIntersections < MaxNumIterativeIntersections);

	if (NumIterativeIntersections >= MaxNumIterativeIntersections)
	{
		ClosestIntersection.bIntersects = false;
	}
	// Must not return an intersection with a translucent mesh
	checkSlow(!ClosestIntersection.bIntersects 
		|| (!ClosestIntersection.Mesh->IsTranslucent(ClosestIntersection.ElementIndex) && !ClosestIntersection.Mesh->IsSurfaceDomain(ClosestIntersection.ElementIndex))
		|| bDirectShadowingRay && ClosestIntersection.Mesh->IsCastingShadowsAsMasked(ClosestIntersection.ElementIndex));
	ClosestIntersection.Transmission = Transmission;
	return ClosestIntersection.bIntersects;
}

const FStaticLightingMesh* FDefaultAggregateMesh::IntersectBox(const FBox3f Box)  const
{
	FLightRay UnusedRay;
	FStaticLightingAggregateMeshDataProvider kDOPDataProvider(this, UnusedRay);
	TkDOPBoxCollisionCheck<const FStaticLightingAggregateMeshDataProvider, uint32> kDOPCheck(
		Box,
		kDOPDataProvider);

	kDopTree.BoxCheck(kDOPCheck);

	if (kDOPCheck.Item != -1)
	{
		const FTriangleSOAPayload& Payload = TrianglePayloads[kDOPCheck.Item];
		return Payload.MeshInfo->Mesh;
	}
	else
	{
		return nullptr;
	}
}

} //namespace Lightmass
