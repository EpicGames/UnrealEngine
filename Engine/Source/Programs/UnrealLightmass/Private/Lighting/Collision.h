// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LMOctree.h"
#include "Mappings.h"
#include "LightingMesh.h"
#include "LMkDOP.h"

namespace Lightmass
{	
class FLight;
class FScene;
class FStaticLightingMesh;
class FStaticLightingMapping;

const float TRIANGLE_AREA_THRESHOLD =0.00001f;

/** Flags set on a FLightRay that control how the ray is intersected with the scene. */
enum ELightRayIntersectionFlags
{
	LIGHTRAY_NONE				= 1<<0,
	/** Whether the ray should intersect with triangles from the ray's mesh. */
	LIGHTRAY_SELFSHADOWDISABLE	= 1<<1,
	/** Whether the ray should only intersect with static, opaque materials, and exlude masked and translucent materials. */
	LIGHTRAY_STATIC_AND_OPAQUEONLY	= 1<<2,
	/** Whether to flip what is considered a backface.  This is useful for getting consistent backface culling regardless of which side of a ray the trace starts at. */
	LIGHTRAY_FLIP_SIDEDNESS		= 1<<3
};

/** A line segment representing a direct light path through the scene. */
class FLightRay
{
public: 

	FVector4f Start;
	FVector4f End;
	FVector4f Direction;
	float Length;

	/** The mapping that the ray originated from, used for conditional intersections. */
	const FStaticLightingMapping* Mapping;
	const FStaticLightingMesh* Mesh;
	const FLight* Light;

	uint32 TraceFlags;

	FLightRay() {}

	/** Initialization constructor. */
	FLightRay(const FVector4f& InStart, const FVector4f& InEnd, const FStaticLightingMapping* InMapping, const FLight* InLight, uint32 InTraceFlags = LIGHTRAY_NONE)
	:	Start(InStart)
	,	End(InEnd)
	,	Direction(InEnd - InStart)
	,	Mapping(InMapping)
	,	Mesh(InMapping ? InMapping->Mesh : NULL)
	,	Light(InLight)
	,	TraceFlags(InTraceFlags)
	{
		Length = 1.0f;
	}

	/** Clips the light ray from the original start to an intersection point. */
	void ClipAgainstIntersectionFromStart(const FVector4f& IntersectionPoint)
	{
		End = IntersectionPoint;
		Direction = End - Start;
	}

	/** Clips the light ray from the original end to an intersection point. */
	void ClipAgainstIntersectionFromEnd(const FVector4f& IntersectionPoint)
	{
		Start = IntersectionPoint;
		Direction = End - Start;
	}
};

class FStaticLightingAggregateMeshDataProvider;  /* needs predeclaration... */

/** Information about a single mesh that got aggregated. */
struct FStaticLightingMeshInfo
{
	/** First index of the mesh into FStaticLightingAggregateMesh::Vertices, UVs and LightmapUVs */
	const int32 BaseIndex;
	const FStaticLightingMesh* Mesh;

	FStaticLightingMeshInfo(int32 InBaseIndex, const FStaticLightingMesh* InMesh) :
		BaseIndex(InBaseIndex),
		Mesh(InMesh)
	{
		checkSlow(InMesh);
	}
};

/** Each FTriangleSOA in the kDOP references 4 of these, one for each triangle it represents. */
struct FTriangleSOAPayload
{
	/** Constructor. */
	FTriangleSOAPayload(const FStaticLightingMeshInfo* InMeshInfo, const FStaticLightingMapping* InMapping, int32 InElementIndex, int32 VertexIndex1, int32 VertexIndex2, int32 VertexIndex3 )
	:	MeshInfo(InMeshInfo)
	,	Mapping(InMapping)
	,	ElementIndex(InElementIndex)
	{
		VertexIndex[0] = VertexIndex1;
		VertexIndex[1] = VertexIndex2;
		VertexIndex[2] = VertexIndex3;
		checkSlow(MeshInfo->Mesh != NULL);
		checkSlow(ElementIndex >= 0 && ElementIndex < MeshInfo->Mesh->GetNumElements());
	}

	/** Information about the Mesh that uses the triangle. */
	const FStaticLightingMeshInfo* MeshInfo;
	/** The Mapping that uses the triangle. */
	const FStaticLightingMapping* Mapping;
	/** Mesh element index */
	int32 ElementIndex;
	/** Index into FStaticLightingAggregateMesh::Vertices, UVs and LightmapUVs, for each of the vertices of the triangle. */
	int32 VertexIndex[3];
};

/** The static lighting mesh. */
class FStaticLightingAggregateMesh
{
public:

	/** Initialization constructor. */
	FStaticLightingAggregateMesh(const FScene& InScene);

	virtual ~FStaticLightingAggregateMesh() {}

	/**
	 * Merges a mesh into the shadow mesh.
	 * @param Mesh - The mesh the triangle comes from.
	 */
	virtual void AddMesh(const FStaticLightingMesh* Mesh, const FStaticLightingMapping* Mapping) = 0;

	/**
	 * Pre-allocates memory ahead of time, before calling AddMesh() a bunch of times.
	 *
	 * @param NumMeshes		- Expected number of meshes which will be added
	 * @param NumVertices	- Expected number of vertices which will be added
	 * @param NumTriangles	- Expected number of triangles which will be added
	 */
	virtual void ReserveMemory( int32 NumMeshes, int32 NumVertices, int32 NumTriangles ) = 0;

	/** Prepares the mesh for raytracing. */
	virtual void PrepareForRaytracing() = 0;

	virtual void DumpStats() const = 0;

	// Used to dump additional stats about intersect checks that where done
	virtual void DumpCheckStats() const {}

	virtual void DumpPostBuildStats() const {}

	/**
	 * Checks a light ray for intersection with the shadow mesh.
	 * @param LightRay - The line segment to check for intersection.
	 * @param bFindClosestIntersection - true if the intersection must return the closest intersection.  false if it may return any intersection.
	 *		This can be used as an optimization for rays which only need to know if there was an intersection or not, but not any other information about the intersection.
	 *		Note: bFindClosestIntersection == false currently does not handle masked materials correctly, it treats them as if they were opaque.
	 *		However, bFindClosestIntersection == false does work correctly in conjunction with LIGHTRAY_STATIC_AND_OPAQUEONLY.
	 * @param bCalculateTransmission - Whether to keep track of transmission or not.  If this is true, bFindClosestIntersection must also be true.
	 * @param bDirectShadowingRay - Whether this ray is being used to calculate direct shadowing.
	 * @param CoherentRayCache - The calling thread's collision cache.
	 * @param [out] Intersection - The intersection of between the light ray and the mesh.
	 * @return true if there is an intersection, false otherwise
	 */
	virtual bool IntersectLightRay(
		const FLightRay& LightRay,
		bool bFindClosestIntersection,
		bool bCalculateTransmission,
		bool bDirectShadowingRay,
		class FCoherentRayCache& CoherentRayCache,
		FLightRayIntersection& Intersection) const = 0;

	FBox3f GetBounds() const;

	/** The total surface area of everything in the aggregate mesh */
	float GetSurfaceArea() const { return SceneSurfaceArea; }
	/** The total surface area of everything in the aggregate mesh within the importance volume, if there is one */
	float GetSurfaceAreaWithinImportanceVolume() const { return SceneSurfaceAreaWithinImportanceVolume; }

protected:

	const FScene& Scene;

	/** True if some mesh where added to the scene. */
	bool bHasShadowCastingPrimitives;

	/** The bounding box of everything in the aggregate mesh. */
	FBox3f SceneBounds;

	/** The total surface area of everything in the aggregate mesh */
	float SceneSurfaceArea;
	/** The total surface area of everything in the aggregate mesh within the importance volume, if there is one */
	float SceneSurfaceAreaWithinImportanceVolume;
};

class FDefaultAggregateMesh final : public  FStaticLightingAggregateMesh 
{
public:

	/** Initialization constructor. */
	FDefaultAggregateMesh(const FScene& InScene) : FStaticLightingAggregateMesh(InScene) {}

	virtual ~FDefaultAggregateMesh();

	FORCEINLINE const FVector2f& GetUV(uint32 Index) const
	{
		return UVs[Index];
	}

	FORCEINLINE const FVector2f& GetLightmapUV(uint32 Index) const
	{
		return LightmapUVs[Index];
	}

	virtual void AddMesh(const FStaticLightingMesh* Mesh, const FStaticLightingMapping* Mapping) override;

	void AddMeshForVoxelization(const FStaticLightingMesh* Mesh, const FStaticLightingMapping* Mapping, bool bUseForInstancing = false);

	virtual void ReserveMemory( int32 NumMeshes, int32 NumVertices, int32 NumTriangles ) override;

	virtual void PrepareForRaytracing() override;

	virtual void DumpStats() const override;

	virtual void DumpPostBuildStats() const override;

	virtual bool IntersectLightRay(
		const FLightRay& LightRay,
		bool bFindClosestIntersection,
		bool bCalculateTransmission,
		bool bDirectShadowingRay,
		class FCoherentRayCache& CoherentRayCache,
		FLightRayIntersection& Intersection) const override;

	const FStaticLightingMesh* IntersectBox(const FBox3f Box) const;

private:

	friend class FStaticLightingAggregateMeshDataProvider;

	/** The world-space kDOP which is used by the simple meshes in the world. */
	TkDOPTree<const FStaticLightingAggregateMeshDataProvider,uint32> kDopTree;

	/** The triangles used to build the kDOP, valid until PrepareForRaytracing is called. */
	TArray<FkDOPBuildCollisionTriangle<uint32> > kDOPTriangles;
 
	/** TriangleSOA payload. Each TriangleSOA in the kDOP references 4 of these (one for each of the 4 triangles in a TriangleSOA). */
	TArray<FTriangleSOAPayload> TrianglePayloads;

	/** Information about the meshes used in the kDOP tree. */
	TArray<const FStaticLightingMeshInfo*> MeshInfos;

	/** 
	 * The vertices used by the kDOP. 
	 * @todo - should all of these vertex attributes be stored in the same array? (ArrayOfStructures instead of SoA)
	 */
	TArray<FVector4f> Vertices;

	/** The texture coordinates used by the kDOP. */
	TArray<FVector2f> UVs;

	/** The lightmap coordinates used by the kDOP. */
	TArray<FVector2f> LightmapUVs;

	/** Needed to access properties modified by AddMesh. */
	friend class FEmbreeVerifyAggregateMesh;
};

/** Information which is cached while processing a group of coherent rays. */
class FCoherentRayCache
{
public:
	uint64 NumFirstHitRaysTraced;
	uint64 NumBooleanRaysTraced;
	float FirstHitRayTraceTime;
	float BooleanRayTraceTime;

	/** 
	 * Stores the index of the last hit kDOP node when doing a boolean visibility check. 
	 * Used to optimize coherent boolean visibliity traces.
	 */
	uint32 kDOPNodeIndex;

	/** Initialization constructor. */
	FCoherentRayCache() :
		NumFirstHitRaysTraced(0),
		NumBooleanRaysTraced(0),
		FirstHitRayTraceTime(0),
		BooleanRayTraceTime(0),
		kDOPNodeIndex(0xFFFFFFFF)
	{}

	void Clear()
	{
		kDOPNodeIndex = 0xFFFFFFFF;
	}
};

} //namespace Lightmass
