// Copyright Epic Games, Inc. All Rights Reserved.

#include "Embree.h"
#include "LightingSystem.h"
#include "UnrealLightmass.h"
#include "HAL/PlatformTime.h"

// #pragma optimize( "", off )
// #define EMBREE_INLINE

#define EMBREE_INLINE FORCEINLINE

#if USE_EMBREE

namespace Lightmass
{

volatile int64 GEmbreeAllocatedSpace = 0;

void FEmbreeTransmissionAccumulator::Resolve(FLinearColor& FinalColor, float tCollide)
{
	FinalColor = FLinearColor::White;

	for (int32 i = 0; i < Colors.Num(); ++i)
	{
		const FLinearColor& Color  = Colors[i];
		if (Color.A < tCollide)
		{
			FinalColor.R *= Color.R;
			FinalColor.G *= Color.G;
			FinalColor.B *= Color.B;
		}
	}
}

void FEmbreeTransmissionAccumulator::Resolve(FLinearColor& FinalColor)
{
	FinalColor = FLinearColor::White;

	for (int32 i = 0; i < Colors.Num(); ++i)
	{
		const FLinearColor& Color  = Colors[i];
		FinalColor.R *= Color.R;
		FinalColor.G *= Color.G;
		FinalColor.B *= Color.B;
	}
}

#if USE_EMBREE_MAJOR_VERSION >= 3
FEmbreeContext::FEmbreeContext(
	const FStaticLightingMesh* InShadowMesh,
	const FStaticLightingMesh* InMappingMesh,
	uint32 InTraceFlags,
	bool InFindClosestIntersection,
	bool InCalculateTransmission,
	bool InDirectShadowingRay
	) :
	ShadowMesh(InShadowMesh),
	MappingMesh(InMappingMesh),
	TraceFlags(InTraceFlags),
	bFindClosestIntersection(InFindClosestIntersection),
	bCalculateTransmission(InCalculateTransmission),
	bDirectShadowingRay(InDirectShadowingRay),
	bStaticAndOpaqueOnly((TraceFlags & LIGHTRAY_STATIC_AND_OPAQUEONLY) != 0),
	bTwoSidedCollision(!InDirectShadowingRay),
	bFlipSidedness((TraceFlags & LIGHTRAY_FLIP_SIDEDNESS) != 0),
	ElementIndex(-1),
	TextureCoordinates(0, 0),
	LightmapCoordinates(0, 0)
{
}
#else
FEmbreeRay::FEmbreeRay(
	const FStaticLightingMesh* InShadowMesh, 
	const FStaticLightingMesh* InMappingMesh, 
	uint32 InTraceFlags,
	bool InFindClosestIntersection, 
	bool InCalculateTransmission, 
	bool InDirectShadowingRay
	) :
	ShadowMesh(InShadowMesh),
	MappingMesh(InMappingMesh),
	TraceFlags(InTraceFlags),
	bFindClosestIntersection(InFindClosestIntersection),
	bCalculateTransmission(InCalculateTransmission),
	bDirectShadowingRay(InDirectShadowingRay),
	bStaticAndOpaqueOnly((TraceFlags & LIGHTRAY_STATIC_AND_OPAQUEONLY) != 0),
	bTwoSidedCollision(!InDirectShadowingRay),
	bFlipSidedness((TraceFlags & LIGHTRAY_FLIP_SIDEDNESS) != 0),
	ElementIndex(-1),
	TextureCoordinates(0, 0),
	LightmapCoordinates(0, 0)
{
	u = v = 0;
	time = 0;
	mask = 0xFFFFFFFF;
	geomID = -1;
	instID = -1;
	primID = -1;
}

FEmbreeRay4::FEmbreeRay4(
	const FStaticLightingMesh* InShadowMesh, 
	const FStaticLightingMesh* InMappingMesh, 
	uint32 InTraceFlags,
	bool InFindClosestIntersection, 
	bool InCalculateTransmission, 
	bool InDirectShadowingRay
	) :
	ShadowMesh(InShadowMesh),
	MappingMesh(InMappingMesh),
	TraceFlags(InTraceFlags),
	bFindClosestIntersection(InFindClosestIntersection),
	bCalculateTransmission(InCalculateTransmission),
	bDirectShadowingRay(InDirectShadowingRay),
	bStaticAndOpaqueOnly((TraceFlags & LIGHTRAY_STATIC_AND_OPAQUEONLY) != 0),
	bTwoSidedCollision(!InDirectShadowingRay),
	bFlipSidedness((TraceFlags & LIGHTRAY_FLIP_SIDEDNESS) != 0)
{
	for (int32 i = 0; i < 4; i++)
	{
		u[i] = v[i] = 0;
		time[i] = 0;
		mask[i] = 0xFFFFFFFF;
		geomID[i] = -1;
		instID[i] = -1;
		primID[i] = -1;

		ElementIndex[i] = -1,
		TextureCoordinates[i] = FVector2f(0, 0);
		LightmapCoordinates[i] = FVector2f(0, 0);
	}
}
#endif // USE_EMBREE_MAJOR_VERSION >= 3

struct FEmbreeFilterProcessor
{
#if USE_EMBREE_MAJOR_VERSION >= 3
	FEmbreeFilterProcessor(
		FEmbreeContext& InEmbreeContext,
		RTCRay& InRay,
		RTCHit& InHit,
		int*& InValidMask,
		const FEmbreeGeometry& InGeo
	) :
	EmbreeContext(InEmbreeContext),
	EmbreeRay(InRay),
	EmbreeHit(InHit),
	ValidMask(InValidMask),
	Geo(InGeo),
	bCoordsDirty(true)
#else
	FEmbreeFilterProcessor(FEmbreeRay& InRay, const FEmbreeGeometry& InGeo) : Ray(InRay), Geo(InGeo), bCoordsDirty(true)
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	{
		Mesh = Geo.Mesh;

#if USE_EMBREE_MAJOR_VERSION >= 3
		Desc = Geo.TriangleDescs[EmbreeHit.primID];
		if (EmbreeHit.instID[0] != RTC_INVALID_GEOMETRY_ID)
		{
			// if instancing is used, material evaluation is deferred here
			Mesh = Geo.ParentAggregateMesh->StaticMeshInstancesToMappings[EmbreeHit.instID[0]]->Mesh;
#else
		Desc = Geo.TriangleDescs[Ray.primID];
		if (Ray.instID != -1)
		{
			// if instancing is used, material evaluation is deferred here
			Mesh = Geo.ParentAggregateMesh->StaticMeshInstancesToMappings[Ray.instID]->Mesh;
#endif // USE_EMBREE_MAJOR_VERSION >= 3

			int32 ElementIndex = Desc.ElementIndex;
			Desc.CastShadow = Mesh->IsElementCastingShadow(ElementIndex);
			Desc.StaticAndOpaqueMask = !Mesh->IsMasked(ElementIndex) && !Mesh->IsTranslucent(ElementIndex) && !Mesh->bMovable;;
			Desc.TwoSidedMask = Mesh->IsTwoSided(ElementIndex) || Mesh->IsCastingShadowAsTwoSided();
			Desc.Translucent = Mesh->IsTranslucent(ElementIndex);
			Desc.SurfaceDomain = Mesh->IsSurfaceDomain(ElementIndex);
			Desc.IndirectlyShadowedOnly = Mesh->IsIndirectlyShadowedOnly(ElementIndex);
			Desc.Masked = Mesh->IsMasked(ElementIndex);
			Desc.CastShadowAsMasked = Mesh->IsCastingShadowsAsMasked(ElementIndex);
		}
#if USE_EMBREE_MAJOR_VERSION >= 3
		s = 1 - EmbreeHit.u - EmbreeHit.v;
#else
		s = 1 - Ray.u - Ray.v;
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	}

#if USE_EMBREE_MAJOR_VERSION >= 3
	FEmbreeContext& EmbreeContext;
	RTCRay& EmbreeRay;
	RTCHit& EmbreeHit;
	int*& ValidMask;
#else
	FEmbreeRay& Ray;
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	const FEmbreeGeometry& Geo;
	FEmbreeTriangleDesc Desc;
	const FStaticLightingMesh* Mesh;

	float s; // (s,u,v) : Barycentric Weights
	int32 Index0;
	int32 Index1;
	int32 Index2;

	FVector2f TextureCoordinates; // Material Coordinates

	bool bCoordsDirty;

	void UpdateCoordinates();

	// This is called when everything succeeds and the ray is the final collision.
	void UpdateRay();

	EMBREE_INLINE void Invalidate()
	{
#if USE_EMBREE_MAJOR_VERSION >= 3
		ValidMask[0] = 0;
#else
		Ray.geomID = -1;
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	}

	EMBREE_INLINE bool IsBackFace() const
	{
#if USE_EMBREE_MAJOR_VERSION >= 3
		return EmbreeRay.dir_x * -EmbreeHit.Ng_x +  EmbreeRay.dir_y * -EmbreeHit.Ng_y + EmbreeRay.dir_z * -EmbreeHit.Ng_z < 0;
#else
		return Ray.dir[0] * Ray.Ng[0] +  Ray.dir[1] * Ray.Ng[1] + Ray.dir[2] * Ray.Ng[2] < 0;
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	}


	EMBREE_INLINE bool HitRejectedByStaticAndOpaqueOnlyTest() const
	{
#if USE_EMBREE_MAJOR_VERSION >= 3
		return EmbreeContext.bStaticAndOpaqueOnly && !Desc.StaticAndOpaqueMask;
#else
		return Ray.bStaticAndOpaqueOnly && !Desc.StaticAndOpaqueMask;
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	}

	EMBREE_INLINE bool HitRejectedByBackFaceCullingTest() const
	{
#if USE_EMBREE_MAJOR_VERSION >= 3
		if (!EmbreeContext.bTwoSidedCollision && !Desc.TwoSidedMask)
		{
			bool bIsBackFace = IsBackFace();
			if (EmbreeContext.bFlipSidedness ? !bIsBackFace : bIsBackFace)
#else
		if (!Ray.bTwoSidedCollision && !Desc.TwoSidedMask)
		{
			bool bIsBackFace = IsBackFace();
			if (Ray.bFlipSidedness ? !bIsBackFace : bIsBackFace)
#endif // USE_EMBREE_MAJOR_VERSION >= 3
			{
				return true;
			}
		}
		return false;
	}

	/**
	 * Determine ray interaction with HLODs (hierarchical LODs):
	 *
	 *				A
	 *		 /			   \
	 *		B				E
	 *	 /	   \		 /	   \
	 *	C		D		F		G
	 *
	 * Above is a HLOD tree where A is tier 2 HLOD, B and E are tier 1 HLODs. C,D,F and G are LOD0 nodes.
	 * Node range indices are assigned by a depth-first traversal beginning at the largest HLOD, i.e. node A,
	 * as this allows each HLOD to know the contained children for later rejection. Leaf nodes are always LOD0s.
	 *
	 * Stored HLOD data per node:
	 * HLODTreeIndex:	Unique index assigned to this tree of nodes.
	 * HLODRange:		Range of nodes that make up this HLOD node (self-inclusive).
	 * HLODRangeStart:	The index within the tree of this node.
	 * HLODRangeEnd:	The index within the tree of this node's final child.
	 *
	 * @return	true if the ray is rejected.
     */
	EMBREE_INLINE bool HitRejectedByHLODTest() const
	{
#if USE_EMBREE_MAJOR_VERSION >= 3
		const FStaticLightingMesh* MappingMesh = EmbreeContext.MappingMesh;
#else
		const FStaticLightingMesh* MappingMesh = Ray.MappingMesh;
#endif // USE_EMBREE_MAJOR_VERSION >= 3

		const uint32 InvalidIndex = 0xFFFF;
		uint32 GeoHLODTreeIndex = (Mesh->GetLODIndices() & 0xFFFF0000) >> 16;
		uint32 RayHLODTreeIndex = MappingMesh ? (MappingMesh->GetLODIndices() & 0xFFFF0000) >> 16 : InvalidIndex;

		bool bRejectHit = false;

		// If either Geo or Ray is a HLOD (0xFFFF being invalid HLOD)
		if (GeoHLODTreeIndex != InvalidIndex || RayHLODTreeIndex != InvalidIndex)
		{
			uint32 GeoHLODRange = Mesh->GetHLODRange();
			uint32 GeoHLODRangeStart = GeoHLODRange & 0xFFFF;
			uint32 GeoHLODRangeEnd = (GeoHLODRange & 0xFFFF0000) >> 16;

			uint32 RayHLODRange = MappingMesh ? MappingMesh->GetHLODRange() : 0;
			uint32 RayHLODRangeStart = RayHLODRange & 0xFFFF;
			uint32 RayHLODRangeEnd = (RayHLODRange & 0xFFFF0000) >> 16;

			// Different rules if nodes are within the same HLOD tree
			if (GeoHLODTreeIndex != RayHLODTreeIndex)
			{
				// Allow other meshes to interact with this tree's LOD0 nodes, else reject
				if (GeoHLODRangeStart != GeoHLODRangeEnd)
				{
					bRejectHit = true;
				}
			}
			else
			{
				// Allow shadowing within HLOD tree if:
				// * Ray and geo are same node, i.e. self-shadowing
				// * Geo is LOD0 and not a child of Ray node
				bool bIsRaySameNodeAsGeo = GeoHLODRange == RayHLODRange;
				bool bIsGeoLOD0 = GeoHLODRangeStart == GeoHLODRangeEnd;
				bool bIsGeoOutsideRayRange = GeoHLODRangeStart < RayHLODRangeStart || GeoHLODRangeStart > RayHLODRangeEnd;

				if (!(bIsRaySameNodeAsGeo || (bIsGeoLOD0 && bIsGeoOutsideRayRange)))
				{
					bRejectHit = true;
				}
			}
		}

		return bRejectHit;
	}

	EMBREE_INLINE bool HitRejectedByLODIndexTest() const
	{
		uint32 GeoMeshLODIndex = Mesh->GetLODIndices() & 0xFFFF;

#if USE_EMBREE_MAJOR_VERSION >= 3
		const FStaticLightingMesh* MappingMesh = EmbreeContext.MappingMesh;
#else
		const FStaticLightingMesh* MappingMesh = Ray.MappingMesh;
#endif // USE_EMBREE_MAJOR_VERSION >= 3

		// Only shadows from appropriate mesh LODs.
		if (MappingMesh)
		{
			// If it is not from the same mesh, then only LOD 0 can cast shadow.
			if (MappingMesh->MeshIndex != Mesh->MeshIndex)
			{
				return GeoMeshLODIndex != 0;
			}
			else
			{
				// If it is from the same mesh, then only same LOD can cast shadow.
				return (MappingMesh->GetLODIndices() & 0xFFFF) != GeoMeshLODIndex;
			}
		}

		// If the ray didn't originate from a mesh, only intersect against LOD0
		return GeoMeshLODIndex != 0;
	}

	EMBREE_INLINE bool HitRejectedBySelfShadowTest() const
	{
		// No self shadows, or only self shadow
#if USE_EMBREE_MAJOR_VERSION >= 3
		if ((Mesh == EmbreeContext.ShadowMesh && ((Mesh->LightingFlags & GI_INSTANCE_SELFSHADOWDISABLE) || (EmbreeContext.TraceFlags & LIGHTRAY_SELFSHADOWDISABLE)))
		|| (EmbreeContext.bDirectShadowingRay && Desc.IndirectlyShadowedOnly)
		|| (Mesh != EmbreeContext.ShadowMesh && (Mesh->LightingFlags & GI_INSTANCE_SELFSHADOWONLY)))
#else
		if ((Mesh == Ray.ShadowMesh && ((Mesh->LightingFlags & GI_INSTANCE_SELFSHADOWDISABLE) || (Ray.TraceFlags & LIGHTRAY_SELFSHADOWDISABLE)))
		|| (Ray.bDirectShadowingRay && Desc.IndirectlyShadowedOnly)
		|| (Mesh != Ray.ShadowMesh && (Mesh->LightingFlags & GI_INSTANCE_SELFSHADOWONLY)))
#endif // USE_EMBREE_MAJOR_VERSION >= 3
		{
			return true;
		}
		return false;
	}

	EMBREE_INLINE bool HitRejectedByAlphaTestTest()
	{
		UpdateCoordinates();

#if USE_EMBREE_MAJOR_VERSION >= 3
		if (Desc.Masked || (EmbreeContext.bDirectShadowingRay && Desc.CastShadowAsMasked))
#else
		if (Desc.Masked || (Ray.bDirectShadowingRay && Desc.CastShadowAsMasked))
#endif // USE_EMBREE_MAJOR_VERSION >= 3
		{
 			return !Mesh->EvaluateMaskedCollision(TextureCoordinates, Desc.ElementIndex);
		}
		return false;
	}
};

void FEmbreeFilterProcessor::UpdateCoordinates()
{
	if (bCoordsDirty)
	{
#if USE_EMBREE_MAJOR_VERSION >= 3
		unsigned int InstanceID = EmbreeHit.instID[0];
		unsigned int PrimID = EmbreeHit.primID;
		float HitU = EmbreeHit.u;
		float HitV = EmbreeHit.v;
#else
		unsigned int InstanceID = Ray.instID;
		unsigned int PrimID = Ray.primID;
		float HitU = Ray.u;
		float HitV = Ray.v;
#endif // USE_EMBREE_MAJOR_VERSION >= 3

		if (InstanceID == RTC_INVALID_GEOMETRY_ID)
		{
			Mesh->GetTriangleIndices(PrimID, Index0, Index1, Index2);
		}
		else
		{
			Mesh->GetInstanceableStaticMesh()->GetNonTransformedTriangleIndices(PrimID, Index0, Index1, Index2);
		}

		const FVector2f& UV1 = Geo.UVs[Index0];
		const FVector2f& UV2 = Geo.UVs[Index1];
		const FVector2f& UV3 = Geo.UVs[Index2];
		TextureCoordinates = UV1 * s + UV2 * HitU + UV3 * HitV;

		bCoordsDirty = false;
	}
}

// This is called when everything succeeds and the ray is the final collision.
void FEmbreeFilterProcessor::UpdateRay()
{
#if USE_EMBREE_MAJOR_VERSION >= 3
	// ElementIndex
	EmbreeContext.ElementIndex = Desc.ElementIndex;

	if (EmbreeContext.bFindClosestIntersection)
	{
		//@todo - only lookup and interpolate UV's if needed
		UpdateCoordinates();

		// TextureCoordinates
		EmbreeContext.TextureCoordinates = TextureCoordinates;

		// LightmapCoordinates
		const FVector2f& LightmapUV1 = Geo.LightmapUVs[Index0];
		const FVector2f& LightmapUV2 = Geo.LightmapUVs[Index1];
		const FVector2f& LightmapUV3 = Geo.LightmapUVs[Index2];
		EmbreeContext.LightmapCoordinates = LightmapUV1 * s + LightmapUV2 * EmbreeHit.u + LightmapUV3 * EmbreeHit.v;
#else
	// ElementIndex
	Ray.ElementIndex = Desc.ElementIndex;

	if (Ray.bFindClosestIntersection)
	{
		//@todo - only lookup and interpolate UV's if needed
		UpdateCoordinates();

		// TextureCoordinates
		Ray.TextureCoordinates = TextureCoordinates;

		// LightmapCoordinates
		const FVector2f& LightmapUV1 = Geo.LightmapUVs[Index0];
		const FVector2f& LightmapUV2 = Geo.LightmapUVs[Index1];
		const FVector2f& LightmapUV3 = Geo.LightmapUVs[Index2];
		Ray.LightmapCoordinates = LightmapUV1 * s + LightmapUV2 * Ray.u + LightmapUV3 * Ray.v;
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	}
	else if (bCoordsDirty)
	{
#if USE_EMBREE_MAJOR_VERSION >= 3
		unsigned int InstanceID = EmbreeHit.instID[0];
		unsigned int PrimID = EmbreeHit.primID;
#else
		unsigned int InstanceID = Ray.instID;
		unsigned int PrimID = Ray.primID;
#endif // USE_EMBREE_MAJOR_VERSION >= 3

		if (InstanceID == RTC_INVALID_GEOMETRY_ID)
		{
			Mesh->GetTriangleIndices(PrimID, Index0, Index1, Index2);
		}
		else
		{
			Mesh->GetInstanceableStaticMesh()->GetNonTransformedTriangleIndices(PrimID, Index0, Index1, Index2);
		}
	}

	// Transmission : updated outside of this scope.
}

#if USE_EMBREE_MAJOR_VERSION >= 3
// Warning: EmbreeFilterFunc must only modify FEmbreeContext outputs!
void EmbreeFilterFunc(const RTCFilterFunctionNArguments* args)
{
	int* EmbreeValid = args->valid;
	FEmbreeGeometry& EmbreeGeom = *(FEmbreeGeometry*)args->geometryUserPtr;
	FEmbreeContext& EmbreeContext = *(FEmbreeContext*)args->context;

	// We expect single-ray packets here since we use rtcIntersect1/rtcOccluded1.
	check(args->N == 1u);

	// Ignore invalid rays.
	if (EmbreeValid[0] != -1)
	{
		return;
	}

	RTCRay& EmbreeRay = *(RTCRay*)args->ray;
	RTCHit& EmbreeHit = *(RTCHit*)args->hit;

	FEmbreeFilterProcessor Proc(EmbreeContext, EmbreeRay, EmbreeHit, EmbreeValid, EmbreeGeom);
	checkSlow(Proc.Geo.GeomID == Proc.EmbreeHit.geomID);
#else
// Warning: EmbreeFilterFunc must only modify RTCRay outputs!  Nothing else is copied back to FEmbreeRay4.
void EmbreeFilterFunc(void* UserPtr, RTCRay& InRay)
{
	FEmbreeFilterProcessor Proc((FEmbreeRay&)InRay, *(FEmbreeGeometry*)UserPtr);
	checkSlow(Proc.Geo.GeomID == Proc.Ray.geomID);
#endif // USE_EMBREE_MAJOR_VERSION >= 3

	if (!Proc.Desc.CastShadow)
	{
		Proc.Invalidate();
		return;
	}

	// appLineCheckTriangleSOA
	if (Proc.HitRejectedByStaticAndOpaqueOnlyTest() || 
		Proc.HitRejectedByBackFaceCullingTest() ||
		Proc.HitRejectedByLODIndexTest() ||
		Proc.HitRejectedByHLODTest())
	{
		Proc.Invalidate();
		return;
	}

	// Only collide with surface domain materials
	if (!Proc.Desc.SurfaceDomain)
	{
		Proc.Invalidate();
		return;
	}

	// No collision with translucent primitives.
#if USE_EMBREE_MAJOR_VERSION >= 3
	if (Proc.Desc.Translucent && !(Proc.EmbreeContext.bDirectShadowingRay && Proc.Desc.CastShadowAsMasked))
	{
		if (Proc.EmbreeContext.bCalculateTransmission)
		{
			Proc.UpdateCoordinates();

			// Accumulate the total transmission along the ray
			// The result is order independent so the intersections don't have to be strictly front to back
			Proc.EmbreeContext.TransmissionAcc.Push(Proc.Mesh->EvaluateTransmission(Proc.TextureCoordinates, Proc.Desc.ElementIndex), Proc.EmbreeRay.tfar);
		}
#else
	if (Proc.Desc.Translucent && !(Proc.Ray.bDirectShadowingRay && Proc.Desc.CastShadowAsMasked))
	{
		if (Proc.Ray.bCalculateTransmission)
		{
			Proc.UpdateCoordinates();

			// Accumulate the total transmission along the ray
			// The result is order independent so the intersections don't have to be strictly front to back
			Proc.Ray.TransmissionAcc.Push(Proc.Mesh->EvaluateTransmission(Proc.TextureCoordinates, Proc.Desc.ElementIndex), Proc.Ray.tfar);
		}
#endif // USE_EMBREE_MAJOR_VERSION >= 3

		Proc.Invalidate();
		return;
	}

	// No self shadows, or only self shadow
	if (Proc.HitRejectedBySelfShadowTest())
	{
		Proc.Invalidate();
		return;
	}

#if USE_EMBREE_MAJOR_VERSION >= 3
	if (Proc.EmbreeContext.bFindClosestIntersection)
#else
	if (Proc.Ray.bFindClosestIntersection)
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	{
		if (Proc.HitRejectedByAlphaTestTest())
		{
			Proc.Invalidate();
			return;
		}
	}

	// Ray Properties need to be updated only once everything has been validated.
	// Otherwise, after a valid collision, a failed collision could be tested which must not change any property.
	Proc.UpdateRay();
}

#if USE_EMBREE_MAJOR_VERSION < 3
void EmbreeFilterFunc4(const void* valid, void* UserPtr, RTCRay4& InRay)
{
	FEmbreeRay4& EmbreeRay4 = (FEmbreeRay4&)InRay;

	const FEmbreeGeometry* TestGeometry = (FEmbreeGeometry*)UserPtr;

	for (int32 i = 0; i < 4; i++)
	{
		if (EmbreeRay4.primID[i] != uint32(-1) && EmbreeRay4.geomID[i] == TestGeometry->GeomID)
		{
			FEmbreeRay SingleRay = EmbreeRay4.BuildSingleRay(i);
			EmbreeFilterFunc(UserPtr, SingleRay);
			EmbreeRay4.SetFromSingleRay(SingleRay, i);
		}
	}
}
#endif // USE_EMBREE_MAJOR_VERSION < 3

FEmbreeGeometry::FEmbreeGeometry(
	RTCDevice EmbreeDevice, 
	RTCScene EmbreeScene, 
	const FBoxSphereBounds3f& ImportanceBounds,
	const FStaticLightingMesh* InMesh,
	const FStaticLightingMapping* InMapping,
	bool bUseForInstancing
	) :
	Mesh(InMesh),
	Mapping(InMapping),
	SurfaceArea(0),
	SurfaceAreaWithinImportanceVolume(0),
	bHasShadowCastingPrimitives(false)
{
	if (bUseForInstancing)
	{
		check(Mesh->GetInstanceableStaticMesh() != nullptr);
	}

	const FStaticLightingTextureMapping* TextureMapping = Mapping ? Mapping->GetTextureMapping() : NULL;

	TriangleDescs.AddZeroed(Mesh->NumTriangles);
	UVs.AddZeroed(Mesh->NumVertices);
	LightmapUVs.AddZeroed(Mesh->NumVertices);

#if USE_EMBREE_MAJOR_VERSION >= 3
	RTCGeometry EmbreeGeom = rtcNewGeometry(EmbreeDevice, RTC_GEOMETRY_TYPE_TRIANGLE);
	rtcSetGeometryBuildQuality(EmbreeGeom, RTC_BUILD_QUALITY_MEDIUM);
	rtcSetGeometryTimeStepCount(EmbreeGeom, 1);

	FVector4f* Vertices = (FVector4f*) rtcSetNewGeometryBuffer(
		EmbreeGeom, RTC_BUFFER_TYPE_VERTEX, 0, RTC_FORMAT_FLOAT3, 4 * sizeof(float), Mesh->NumVertices);
	int32* Indices = (int32*) rtcSetNewGeometryBuffer(
		EmbreeGeom, RTC_BUFFER_TYPE_INDEX, 0, RTC_FORMAT_UINT3, 3 * sizeof(int32), Mesh->NumTriangles);
#else
	GeomID = rtcNewTriangleMesh(EmbreeScene, RTC_GEOMETRY_STATIC, Mesh->NumTriangles, Mesh->NumVertices);

	FVector4f* Vertices = (FVector4f*) rtcMapBuffer(EmbreeScene, GeomID, RTC_VERTEX_BUFFER);
	int32* Indices = (int32*) rtcMapBuffer(EmbreeScene, GeomID, RTC_INDEX_BUFFER);
#endif // USE_EMBREE_MAJOR_VERSION >= 3

	for (int32 TriangleIndex = 0;TriangleIndex < Mesh->NumTriangles;TriangleIndex++)
	{
		int32 I0 = 0, I1 = 0, I2 = 0;
		FStaticLightingVertex V0, V1, V2;
		int32 ElementIndex;

		if (bUseForInstancing)
		{
			Mesh->GetInstanceableStaticMesh()->GetNonTransformedTriangleIndices(TriangleIndex, I0, I1, I2);
			Mesh->GetInstanceableStaticMesh()->GetNonTransformedTriangle(TriangleIndex, V0, V1, V2, ElementIndex);
		}
		else
		{
			Mesh->GetTriangleIndices(TriangleIndex, I0, I1, I2);
			Mesh->GetTriangle(TriangleIndex, V0, V1, V2, ElementIndex);
		}

		// Compute the triangle's normal.
		const FVector4f TriangleNormal = (V2.WorldPosition - V0.WorldPosition) ^ (V1.WorldPosition - V0.WorldPosition);
		// Compute the triangle area.
		const float TriangleArea = TriangleNormal.Size3() * 0.5f;

		FEmbreeTriangleDesc& Desc = TriangleDescs[TriangleIndex];
		Desc.ElementIndex = ElementIndex;
		Desc.CastShadow = false;

		if (!bUseForInstancing)
		{
			// When instancing is not used, evaluate material properties here
			// Otherwise, defer material evaluation until ray intersection
			Desc.CastShadow = TriangleArea > TRIANGLE_AREA_THRESHOLD && Mesh->IsElementCastingShadow(ElementIndex);
			Desc.StaticAndOpaqueMask = !Mesh->IsMasked(ElementIndex) && !Mesh->IsTranslucent(ElementIndex) && !Mesh->bMovable;;
			Desc.TwoSidedMask = Mesh->IsTwoSided(ElementIndex) || Mesh->IsCastingShadowAsTwoSided();
			Desc.Translucent = Mesh->IsTranslucent(ElementIndex);
			Desc.SurfaceDomain = Mesh->IsSurfaceDomain(ElementIndex);
			Desc.IndirectlyShadowedOnly = Mesh->IsIndirectlyShadowedOnly(ElementIndex);
			Desc.Masked = Mesh->IsMasked(ElementIndex);
			Desc.CastShadowAsMasked = Mesh->IsCastingShadowsAsMasked(ElementIndex);
		}

		if (TriangleArea > TRIANGLE_AREA_THRESHOLD && (bUseForInstancing || Desc.CastShadow))
		{
			Indices[TriangleIndex * 3 + 0] = I0;
			Indices[TriangleIndex * 3 + 1] = I1;
			Indices[TriangleIndex * 3 + 2] = I2;
			bHasShadowCastingPrimitives = true;
		}
		else // Otherwise map an degenerated triangle to reduce intersections
		{
			Indices[TriangleIndex * 3 + 0] = I0;
			Indices[TriangleIndex * 3 + 1] = I0;
			Indices[TriangleIndex * 3 + 2] = I0;
		}

		Vertices[I0] = V0.WorldPosition;
		Vertices[I1] = V1.WorldPosition;
		Vertices[I2] = V2.WorldPosition;

		UVs[I0] = V0.TextureCoordinates[Mesh->TextureCoordinateIndex];
		UVs[I1] = V1.TextureCoordinates[Mesh->TextureCoordinateIndex];
		UVs[I2] = V2.TextureCoordinates[Mesh->TextureCoordinateIndex];
		if (TextureMapping)
		{
			LightmapUVs[I0] = V0.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
			LightmapUVs[I1] = V1.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
			LightmapUVs[I2] = V2.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
		}

		if (!bUseForInstancing)
		{
			SurfaceArea += TriangleArea;

			// Sum the total triangle area of everything in the aggregate mesh within the importance volume,
			// if any vertex is contained or if there is no importance volume at all
			if (ImportanceBounds.SphereRadius < DELTA ||
				ImportanceBounds.GetBox().IsInside(V0.WorldPosition) ||
				ImportanceBounds.GetBox().IsInside(V1.WorldPosition) ||
				ImportanceBounds.GetBox().IsInside(V2.WorldPosition))
			{
				SurfaceAreaWithinImportanceVolume += TriangleArea;
			}
		}

	}

#if USE_EMBREE_MAJOR_VERSION >= 3
	rtcCommitGeometry(EmbreeGeom);
	GeomID = rtcAttachGeometry(EmbreeScene, EmbreeGeom);
	rtcReleaseGeometry(EmbreeGeom);

	check(rtcGetDeviceError(EmbreeDevice) == RTC_ERROR_NONE);
#else
	rtcUnmapBuffer(EmbreeScene, GeomID, RTC_VERTEX_BUFFER);
	rtcUnmapBuffer(EmbreeScene, GeomID, RTC_INDEX_BUFFER);

	check(rtcDeviceGetError(EmbreeDevice) == RTC_NO_ERROR);
#endif // USE_EMBREE_MAJOR_VERSION >= 3
}

void CalculateSurfaceArea(const FStaticLightingMesh* Mesh, const FBoxSphereBounds3f& ImportanceBounds, float& SurfaceArea, float& SurfaceAreaWithinImportanceVolume)
{
	SurfaceArea = 0.0f;
	SurfaceAreaWithinImportanceVolume = 0.0f;

	for (int32 TriangleIndex = 0; TriangleIndex < Mesh->NumTriangles; TriangleIndex++)
	{
		int32 I0 = 0, I1 = 0, I2 = 0;
		FStaticLightingVertex V0, V1, V2;
		int32 ElementIndex;

		Mesh->GetTriangleIndices(TriangleIndex, I0, I1, I2);
		Mesh->GetTriangle(TriangleIndex, V0, V1, V2, ElementIndex);

		// Compute the triangle's normal.
		const FVector4f TriangleNormal = (V2.WorldPosition - V0.WorldPosition) ^ (V1.WorldPosition - V0.WorldPosition);
		// Compute the triangle area.
		const float TriangleArea = TriangleNormal.Size3() * 0.5f;

		SurfaceArea += TriangleArea;

		// Sum the total triangle area of everything in the aggregate mesh within the importance volume,
		// if any vertex is contained or if there is no importance volume at all
		if (ImportanceBounds.SphereRadius < DELTA ||
			ImportanceBounds.GetBox().IsInside(V0.WorldPosition) ||
			ImportanceBounds.GetBox().IsInside(V1.WorldPosition) ||
			ImportanceBounds.GetBox().IsInside(V2.WorldPosition))
		{
			SurfaceAreaWithinImportanceVolume += TriangleArea;
		}
	}
}

#if USE_EMBREE_MAJOR_VERSION >= 3
bool EmbreeMemoryMonitor(void* ptr, ssize_t bytes, bool post)
#else
bool EmbreeMemoryMonitor(const ssize_t bytes, const bool post)
#endif // USE_EMBREE_MAJOR_VERSION >= 3
{
	FPlatformAtomics::InterlockedAdd(&GEmbreeAllocatedSpace, bytes);
	return true;
}

FEmbreeAggregateMesh::FEmbreeAggregateMesh(const FScene& InScene):
	FStaticLightingAggregateMesh(InScene),
	EmbreeDevice(NULL),
	EmbreeScene(NULL),
	TotalNumTriangles(0),
	TotalNumTrianglesInstanced(0)
{
	EmbreeDevice = InScene.EmbreeDevice;

#if USE_EMBREE_MAJOR_VERSION >= 3
	rtcSetDeviceMemoryMonitorFunction(EmbreeDevice, EmbreeMemoryMonitor, NULL);

	EmbreeScene = rtcNewScene(EmbreeDevice);
	rtcSetSceneBuildQuality(EmbreeScene, RTC_BUILD_QUALITY_MEDIUM);
	check(rtcGetDeviceError(EmbreeDevice) == RTC_ERROR_NONE);
#else
	rtcDeviceSetMemoryMonitorFunction(EmbreeDevice, EmbreeMemoryMonitor);

	uint32 AlgorithmFlags = RTC_INTERSECT1;

	if (InScene.GeneralSettings.bUseEmbreePacketTracing)
	{
		AlgorithmFlags |= RTC_INTERSECT4;
	}

	EmbreeScene = rtcDeviceNewScene(EmbreeDevice, RTC_SCENE_STATIC, (RTCAlgorithmFlags)AlgorithmFlags);
	check(rtcDeviceGetError(EmbreeDevice) == RTC_NO_ERROR);
#endif // USE_EMBREE_MAJOR_VERSION >= 3

	if (InScene.GeneralSettings.bUseEmbreePacketTracing)
	{
#if USE_EMBREE_MAJOR_VERSION >= 3
		ssize_t SupportsPacketTracing = rtcGetDeviceProperty(EmbreeDevice, RTC_DEVICE_PROPERTY_NATIVE_RAY4_SUPPORTED);
		check(SupportsPacketTracing);
#else
#if RTCORE_VERSION_MAJOR >= 2 && RTCORE_VERSION_MINOR >= 14
		ssize_t SupportsPacketTracing = rtcDeviceGetParameter1i(EmbreeDevice, RTC_CONFIG_INTERSECT4);
		check(SupportsPacketTracing);
#endif
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	}
}

FEmbreeAggregateMesh::~FEmbreeAggregateMesh()
{
	for (int32 i = 0; i < MeshInfos.Num(); i++)
	{
		delete MeshInfos[i];
	}

#if USE_EMBREE_MAJOR_VERSION >= 3
	rtcReleaseScene(EmbreeScene);
#else
	rtcDeleteScene(EmbreeScene);
#endif // USE_EMBREE_MAJOR_VERSION >= 3
}

/**
* Merges a mesh into the shadow mesh.
* @param Mesh - The mesh the triangle comes from.
*/
void FEmbreeAggregateMesh::AddMesh(const FStaticLightingMesh* Mesh, const FStaticLightingMapping* Mapping)
{
 	// Only use shadow casting meshes.
 	if( Mesh->LightingFlags&GI_INSTANCE_CASTSHADOW )
 	{
 		SceneBounds = SceneBounds + Mesh->BoundingBox;

		if (Scene.GeneralSettings.bUseEmbreeInstancing && Mesh->GetInstanceableStaticMesh() != nullptr)
		{
			const FStaticMeshStaticLightingMesh* StaticMeshInstance = Mesh->GetInstanceableStaticMesh();

			const FStaticMeshLOD* LOD = &StaticMeshInstance->StaticMesh->GetLOD(StaticMeshInstance->GetMeshLODIndex());

			if (!StaticMeshGeometries.Find(LOD))
			{
#if USE_EMBREE_MAJOR_VERSION >= 3
				RTCScene MeshScene = rtcNewScene(EmbreeDevice);
				rtcSetSceneBuildQuality(MeshScene, RTC_BUILD_QUALITY_MEDIUM);
#else
				RTCScene MeshScene = rtcDeviceNewScene(EmbreeDevice, RTC_SCENE_STATIC, RTC_INTERSECT1);
#endif // USE_EMBREE_MAJOR_VERSION >= 3

				FEmbreeGeometry* Geo = new FEmbreeGeometry(EmbreeDevice, MeshScene, Scene.GetImportanceBounds(), Mesh, Mapping, true);
				Geo->ParentAggregateMesh = this;
				StaticMeshGeometries.Add(LOD, FEmbreeStaticMeshGeometry { MeshScene, Geo });
				MeshInfos.Add(Geo);

#if USE_EMBREE_MAJOR_VERSION >= 3
				rtcSetGeometryIntersectFilterFunction(rtcGetGeometry(MeshScene, Geo->GeomID), EmbreeFilterFunc);
				rtcSetGeometryOccludedFilterFunction(rtcGetGeometry(MeshScene, Geo->GeomID), EmbreeFilterFunc);

				rtcSetGeometryUserData(rtcGetGeometry(MeshScene, Geo->GeomID), Geo);

				rtcCommitScene(MeshScene);
#else
				rtcSetIntersectionFilterFunction(MeshScene, Geo->GeomID, EmbreeFilterFunc);
				rtcSetOcclusionFilterFunction(MeshScene, Geo->GeomID, EmbreeFilterFunc);

				rtcSetIntersectionFilterFunction4(MeshScene, Geo->GeomID, EmbreeFilterFunc4);
				rtcSetOcclusionFilterFunction4(MeshScene, Geo->GeomID, EmbreeFilterFunc4);

				rtcSetUserData(MeshScene, Geo->GeomID, Geo);

				rtcCommit(MeshScene);
#endif // USE_EMBREE_MAJOR_VERSION >= 3

				bHasShadowCastingPrimitives |= Geo->bHasShadowCastingPrimitives;

				TotalNumTriangles += Mesh->NumTriangles;
			}
			else
			{
				TotalNumTrianglesInstanced += Mesh->NumTriangles;
			}

			// Sum the total triangle area of everything in the aggregate mesh
			float SurfaceArea, SurfaceAreaWithinImportanceVolume;
			CalculateSurfaceArea(Mesh, Scene.GetImportanceBounds(), SurfaceArea, SurfaceAreaWithinImportanceVolume);
			SceneSurfaceArea += SurfaceArea;
			SceneSurfaceAreaWithinImportanceVolume += SurfaceAreaWithinImportanceVolume;

#if USE_EMBREE_MAJOR_VERSION >= 3
			RTCGeometry EmbreeGeom = rtcNewGeometry(EmbreeDevice, RTC_GEOMETRY_TYPE_INSTANCE);
			rtcSetGeometryInstancedScene(EmbreeGeom, StaticMeshGeometries[LOD].MeshScene);
			rtcSetGeometryTimeStepCount(EmbreeGeom, 1);
			rtcCommitGeometry(EmbreeGeom);
			unsigned int InstID = rtcAttachGeometry(EmbreeScene, EmbreeGeom);
			rtcSetGeometryUserData(EmbreeGeom, (void*)Mapping);
			rtcSetGeometryTransform(EmbreeGeom, 0, RTC_FORMAT_FLOAT4X4_COLUMN_MAJOR, (const float*)StaticMeshInstance->LocalToWorld.M);
			rtcReleaseGeometry(EmbreeGeom);
			check(rtcGetDeviceError(EmbreeDevice) == RTC_ERROR_NONE);
#else
			auto InstID = rtcNewInstance(EmbreeScene, StaticMeshGeometries[LOD].MeshScene);
			rtcSetUserData(EmbreeScene, InstID, (void*)Mapping);
			rtcSetTransform(EmbreeScene, InstID, RTC_MATRIX_COLUMN_MAJOR_ALIGNED16, (const float*)StaticMeshInstance->LocalToWorld.M);
#endif // USE_EMBREE_MAJOR_VERSION >= 3

			if (StaticMeshInstancesToMappings.Num() < (int32)InstID + 1)
			{
				StaticMeshInstancesToMappings.SetNum(InstID + 1);
			}
			StaticMeshInstancesToMappings[InstID] = Mapping;

		}
		else
		{
			FEmbreeGeometry* Geo = new FEmbreeGeometry(EmbreeDevice, EmbreeScene, Scene.GetImportanceBounds(), Mesh, Mapping, false);
			MeshInfos.Add(Geo);

#if USE_EMBREE_MAJOR_VERSION >= 3
			rtcSetGeometryUserData(rtcGetGeometry(EmbreeScene, Geo->GeomID), Geo);
			rtcSetGeometryIntersectFilterFunction(rtcGetGeometry(EmbreeScene, Geo->GeomID), EmbreeFilterFunc);
			rtcSetGeometryOccludedFilterFunction(rtcGetGeometry(EmbreeScene, Geo->GeomID), EmbreeFilterFunc);
#else
			rtcSetUserData(EmbreeScene, Geo->GeomID, Geo);
			rtcSetIntersectionFilterFunction(EmbreeScene, Geo->GeomID, EmbreeFilterFunc);
			rtcSetOcclusionFilterFunction(EmbreeScene, Geo->GeomID, EmbreeFilterFunc);

			rtcSetIntersectionFilterFunction4(EmbreeScene, Geo->GeomID, EmbreeFilterFunc4);
			rtcSetOcclusionFilterFunction4(EmbreeScene, Geo->GeomID, EmbreeFilterFunc4);
#endif // USE_EMBREE_MAJOR_VERSION >= 3

			bHasShadowCastingPrimitives |= Geo->bHasShadowCastingPrimitives;

			// Sum the total triangle area of everything in the aggregate mesh
			SceneSurfaceArea += Geo->SurfaceArea;
			SceneSurfaceAreaWithinImportanceVolume += Geo->SurfaceAreaWithinImportanceVolume;
			TotalNumTriangles += Mesh->NumTriangles;
		}
	}
}

void FEmbreeAggregateMesh::PrepareForRaytracing()
{
	const double StartTime = FPlatformTime::Seconds();

#if USE_EMBREE_MAJOR_VERSION >= 3
	rtcCommitScene(EmbreeScene);
	check(rtcGetDeviceError(EmbreeDevice) == RTC_ERROR_NONE);
#else
	rtcCommit(EmbreeScene);
	check(rtcDeviceGetError(EmbreeDevice) == RTC_NO_ERROR);
#endif // USE_EMBREE_MAJOR_VERSION >= 3

	const float Buildtime = FPlatformTime::Seconds() - StartTime;
	UE_LOGF(LogLightmass, Log, "Embree Build %.1fs", Buildtime);
}

void FEmbreeAggregateMesh::DumpStats() const 
{
	int64 MeshInfoSize = sizeof(FEmbreeGeometry) * MeshInfos.Num();
	int64 UVSize = 0;
	int64 LightmapUV = 0;

	for (int32 i = 0; i < MeshInfos.Num(); i++)
	{
		const FEmbreeGeometry* Geo = MeshInfos[i];
		UVSize += Geo->UVs.GetAllocatedSize();
		LightmapUV += Geo->LightmapUVs.GetAllocatedSize();
	}

	UE_LOGF(LogLightmass, Log, "\n");
	UE_LOGF(LogLightmass, Log, "Collision Mesh Overview:");
	if (Scene.GeneralSettings.bUseEmbreeInstancing)
	{
		UE_LOGF(LogLightmass, Log, "Num Triangles         : %d (Instanced to %d)", TotalNumTriangles, TotalNumTriangles + TotalNumTrianglesInstanced);
	}
	else
	{
		UE_LOGF(LogLightmass, Log, "Num Triangles         : %d", TotalNumTriangles);
	}
	UE_LOGF(LogLightmass, Log, "MeshInfos             : %7.1fMb", MeshInfoSize / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "UVs                   : %7.1fMb", UVSize / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "LightmapUVs           : %7.1fMb", LightmapUV / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "Embree Used Memory    : %7.1fMb", GEmbreeAllocatedSpace / 1048576.0f);
	UE_LOGF(LogLightmass, Log, "\n");
}

bool FEmbreeAggregateMesh::IntersectLightRay(
	const FLightRay& LightRay,
	bool bFindClosestIntersection,
	bool bCalculateTransmission,
	bool bDirectShadowingRay,
	FCoherentRayCache& CoherentRayCache,
	FLightRayIntersection& ClosestIntersection) const
{
	LIGHTINGSTAT(FScopedRDTSCTimer RayTraceTimer(bFindClosestIntersection ? CoherentRayCache.FirstHitRayTraceTime : CoherentRayCache.BooleanRayTraceTime);)
	bFindClosestIntersection ? CoherentRayCache.NumFirstHitRaysTraced++ : CoherentRayCache.NumBooleanRaysTraced++;
	// Calculating transmission requires finding the closest intersection for now
	//@todo - allow boolean visibility tests while calculating transmission
	checkSlow(!bCalculateTransmission || bFindClosestIntersection);

	ClosestIntersection.bIntersects = false;

#if USE_EMBREE_MAJOR_VERSION >= 3
	FEmbreeContext EmbreeContext(
		LightRay.Mesh,
		LightRay.Mapping ? LightRay.Mapping->Mesh : NULL,
		LightRay.TraceFlags,
		bFindClosestIntersection,
		bCalculateTransmission,
		bDirectShadowingRay);
#if USE_EMBREE_MAJOR_VERSION >= 4
	rtcInitRayQueryContext(&EmbreeContext);
#else
	rtcInitIntersectContext(&EmbreeContext);
#endif // USE_EMBREE_MAJOR_VERSION >= 4

	RTCRayHit EmbreeRayHit;
	EmbreeRayHit.ray.org_x = LightRay.Start.X;
	EmbreeRayHit.ray.org_y = LightRay.Start.Y;
	EmbreeRayHit.ray.org_z = LightRay.Start.Z;
	EmbreeRayHit.ray.tnear = 0.0f;

	EmbreeRayHit.ray.dir_x = LightRay.Direction.X;
	EmbreeRayHit.ray.dir_y = LightRay.Direction.Y;
	EmbreeRayHit.ray.dir_z = LightRay.Direction.Z;
	EmbreeRayHit.ray.tfar = LightRay.Length;

	EmbreeRayHit.ray.time = 0.0f;
	EmbreeRayHit.ray.mask = 0xFFFFFFFF;
	EmbreeRayHit.ray.flags = 0u;

	EmbreeRayHit.hit.u = EmbreeRayHit.hit.v = 0.0f;

	EmbreeRayHit.hit.geomID = RTC_INVALID_GEOMETRY_ID;
#else
	FEmbreeRay EmbreeRay(
		LightRay.Mesh, 
		LightRay.Mapping ? LightRay.Mapping->Mesh : NULL,
		LightRay.TraceFlags, 
		bFindClosestIntersection, 
		bCalculateTransmission, 
		bDirectShadowingRay);

	EmbreeRay.org[0] = LightRay.Start.X;
	EmbreeRay.org[1] = LightRay.Start.Y;
	EmbreeRay.org[2] = LightRay.Start.Z;
	EmbreeRay.dir[0] = LightRay.Direction.X;
	EmbreeRay.dir[1] = LightRay.Direction.Y;
	EmbreeRay.dir[2] = LightRay.Direction.Z;
	EmbreeRay.tnear = 0;
	EmbreeRay.tfar = LightRay.Length;
#endif // USE_EMBREE_MAJOR_VERSION >= 3

	if (bFindClosestIntersection)
	{
#if USE_EMBREE_MAJOR_VERSION >= 3
#if USE_EMBREE_MAJOR_VERSION >= 4
		RTCIntersectArguments args;
		rtcInitIntersectArguments(&args);
		args.context = &EmbreeContext;
		rtcIntersect1(EmbreeScene, &EmbreeRayHit, &args);
#else
		rtcIntersect1(EmbreeScene, &EmbreeContext, &EmbreeRayHit);
#endif // USE_EMBREE_MAJOR_VERSION >= 4
		EmbreeRayHit.hit.Ng_x = -EmbreeRayHit.hit.Ng_x;
		EmbreeRayHit.hit.Ng_y = -EmbreeRayHit.hit.Ng_y;
		EmbreeRayHit.hit.Ng_z = -EmbreeRayHit.hit.Ng_z;
#else
		rtcIntersect(EmbreeScene, EmbreeRay);
#endif // USE_EMBREE_MAJOR_VERSION >= 3
	}
	else
	{
#if USE_EMBREE_MAJOR_VERSION >= 4
		RTCOccludedArguments args;
		rtcInitOccludedArguments(&args);
		args.context = &EmbreeContext;
		rtcOccluded1(EmbreeScene, &EmbreeRayHit.ray, &args);
#elif USE_EMBREE_MAJOR_VERSION == 3
		rtcOccluded1(EmbreeScene, &EmbreeContext, &EmbreeRayHit.ray);
#else
		rtcOccluded(EmbreeScene, EmbreeRay);
#endif // USE_EMBREE_MAJOR_VERSION >= 4
	}

#if USE_EMBREE_MAJOR_VERSION >= 3
	if (!bFindClosestIntersection)
	{
		// rtcOccluded1 : When occluded tfar is set to -infinity
		ClosestIntersection.bIntersects = (EmbreeRayHit.ray.tfar < 0.0f);
	}
	else if (EmbreeRayHit.ray.tfar >= 0.0f && EmbreeRayHit.hit.geomID != -1 && EmbreeRayHit.hit.primID != -1)
	{
		FMinimalStaticLightingVertex EmbreeVertex;
		EmbreeVertex.WorldPosition = LightRay.Start + LightRay.Direction * EmbreeRayHit.ray.tfar;

		EmbreeVertex.TextureCoordinates[0] = EmbreeContext.TextureCoordinates;
		EmbreeVertex.TextureCoordinates[1] = EmbreeContext.LightmapCoordinates;

		if (EmbreeRayHit.hit.instID[0] == -1)
		{
			const FEmbreeGeometry& Geo = *(FEmbreeGeometry*)rtcGetGeometryUserData(rtcGetGeometry(EmbreeScene, EmbreeRayHit.hit.geomID));
			EmbreeVertex.WorldTangentZ = FVector3f(EmbreeRayHit.hit.Ng_x, EmbreeRayHit.hit.Ng_y, EmbreeRayHit.hit.Ng_z).GetSafeNormal();
			ClosestIntersection = FLightRayIntersection(true, EmbreeVertex, Geo.Mesh, Geo.Mapping, EmbreeContext.ElementIndex);
		}
		else
		{
			FStaticLightingMapping* Mapping = (FStaticLightingMapping*)rtcGetGeometryUserData(rtcGetGeometry(EmbreeScene, EmbreeRayHit.hit.instID[0]));
			FVector3f GeometryNormal(EmbreeRayHit.hit.Ng_x, EmbreeRayHit.hit.Ng_y, EmbreeRayHit.hit.Ng_z);
			EmbreeVertex.WorldTangentZ = Mapping->Mesh->GetInstanceableStaticMesh()->LocalToWorldInverseTranspose.TransformVector(GeometryNormal).GetSafeNormal();
			ClosestIntersection = FLightRayIntersection(true, EmbreeVertex, Mapping->Mesh, Mapping, EmbreeContext.ElementIndex);
		}
		EmbreeContext.TransmissionAcc.Resolve(ClosestIntersection.Transmission, EmbreeRayHit.ray.tfar);
	}
	else
	{
		EmbreeContext.TransmissionAcc.Resolve(ClosestIntersection.Transmission);
	}
#else
	if (EmbreeRay.geomID != -1 && EmbreeRay.primID != -1)
	{
		FMinimalStaticLightingVertex EmbreeVertex;
		EmbreeVertex.WorldPosition = LightRay.Start + LightRay.Direction * EmbreeRay.tfar;

		EmbreeVertex.TextureCoordinates[0] = EmbreeRay.TextureCoordinates;
		EmbreeVertex.TextureCoordinates[1] = EmbreeRay.LightmapCoordinates;

		if (EmbreeRay.instID == -1)
		{
			const FEmbreeGeometry& Geo = *(FEmbreeGeometry*)rtcGetUserData(EmbreeScene, EmbreeRay.geomID);
			EmbreeVertex.WorldTangentZ = FVector3f(EmbreeRay.Ng[0], EmbreeRay.Ng[1], EmbreeRay.Ng[2]).GetSafeNormal();
			ClosestIntersection = FLightRayIntersection(true, EmbreeVertex, Geo.Mesh, Geo.Mapping, EmbreeRay.ElementIndex);
		}
		else
		{
			FStaticLightingMapping* Mapping = (FStaticLightingMapping*)rtcGetUserData(EmbreeScene, EmbreeRay.instID);
			FVector3f GeometryNormal(EmbreeRay.Ng[0], EmbreeRay.Ng[1], EmbreeRay.Ng[2]);
			EmbreeVertex.WorldTangentZ = Mapping->Mesh->GetInstanceableStaticMesh()->LocalToWorldInverseTranspose.TransformVector(GeometryNormal).GetSafeNormal();
			ClosestIntersection = FLightRayIntersection(true, EmbreeVertex, Mapping->Mesh, Mapping, EmbreeRay.ElementIndex);
		}
		EmbreeRay.TransmissionAcc.Resolve(ClosestIntersection.Transmission, EmbreeRay.tfar);
	}
	else
	{
		EmbreeRay.TransmissionAcc.Resolve(ClosestIntersection.Transmission);
	}
#endif // USE_EMBREE_MAJOR_VERSION >= 3

	return ClosestIntersection.bIntersects;
}

FEmbreeVerifyAggregateMesh::FEmbreeVerifyAggregateMesh(const FScene& InScene) :
	Super(InScene),
	DefaultAggregate(InScene),
	EmbreeAggregate(InScene),
	TransmissionMismatchCount(0),
	TransmissionEqualCount(0),
	CheckEqualCount(0),
	CheckMismatchCount(0)
{
}

void FEmbreeVerifyAggregateMesh::AddMesh(const FStaticLightingMesh* Mesh, const FStaticLightingMapping* Mapping)
{
	DefaultAggregate.AddMesh(Mesh, Mapping);
	EmbreeAggregate.AddMesh(Mesh, Mapping);

	// Update properties affected by AddMesh
	bHasShadowCastingPrimitives = DefaultAggregate.bHasShadowCastingPrimitives;
	SceneBounds = DefaultAggregate.SceneBounds;
	SceneSurfaceArea = DefaultAggregate.SceneSurfaceArea;
	SceneSurfaceAreaWithinImportanceVolume = DefaultAggregate.SceneSurfaceAreaWithinImportanceVolume;
}

void FEmbreeVerifyAggregateMesh::ReserveMemory(int32 NumMeshes, int32 NumVertices, int32 NumTriangles)
{
	DefaultAggregate.ReserveMemory(NumMeshes, NumVertices, NumTriangles);
	EmbreeAggregate.ReserveMemory(NumMeshes, NumVertices, NumTriangles);
}

void FEmbreeVerifyAggregateMesh::PrepareForRaytracing()
{
	DefaultAggregate.PrepareForRaytracing();
	EmbreeAggregate.PrepareForRaytracing();
}

void FEmbreeVerifyAggregateMesh::DumpStats() const 
{
	DefaultAggregate.DumpStats();
	EmbreeAggregate.DumpStats();
}

void FEmbreeVerifyAggregateMesh::DumpCheckStats() const
{
	DefaultAggregate.DumpCheckStats();
	EmbreeAggregate.DumpCheckStats();

	UE_LOGF(LogLightmass, Display, "\n\n");
	UE_LOGF(LogLightmass, Display, "============================================================");

	float r = TransmissionMismatchCount > 0 ? (TransmissionMismatchCount / (float)(TransmissionEqualCount + TransmissionMismatchCount)) : 0;
	UE_LOGF(LogLightmass, Log, "Embree transmission divergence : %lld / %lld [%.7f]", TransmissionMismatchCount, TransmissionEqualCount + TransmissionMismatchCount, r);

	r = CheckMismatchCount > 0 ? (CheckMismatchCount / (float)(CheckEqualCount + CheckMismatchCount)) : 0;
	UE_LOGF(LogLightmass, Log, "Embree check divergence : %lld / %lld [%.7f]", CheckMismatchCount, CheckEqualCount + CheckMismatchCount, r);

	UE_LOGF(LogLightmass, Display, "============================================================");
	UE_LOGF(LogLightmass, Display, "\n\n");
}

bool FEmbreeVerifyAggregateMesh::VerifyTransmissions(const FLightRayIntersection& EmbreeIntersection, const FLightRayIntersection& ClosestIntersection)
{
	const_cast<float&>(ClosestIntersection.Transmission.A) = 1.f;
	return EmbreeIntersection.Transmission.Equals(ClosestIntersection.Transmission, 0.01f);
}

bool FEmbreeVerifyAggregateMesh::VerifyChecks(const FLightRayIntersection& EmbreeIntersection, const FLightRayIntersection& ClosestIntersection, bool bFindClosestIntersection)
{
	if (EmbreeIntersection.bIntersects != ClosestIntersection.bIntersects)
	{
		return false;
	}

	if (bFindClosestIntersection && EmbreeIntersection.bIntersects)
	{
		if (EmbreeIntersection.ElementIndex != ClosestIntersection.ElementIndex)
		{
			return false;
		}

		const_cast<FVector4f::FReal&>(EmbreeIntersection.IntersectionVertex.WorldPosition.W) = const_cast<FVector4f::FReal&>(ClosestIntersection.IntersectionVertex.WorldPosition.W) = 1;
		if (!EmbreeIntersection.IntersectionVertex.WorldPosition.Equals(ClosestIntersection.IntersectionVertex.WorldPosition, .1f))
		{
			return false;
		}

		const_cast<FVector4f::FReal&>(EmbreeIntersection.IntersectionVertex.WorldTangentZ.W) = const_cast<FVector4f::FReal&>(ClosestIntersection.IntersectionVertex.WorldTangentZ.W) = 0;
		if (!EmbreeIntersection.IntersectionVertex.WorldTangentZ.Equals(ClosestIntersection.IntersectionVertex.WorldTangentZ, .01f))
		{
			return false;
		}

		FVector4f EmbreeCoord = FVector4f(EmbreeIntersection.IntersectionVertex.TextureCoordinates[0].X, EmbreeIntersection.IntersectionVertex.TextureCoordinates[0].Y,
								EmbreeIntersection.IntersectionVertex.TextureCoordinates[1].X, EmbreeIntersection.IntersectionVertex.TextureCoordinates[1].Y);
		FVector4f ClosestCoord = FVector4f(ClosestIntersection.IntersectionVertex.TextureCoordinates[0].X, ClosestIntersection.IntersectionVertex.TextureCoordinates[0].Y,
								ClosestIntersection.IntersectionVertex.TextureCoordinates[1].X, ClosestIntersection.IntersectionVertex.TextureCoordinates[1].Y);

		if (!EmbreeCoord.Equals(ClosestCoord, .01f))
		{
			return false;
		}
	}

	return true;
}

bool FEmbreeVerifyAggregateMesh::IntersectLightRay(
	const FLightRay& LightRay,
	bool bFindClosestIntersection,
	bool bCalculateTransmission,
	bool bDirectShadowingRay,
	FCoherentRayCache& CoherentRayCache,
	FLightRayIntersection& ClosestIntersection) const
{
	DefaultAggregate.IntersectLightRay(LightRay, bFindClosestIntersection, bCalculateTransmission, bDirectShadowingRay, CoherentRayCache, ClosestIntersection);

	FLightRayIntersection EmbreeIntersection;
	EmbreeAggregate.IntersectLightRay(LightRay, bFindClosestIntersection, bCalculateTransmission, bDirectShadowingRay, CoherentRayCache, EmbreeIntersection);

	if (bCalculateTransmission)
	{
		if (VerifyTransmissions(EmbreeIntersection, ClosestIntersection))
		{
			FPlatformAtomics::InterlockedIncrement(&TransmissionEqualCount);
		}
		else
		{
			FPlatformAtomics::InterlockedIncrement(&TransmissionMismatchCount);
		}
	}

	if (VerifyChecks(EmbreeIntersection, ClosestIntersection, bFindClosestIntersection))
	{
		FPlatformAtomics::InterlockedIncrement(&CheckEqualCount);
	}
	else
	{
		FPlatformAtomics::InterlockedIncrement(&CheckMismatchCount);
	}

	return ClosestIntersection.bIntersects;
}

} // namespace
#endif // USE_EMBREE