// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Lighting.h"

#if USE_EMBREE

namespace Lightmass
{	
	class FStaticLightingMesh;

#if USE_EMBREE_MAJOR_VERSION >= 3
	void EmbreeFilterFunc(const RTCFilterFunctionNArguments* args);
#else
	void EmbreeFilterFunc(void* UserPtr, RTCRay& InRay);
	void EmbreeFilterFunc4(const void* valid, void* UserPtr, RTCRay4& InRay);
#endif // USE_EMBREE_MAJOR_VERSION >= 3

	// Accumulates transmission color, and store the distance.
	// This is required since Embree test collision in any order, 
	// possibly accumulating transmission that are being collision
	struct FEmbreeTransmissionAccumulator
	{
		TArray<FLinearColor, TInlineAllocator<64> > Colors;
		FORCEINLINE void Push(const FLinearColor& Color, float tFar)
		{
			new (Colors) FLinearColor(Color.R, Color.G, Color.B, tFar);
		}

		void Resolve(FLinearColor& FinalColor, float tCollide);
		void Resolve(FLinearColor& FinalColor);
	};

#if USE_EMBREE_MAJOR_VERSION >= 3
#if USE_EMBREE_MAJOR_VERSION >= 4
	struct FEmbreeContext : public RTCRayQueryContext
#else
	struct FEmbreeContext : public RTCIntersectContext
#endif // USE_EMBREE_MAJOR_VERSION >= 4
	{
		FEmbreeContext(
			const FStaticLightingMesh* InShadowMesh,
			const FStaticLightingMesh* InMappingMesh,
			uint32 InTraceFlags,
			bool InFindClosestIntersection,
			bool InCalculateTransmission,
			bool InDirectShadowingRay
			);

		const FStaticLightingMesh* ShadowMesh; // Controls self shadow behavior, such as only self shadow and no self shadow.
		const FStaticLightingMesh* MappingMesh; // Controls LOD behavior to control which mesh are allowed to cast shadows
		uint32 TraceFlags;
		bool bFindClosestIntersection;
		bool bCalculateTransmission;
		bool bDirectShadowingRay;
		bool bStaticAndOpaqueOnly;
		bool bTwoSidedCollision;
		bool bFlipSidedness;

		// Outputs
		// Warning: EmbreeFilterFunc must only modify these!
		int32 ElementIndex; // Material Index
		FVector2f TextureCoordinates; // Material Coordinates
		FVector2f LightmapCoordinates;
		FEmbreeTransmissionAccumulator TransmissionAcc;
	};
#else
	struct FEmbreeRay : public RTCRay
	{
		FEmbreeRay(
			const FStaticLightingMesh* InShadowMesh, 
			const FStaticLightingMesh* InMappingMesh, 
			uint32 InTraceFlags,
			bool InFindClosestIntersection, 
			bool InCalculateTransmission, 
			bool InDirectShadowingRay
			);

		// Inputs required by filter functions
		const FStaticLightingMesh* ShadowMesh; // Controls self shadow behavior, such as only self shadow and no self shadow.
		const FStaticLightingMesh* MappingMesh; // Controls LOD behavior to control which mesh are allowed to cast shadows
		uint32 TraceFlags;
		bool bFindClosestIntersection;
		bool bCalculateTransmission;
		bool bDirectShadowingRay;
		bool bStaticAndOpaqueOnly;
		bool bTwoSidedCollision;
		bool bFlipSidedness;

		// Additional Outputs.
		// Warning: EmbreeFilterFunc must only modify these!  Nothing else is copied back to FEmbreeRay4.
		int32 ElementIndex; // Material Index
		FVector2f TextureCoordinates; // Material Coordinates
		FVector2f LightmapCoordinates;
		FEmbreeTransmissionAccumulator TransmissionAcc;
	};

	
	struct FEmbreeRay4 : public RTCRay4
	{
		FEmbreeRay4(
			const FStaticLightingMesh* InShadowMesh, 
			const FStaticLightingMesh* InMappingMesh, 
			uint32 InTraceFlags,
			bool InFindClosestIntersection, 
			bool InCalculateTransmission, 
			bool InDirectShadowingRay
			);

		inline FEmbreeRay BuildSingleRay(int32 RayIndex)
		{
			FEmbreeRay Ray(ShadowMesh, MappingMesh, TraceFlags, bFindClosestIntersection, bCalculateTransmission, bDirectShadowingRay);

			int32 debug = sizeof(FEmbreeRay);
			int32 debug2 = sizeof(FEmbreeRay4);

			//static_assert(sizeof(FEmbreeRay) == );
			//static_assert(sizeof(FEmbreeRay4) == );

			// Outputs
			Ray.ElementIndex = ElementIndex[RayIndex];
			Ray.TextureCoordinates = TextureCoordinates[RayIndex];
			Ray.LightmapCoordinates = LightmapCoordinates[RayIndex];
			Ray.TransmissionAcc = TransmissionAcc[RayIndex];


			// RTCRay members
			Ray.org[0] = orgx[RayIndex];
			Ray.dir[0] = dirx[RayIndex];
			Ray.Ng[0] = Ngx[RayIndex];

			Ray.org[1] = orgy[RayIndex];
			Ray.dir[1] = diry[RayIndex];
			Ray.Ng[1] = Ngy[RayIndex];

			Ray.org[2] = orgz[RayIndex];
			Ray.dir[2] = dirz[RayIndex];
			Ray.Ng[2] = Ngz[RayIndex];

			Ray.tnear = tnear[RayIndex];
			Ray.tfar = tfar[RayIndex];
			Ray.time = time[RayIndex];
			Ray.mask = mask[RayIndex];
			Ray.u = u[RayIndex];
			Ray.v = v[RayIndex];
			Ray.geomID = geomID[RayIndex];
			Ray.primID = primID[RayIndex];
			Ray.instID = instID[RayIndex];
			 
			return Ray;
		}

		inline void SetFromSingleRay(FEmbreeRay SingleRay, int32 RayIndex)
		{
			// Copy outputs only
			ElementIndex[RayIndex] = SingleRay.ElementIndex;
			TextureCoordinates[RayIndex] = SingleRay.TextureCoordinates;
			LightmapCoordinates[RayIndex] = SingleRay.LightmapCoordinates;
			TransmissionAcc[RayIndex] = SingleRay.TransmissionAcc;

			geomID[RayIndex] = SingleRay.geomID;
		}

		// Inputs required by filter functions
		const FStaticLightingMesh* ShadowMesh; // Controls self shadow behavior, such as only self shadow and no self shadow.
		const FStaticLightingMesh* MappingMesh; // Controls LOD behavior to control which mesh are allowed to cast shadows
		uint32 TraceFlags;
		bool bFindClosestIntersection;
		bool bCalculateTransmission;
		bool bDirectShadowingRay;
		bool bStaticAndOpaqueOnly;
		bool bTwoSidedCollision;
		bool bFlipSidedness;
		
		// Additional Outputs.
		int32 ElementIndex[4]; // Material Index
		FVector2f TextureCoordinates[4]; // Material Coordinates
		FVector2f LightmapCoordinates[4];
		FEmbreeTransmissionAccumulator TransmissionAcc[4];
	};
#endif // USE_EMBREE_MAJOR_VERSION >= 3

	struct FEmbreeTriangleDesc
	{
		int16 ElementIndex;
		uint16 CastShadow : 1;
		uint16 StaticAndOpaqueMask : 1;
		uint16 TwoSidedMask: 1;
		uint16 Translucent : 1;
		uint16 SurfaceDomain : 1;
		uint16 IndirectlyShadowedOnly : 1;
		uint16 Masked : 1;
		uint16 CastShadowAsMasked : 1;
	};

	// Mapping between Embree Geometry Id and Lightmass Mesh/LOD Id
	struct FEmbreeGeometry
	{
		FEmbreeGeometry(
			RTCDevice EmbreeDevice, 
			RTCScene EmbreeScene, 
			const FBoxSphereBounds3f& ImportanceBounds,
			const FStaticLightingMesh* InMesh,
			const FStaticLightingMapping* InMapping,
			bool bUseForInstancing
			);

		const FStaticLightingMesh* Mesh;
		const FStaticLightingMapping* Mapping;

		TArray<FEmbreeTriangleDesc> TriangleDescs; // The material ID of each triangle.
		TArray<FVector2f> UVs;
		TArray<FVector2f> LightmapUVs;

		uint32 GeomID; // Embree ID for this mesh.

		float SurfaceArea;
		float SurfaceAreaWithinImportanceVolume;
		bool bHasShadowCastingPrimitives;

		class FEmbreeAggregateMesh* ParentAggregateMesh;
	};

	class FEmbreeAggregateMesh final : public FStaticLightingAggregateMesh
	{
	public:

		/** Initialization constructor. */
		FEmbreeAggregateMesh(const FScene& InScene);

		virtual ~FEmbreeAggregateMesh();

		virtual void AddMesh(const FStaticLightingMesh* Mesh, const FStaticLightingMapping* Mapping) override;

		virtual void ReserveMemory( int32 NumMeshes, int32 NumVertices, int32 NumTriangles ) override {}

		/** Prepares the mesh for raytracing. */
		virtual void PrepareForRaytracing() override;

		virtual void DumpStats() const override;

		virtual bool IntersectLightRay(
			const FLightRay& LightRay,
			bool bFindClosestIntersection,
			bool bCalculateTransmission,
			bool bDirectShadowingRay,
			class FCoherentRayCache& CoherentRayCache,
			FLightRayIntersection& Intersection) const override;

		TArray<const FStaticLightingMapping*> StaticMeshInstancesToMappings;

	private:

		/** Information about the meshes used in the kDOP tree. */
		TArray<const FEmbreeGeometry*> MeshInfos;

		struct FEmbreeStaticMeshGeometry
		{
			RTCScene MeshScene;
			FEmbreeGeometry* Geo;
		};
		TMap<const FStaticMeshLOD*, FEmbreeStaticMeshGeometry> StaticMeshGeometries;

		/** Embree scene */
		RTCDevice EmbreeDevice;
		RTCScene EmbreeScene;
		
		/** Total number of triangles in the shadow mesh */
		int32 TotalNumTriangles;

		/** Total number of instanced triangles in the shadow mesh */
		int32 TotalNumTrianglesInstanced;
	};

	class FEmbreeVerifyAggregateMesh final : public FStaticLightingAggregateMesh
	{
		typedef FStaticLightingAggregateMesh Super;

	public:

		FEmbreeVerifyAggregateMesh(const FScene& InScene);

		virtual void AddMesh(const FStaticLightingMesh* Mesh, const FStaticLightingMapping* Mapping) override;

		virtual void ReserveMemory( int32 NumMeshes, int32 NumVertices, int32 NumTriangles ) override;

		virtual void PrepareForRaytracing() override;

		virtual void DumpStats() const override;
		virtual void DumpCheckStats() const override;

		virtual bool IntersectLightRay(
			const FLightRay& LightRay,
			bool bFindClosestIntersection,
			bool bCalculateTransmission,
			bool bDirectShadowingRay,
			class FCoherentRayCache& CoherentRayCache,
			FLightRayIntersection& Intersection) const override;

	private:

		static bool VerifyTransmissions(const FLightRayIntersection& EmbreeIntersection, const FLightRayIntersection& ClosestIntersection);
		static bool VerifyChecks(const FLightRayIntersection& EmbreeIntersection, const FLightRayIntersection& ClosestIntersection, bool bFindClosestIntersection);

		FDefaultAggregateMesh DefaultAggregate;
		FEmbreeAggregateMesh EmbreeAggregate;

		mutable volatile int64 TransmissionMismatchCount;
		mutable volatile int64 TransmissionEqualCount;

		mutable volatile int64 CheckEqualCount;
		mutable volatile int64 CheckMismatchCount;
	};

} // namespace
#endif // USE_EMBREE
