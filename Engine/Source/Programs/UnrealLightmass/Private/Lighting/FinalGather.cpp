// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Exporter.h"
#include "LightmassSwarm.h"
#include "CPUSolver.h"
#include "LightingSystem.h"
#include "MonteCarlo.h"
#include "HAL/ExceptionHandling.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
	#include <psapi.h>
#include "Windows/HideWindowsPlatformTypes.h"

#pragma comment(lib, "psapi.lib")
#endif

namespace Lightmass
{

void FStaticLightingSystem::GatherVolumeImportancePhotonDirections(
	const FVector3f WorldPosition,
	FVector3f FirstHemisphereNormal,
	FVector3f SecondHemisphereNormal,
	TArray<FVector4f>& FirstHemisphereImportancePhotonDirections,
	TArray<FVector4f>& SecondHemisphereImportancePhotonDirections,
	bool bDebugThisSample) const
{
	if (GeneralSettings.NumIndirectLightingBounces > 0 && PhotonMappingSettings.bUsePhotonMapping && PhotonMappingSettings.bUsePhotonSegmentsForVolumeLighting)
	{
		TArray<FPhotonSegmentElement> FoundPhotonSegments;
		// Gather nearby first bounce photons, which give an estimate of the first bounce incident radiance function,
		// Which we can use to importance sample the real first bounce incident radiance function.
		// See the "Extended Photon Map Implementation" paper.

		FindNearbyPhotonsInVolumeIterative(
			FirstBouncePhotonSegmentMap, 
			WorldPosition, 
			PhotonMappingSettings.NumImportanceSearchPhotons, 
			PhotonMappingSettings.MinImportancePhotonSearchDistance, 
			PhotonMappingSettings.MaxImportancePhotonSearchDistance, 
			FoundPhotonSegments,
			bDebugThisSample);

		FirstHemisphereImportancePhotonDirections.Empty(FoundPhotonSegments.Num());
		SecondHemisphereImportancePhotonDirections.Empty(FoundPhotonSegments.Num());

		for (int32 PhotonIndex = 0; PhotonIndex < FoundPhotonSegments.Num(); PhotonIndex++)
		{
			const FPhoton& CurrentPhoton = *FoundPhotonSegments[PhotonIndex].Photon;
			// Calculate the direction from the current position to the photon's source
			// Using the photon's incident direction unmodified produces artifacts proportional to the distance to that photon
			const FVector4f NewDirection = CurrentPhoton.GetPosition() + CurrentPhoton.GetIncidentDirection() * CurrentPhoton.GetDistance() - WorldPosition;
			// Only use the direction if it is in the hemisphere of the normal
			// FindNearbyPhotons only returns photons whose incident directions lie in this hemisphere, but the recalculated direction might not.
			if (Dot3(NewDirection, FirstHemisphereNormal) > 0.0f)
			{
				FirstHemisphereImportancePhotonDirections.Add(NewDirection.GetUnsafeNormal3());
			}

			if (Dot3(NewDirection, SecondHemisphereNormal) > 0.0f)
			{
				SecondHemisphereImportancePhotonDirections.Add(NewDirection.GetUnsafeNormal3());
			}
		}
	}
}

/** Calculates incident radiance for a given world space position. */
void FStaticLightingSystem::CalculateVolumeSampleIncidentRadiance(
	const TArray<FVector4f>& UniformHemisphereSamples,
	const TArray<FVector2f>& UniformHemisphereSampleUniforms,
	float MaxUnoccludedLength,
	const TArray<FVector3f, TInlineAllocator<1>>& VertexOffsets,
	FVolumeLightingSample& LightingSample,
	float& OutBackfacingHitsFraction,
	float& OutMinDistanceToSurface,
	FLMRandomStream& RandomStream,
	FStaticLightingMappingContext& MappingContext,
	bool bDebugThisSample
	) const
{
	const double StartTime = FPlatformTime::Seconds();

	const FVector4f Position = LightingSample.GetPosition();

	TArray<FVector4f> UpperHemisphereImportancePhotonDirections;
	TArray<FVector4f> LowerHemisphereImportancePhotonDirections;
	
	GatherVolumeImportancePhotonDirections(
		Position,
		FVector3f(0, 0, 1),
		FVector3f(0, 0, -1),
		UpperHemisphereImportancePhotonDirections,
		LowerHemisphereImportancePhotonDirections,
		bDebugThisSample);

	if (bDebugThisSample)
	{
		FScopeLock DebugOutputLock(&DebugOutputSync);
		FDebugStaticLightingVertex DebugVertex;
		DebugVertex.VertexNormal = FVector3f(0, 0, 1);
		DebugVertex.VertexPosition = LightingSample.GetPosition();
		MappingContext.DebugOutput->Vertices.Add(DebugVertex);
	}
		
	const double EndGatherTime = FPlatformTime::Seconds();
	MappingContext.Stats.VolumetricLightmapGatherImportancePhotonsTime += EndGatherTime - StartTime;

	FFullStaticLightingVertex RepresentativeVertex;
	RepresentativeVertex.WorldPosition = Position;
	RepresentativeVertex.TextureCoordinates[0] = FVector2f(0,0);
	RepresentativeVertex.TextureCoordinates[1] = FVector2f(0,0);

	// Construct a vertex to capture incident radiance for the positive Z hemisphere
	RepresentativeVertex.WorldTangentZ = RepresentativeVertex.TriangleNormal = FVector4f(0,0,1);
	RepresentativeVertex.GenerateVertexTangents();
	RepresentativeVertex.GenerateTriangleTangents();

	FGatheredLightSample3 UpperStaticDirectLighting;
	float UpperToggleableDirectionalLightShadowing = 1;

	CalculateApproximateDirectLighting(RepresentativeVertex, LightingSample.GetRadius(), VertexOffsets, .1f, false, false, bDebugThisSample, MappingContext, UpperStaticDirectLighting, UpperToggleableDirectionalLightShadowing);
	
	const double EndUpperDirectLightingTime = FPlatformTime::Seconds();
	MappingContext.Stats.VolumetricLightmapDirectLightingTime += EndUpperDirectLightingTime - EndGatherTime;

	int32 NumSampleAdaptiveRefinementLevels = ImportanceTracingSettings.NumAdaptiveRefinementLevels;
	float SampleAdaptiveRefinementBrightnessScale = 1.0f;
	FLightingCacheGatherInfo UpperGatherInfo;

	FFinalGatherSample3 UpperHemisphereSample = IncomingRadianceAdaptive<FFinalGatherSample3>(
		NULL, 
		RepresentativeVertex, 
		LightingSample.GetRadius(), 
		false,
		0, 
		1, 
		RBM_ScaledNormalOffset,
		GLM_FinalGather,
		NumSampleAdaptiveRefinementLevels,
		SampleAdaptiveRefinementBrightnessScale,
		UniformHemisphereSamples,
		UniformHemisphereSampleUniforms,
		MaxUnoccludedLength,
		UpperHemisphereImportancePhotonDirections, 
		MappingContext, 
		RandomStream, 
		UpperGatherInfo, 
		false,
		bDebugThisSample);

	const double EndUpperFinalGatherTime = FPlatformTime::Seconds();
	MappingContext.Stats.VolumetricLightmapFinalGatherTime += EndUpperFinalGatherTime - EndUpperDirectLightingTime;

	// Construct a vertex to capture incident radiance for the negative Z hemisphere
	RepresentativeVertex.WorldTangentZ = RepresentativeVertex.TriangleNormal = FVector4f(0,0,-1);
	RepresentativeVertex.GenerateVertexTangents();
	RepresentativeVertex.GenerateTriangleTangents();

	FLightingCacheGatherInfo LowerGatherInfo;
	FGatheredLightSample3 LowerStaticDirectLighting;
	float LowerToggleableDirectionalLightShadowing = 1;

	CalculateApproximateDirectLighting(RepresentativeVertex, LightingSample.GetRadius(), VertexOffsets, .1f, false, false, bDebugThisSample, MappingContext, LowerStaticDirectLighting, LowerToggleableDirectionalLightShadowing);

	const double EndLowerDirectLightingTime = FPlatformTime::Seconds();
	MappingContext.Stats.VolumetricLightmapDirectLightingTime += EndLowerDirectLightingTime - EndUpperFinalGatherTime;

	FFinalGatherSample3 LowerHemisphereSample = IncomingRadianceAdaptive<FFinalGatherSample3>(
		NULL, 
		RepresentativeVertex, 
		LightingSample.GetRadius(), 
		false,
		0, 
		1, 
		RBM_ScaledNormalOffset,
		GLM_FinalGather,
		NumSampleAdaptiveRefinementLevels,
		SampleAdaptiveRefinementBrightnessScale,
		UniformHemisphereSamples,
		UniformHemisphereSampleUniforms,
		MaxUnoccludedLength,
		LowerHemisphereImportancePhotonDirections, 
		MappingContext, 
		RandomStream, 
		LowerGatherInfo, 
		false,
		bDebugThisSample);

	const FGatheredLightSample3 CombinedIndirectLighting = UpperHemisphereSample + LowerHemisphereSample;
	const FGatheredLightSample3 CombinedHighQualitySample = UpperStaticDirectLighting + LowerStaticDirectLighting + CombinedIndirectLighting;

	for (int32 CoefficientIndex = 0; CoefficientIndex < LM_NUM_SH_COEFFICIENTS; CoefficientIndex++)
	{
		LightingSample.HighQualityCoefficients[CoefficientIndex][0] = CombinedHighQualitySample.SHVector.R.V[CoefficientIndex];
		LightingSample.HighQualityCoefficients[CoefficientIndex][1] = CombinedHighQualitySample.SHVector.G.V[CoefficientIndex];
		LightingSample.HighQualityCoefficients[CoefficientIndex][2] = CombinedHighQualitySample.SHVector.B.V[CoefficientIndex];

		// Copy to LQ coefficients
		LightingSample.LowQualityCoefficients[CoefficientIndex][0] = CombinedHighQualitySample.SHVector.R.V[CoefficientIndex];
		LightingSample.LowQualityCoefficients[CoefficientIndex][1] = CombinedHighQualitySample.SHVector.G.V[CoefficientIndex];
		LightingSample.LowQualityCoefficients[CoefficientIndex][2] = CombinedHighQualitySample.SHVector.B.V[CoefficientIndex];
	}

	LightingSample.DirectionalLightShadowing = FMath::Max(UpperToggleableDirectionalLightShadowing, LowerToggleableDirectionalLightShadowing);

	// Only using the upper hemisphere sky bent normal
	LightingSample.SkyBentNormal = UpperHemisphereSample.SkyOcclusion;

	OutBackfacingHitsFraction = .5f * (UpperGatherInfo.BackfacingHitsFraction + LowerGatherInfo.BackfacingHitsFraction);
	OutMinDistanceToSurface = FMath::Min(UpperGatherInfo.MinDistance, LowerGatherInfo.MinDistance);

	const double EndTime = FPlatformTime::Seconds();
	MappingContext.Stats.VolumetricLightmapFinalGatherTime += EndTime - EndLowerDirectLightingTime;
}

/** Returns environment lighting for the given direction. */
FLinearColor FStaticLightingSystem::EvaluateEnvironmentLighting(const FVector4f& IncomingDirection) const
{
	// Upper hemisphere only
	return IncomingDirection.Z < 0 ? (MaterialSettings.EnvironmentColor / (float)PI) : FLinearColor::Black;
}

void FStaticLightingSystem::EvaluateSkyLighting(const FVector4f& IncomingDirection, float PathSolidAngle, bool bShadowed, bool bForDirectLighting, FLinearColor& OutStaticLighting, FLinearColor& OutStationaryLighting) const
{
	for (int32 LightIndex = 0; LightIndex < SkyLights.Num(); LightIndex++)
	{
		FSkyLight* SkyLight = SkyLights[LightIndex];

		if (!bShadowed || !(SkyLight->LightFlags & GI_LIGHT_CASTSHADOWS))
		{
			FLinearColor Lighting = SkyLight->GetPathLighting(IncomingDirection, PathSolidAngle, !bForDirectLighting);

			if (SkyLight->LightFlags & GI_LIGHT_HASSTATICLIGHTING)
			{
				OutStaticLighting += Lighting;
			}
			else if (SkyLight->LightFlags & GI_LIGHT_HASSTATICSHADOWING)
			{
				OutStationaryLighting += Lighting;
			}
		}
	}
}

float FStaticLightingSystem::EvaluateSkyVariance(const FVector4f& IncomingDirection, float PathSolidAngle) const
{
	float Variance = 0;

	for (int32 LightIndex = 0; LightIndex < SkyLights.Num(); LightIndex++)
	{
		FSkyLight* SkyLight = SkyLights[LightIndex];

		Variance = FMath::Max(Variance, SkyLight->GetPathVariance(IncomingDirection, PathSolidAngle));
	}

	return Variance;
}

/** Calculates exitant radiance at a vertex. */
FLinearColor FStaticLightingSystem::CalculateExitantRadiance(
	const FStaticLightingMapping* HitMapping,
	const FStaticLightingMesh* HitMesh,
	const FMinimalStaticLightingVertex& Vertex,
	int32 ElementIndex,
	const FVector4f& OutgoingDirection,
	int32 BounceNumber,
	EHemisphereGatherClassification GatherClassification,
	FStaticLightingMappingContext& MappingContext,
	bool bDebugThisTexel) const
{
	FLinearColor AccumulatedRadiance = FLinearColor::Black;

	if ((GatherClassification & GLM_GatherRadiosityBuffer0) || (GatherClassification & GLM_GatherRadiosityBuffer1))
	{
		const int32 BufferIndex = GatherClassification & GLM_GatherRadiosityBuffer0 ? 0 : 1;
		const FLinearColor CachedRadiosity = HitMapping->GetCachedRadiosity(BufferIndex, HitMapping->GetSurfaceCacheIndex(Vertex));
		AccumulatedRadiance += CachedRadiosity;
	}

	if (GatherClassification & GLM_GatherLightFinalBounced)
	{
		// Reflectance is folded into the surface cache, see FinalizeSurfaceCacheTextureMapping
		AccumulatedRadiance += HitMapping->GetSurfaceCacheLighting(Vertex);
	}

	const int32 BounceNumberForEmissive = BounceNumber - 1;
	const bool bRestrictBounceNumber = GeneralSettings.ViewSingleBounceNumber >= 0 
		// We can only restrict light gathered by bounce on the final gather, on previous radiosity iterations the gathered light contributes to multiple bounces 
		&& GatherClassification == GLM_FinalGather;

	if ((GatherClassification & GLM_GatherLightEmitted) 
		&& (!bRestrictBounceNumber || BounceNumberForEmissive == GeneralSettings.ViewSingleBounceNumber)
		&& HitMesh->IsEmissive(ElementIndex))
	{
		FLinearColor Emissive = HitMesh->EvaluateEmissive(Vertex.TextureCoordinates[0], ElementIndex);
		AccumulatedRadiance += Emissive;
	}

	// So we can compare it against FLinearColor::Black easily
	AccumulatedRadiance.A = 1.0f;
	return AccumulatedRadiance;
}

void FStaticLightingSystem::IntersectLightRay(
	const FStaticLightingMapping* Mapping,
	const FFullStaticLightingVertex& Vertex,
	float SampleRadius,
	const FVector4f& WorldPathDirection,
	const FVector4f& TangentPathDirection,
	EFinalGatherRayBiasMode RayBiasMode,
	FStaticLightingMappingContext& MappingContext,
	FLightRay& OutLightRay,
	FLightRayIntersection& OutLightRayIntersection) const
{
	FVector4f SampleOffset(0,0,0);
	if (GeneralSettings.bAccountForTexelSize)
	{
		// Offset the sample's starting point in the tangent XY plane based on the sample's area of influence. 
		// This is particularly effective for large texels with high variance in the incoming radiance over the area of the texel.
		SampleOffset = Vertex.WorldTangentX * TangentPathDirection.X * SampleRadius * SceneConstants.VisibilityTangentOffsetSampleRadiusScale
			+ Vertex.WorldTangentY * TangentPathDirection.Y * SampleRadius * SceneConstants.VisibilityTangentOffsetSampleRadiusScale;
		
		// Experiment to distribute the starting position over the area of the texel to anti-alias, causes incorrect shadowing at intersections though
		//@todo - use consistent sample set between irradiance cache samples
		//const FVector2f DiskPosition = GetUniformUnitDiskPosition(RandomStream);
		//SampleOffset = Vertex.WorldTangentX * DiskPosition.X * SampleRadius * .5f + Vertex.WorldTangentY * DiskPosition.Y * SampleRadius * .5f;
	}

	const float RayStartNormalBiasScale = RayBiasMode == RBM_ConstantNormalOffset
		? SceneConstants.VisibilityNormalOffsetSampleRadiusScale 
		: (SceneConstants.VisibilityTangentOffsetSampleRadiusScale * TangentPathDirection.Z);

	// Apply various offsets to the start of the ray.
	// The offset along the ray direction is to avoid incorrect self-intersection due to floating point precision.
	// The offset along the normal is to push self-intersection patterns (like triangle shape) on highly curved surfaces onto the backfaces.
	FVector3f RayStart = Vertex.WorldPosition
		+ WorldPathDirection * SceneConstants.VisibilityRayOffsetDistance
		+ Vertex.WorldTangentZ * RayStartNormalBiasScale * SampleRadius
		+ SampleOffset;

	OutLightRay = FLightRay(
		RayStart,
		Vertex.WorldPosition + WorldPathDirection * MaxRayDistance,
		Mapping,
		NULL
		);

	MappingContext.Stats.NumFirstBounceRaysTraced += 1;
	const float LastRayTraceTime = MappingContext.RayCache.FirstHitRayTraceTime;

	AggregateMesh->IntersectLightRay(OutLightRay, true, false, false, MappingContext.RayCache, OutLightRayIntersection);
	
	MappingContext.Stats.FirstBounceRayTraceTime += MappingContext.RayCache.FirstHitRayTraceTime - LastRayTraceTime;
}

FLinearColor FStaticLightingSystem::FinalGatherSample(
	const FStaticLightingMapping* Mapping,
	const FFullStaticLightingVertex& Vertex,
	const FVector4f& WorldPathDirection,
	const FVector4f& TangentPathDirection,
	const FLightRay& PathRay,
	const FLightRayIntersection& RayIntersection,
	float PathSolidAngle,
	int32 BounceNumber,
	EHemisphereGatherClassification GatherClassification,
	bool bGatheringForCachedDirectLighting,
	bool bDebugThisTexel,
	FStaticLightingMappingContext& MappingContext,
	FLMRandomStream& RandomStream,
	FLightingCacheGatherInfo& RecordGatherInfo,
	FFinalGatherInfo& FinalGatherInfo,
	FFinalGatherHitPoint& HitPoint,
	FVector3f& OutUnoccludedSkyVector,
	FLinearColor& OutStationarySkyLighting) const
{
	FLinearColor Lighting = FLinearColor::Black;
	OutStationarySkyLighting = FLinearColor::Black;

	bool bPositiveSample = false;

	OutUnoccludedSkyVector = RayIntersection.bIntersects ? FVector3f(0) : FVector3f(WorldPathDirection);

	if (RayIntersection.bIntersects)
	{
		const float IntersectionDistance = (Vertex.WorldPosition - RayIntersection.IntersectionVertex.WorldPosition).Size3();
		RecordGatherInfo.UpdateOnHit(IntersectionDistance);

		if (IntersectionDistance < AmbientOcclusionSettings.MaxOcclusionDistance)
		{
			const float DistanceFraction = IntersectionDistance / AmbientOcclusionSettings.MaxOcclusionDistance;
			const float DistanceWeight = 1.0f - 1.0f * DistanceFraction * DistanceFraction;
			FinalGatherInfo.NumSamplesOccluded += DistanceWeight / RayIntersection.Mesh->GetFullyOccludedSamplesFraction(RayIntersection.ElementIndex);
		}

		// Only continue if the ray hit the frontface of the polygon, otherwise the ray started inside a mesh
		if (Dot3(PathRay.Direction, -RayIntersection.IntersectionVertex.WorldTangentZ) > 0.0f || RayIntersection.Mesh->IsTwoSided(RayIntersection.ElementIndex))
		{
			if (TangentPathDirection.Z > 0.0f)
			{
				if (RecordGatherInfo.HitPointRecorder)
				{
					HitPoint.MappingIndex = RayIntersection.Mapping->SceneMappingIndex;
					check(HitPoint.MappingIndex >= 0);
					HitPoint.MappingSurfaceCoordinate = RayIntersection.Mapping->GetSurfaceCacheIndex(RayIntersection.IntersectionVertex);
					check(HitPoint.MappingSurfaceCoordinate >= 0);
				}

				if (Dot3(PathRay.Direction, -RayIntersection.IntersectionVertex.WorldTangentZ) > 0.0f && GeneralSettings.NumIndirectLightingBounces > 0)
				{
					LIGHTINGSTAT(FScopedRDTSCTimer CalculateExitantRadianceTimer(MappingContext.Stats.CalculateExitantRadianceTime));

					// Calculate exitant radiance at the final gather ray intersection position.
					const FLinearColor PathVertexOutgoingRadiance = CalculateExitantRadiance(
						RayIntersection.Mapping, 
						RayIntersection.Mesh, 
						RayIntersection.IntersectionVertex, 
						RayIntersection.ElementIndex,
						-WorldPathDirection, 
						BounceNumber, 
						GatherClassification,
						MappingContext, 
						bDebugThisTexel && (!PhotonMappingSettings.bUsePhotonMapping || !PhotonMappingSettings.bVisualizePhotonImportanceSamples));

					checkSlow(FLinearColorUtils::AreFloatsValid(PathVertexOutgoingRadiance));
					Lighting += PathVertexOutgoingRadiance;

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
					if (PathVertexOutgoingRadiance.R > DELTA || PathVertexOutgoingRadiance.G > DELTA || PathVertexOutgoingRadiance.B > DELTA)
					{
						if (bDebugThisTexel)
						{
							int32 TempBreak = 0;
						}
						bPositiveSample = true;
					}
#endif
				}
			}
		}
		else
		{
			FinalGatherInfo.NumBackfaceHits++;
		}
	}
	else
	{
		if (TangentPathDirection.Z > 0 && (GatherClassification & GLM_GatherLightEmitted))
		{
			const FLinearColor EnvironmentLighting = EvaluateEnvironmentLighting(-WorldPathDirection);
			Lighting += EnvironmentLighting;
		}
	}

	const int32 BounceNumberForSkylightInFinalGather = BounceNumber - 1;
	const bool bRestrictBounceNumber = GeneralSettings.ViewSingleBounceNumber >= 0 
		// We can only restrict light gathered by bounce on the final gather, on previous radiosity iterations the gathered light contributes to multiple bounces 
		&& GatherClassification == GLM_FinalGather;

	if ((GatherClassification & GLM_GatherLightEmitted) 
		&& (!bRestrictBounceNumber || BounceNumberForSkylightInFinalGather == GeneralSettings.ViewSingleBounceNumber))
	{
		// When we're gathering lighting to cache it as direct lighting, we should take IndirectLightingScales into account
		const bool bForDirectLighting = !bGatheringForCachedDirectLighting;
		EvaluateSkyLighting(WorldPathDirection, PathSolidAngle, RayIntersection.bIntersects, bForDirectLighting, Lighting, OutStationarySkyLighting);
	}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisTexel
		&& GeneralSettings.ViewSingleBounceNumber == BounceNumber
		&& (!PhotonMappingSettings.bUsePhotonMapping || !PhotonMappingSettings.bVisualizePhotonImportanceSamples))
	{
		FDebugStaticLightingRay DebugRay(PathRay.Start, PathRay.End, RayIntersection.bIntersects, bPositiveSample != 0);
		if (RayIntersection.bIntersects)
		{
			DebugRay.End = RayIntersection.IntersectionVertex.WorldPosition;
		}
		MappingContext.DebugOutput->PathRays.Add(DebugRay);
	}
#endif

	return Lighting;
}

enum class EFinalGatherRefinementCause
{
	None,
	BrightnessDifference,
	ImportancePhotons,
	Portal,
	SkylightVariance
};

/** Stores intermediate data during a traversal of the refinement tree. */
class FRefinementTraversalContext
{
public:
	FSimpleQuadTreeNode<FRefinementElement>* Node;
	FVector2f Min;
	FVector2f Size;
	float SolidAngle;
	EFinalGatherRefinementCause RefinementCause;

	FRefinementTraversalContext(FSimpleQuadTreeNode<FRefinementElement>* InNode, FVector2f InMin, FVector2f InSize, float InSolidAngle, EFinalGatherRefinementCause InRefinementCause) :
		Node(InNode),
		Min(InMin),
		Size(InSize),
		SolidAngle(InSolidAngle),
		RefinementCause(InRefinementCause)
	{}
};

bool SphereIntersectCone(FSphere3f SphereCenterAndRadius, FVector3f ConeVertex, FVector3f ConeAxis, float ConeAngleCos, float ConeAngleSin)
{
	FVector3f U = ConeVertex - (SphereCenterAndRadius.W / ConeAngleSin) * ConeAxis;
	FVector3f D = SphereCenterAndRadius.Center - U;
	float DSizeSq = FVector3f::DotProduct(D, D);
	float E = FVector3f::DotProduct(ConeAxis, D);

	if (E > 0 && E * E >= DSizeSq * ConeAngleCos * ConeAngleCos)
	{
		D = SphereCenterAndRadius.Center - ConeVertex;
		DSizeSq = FVector3f::DotProduct(D, D);
		E = -FVector3f::DotProduct(ConeAxis, D);

		if (E > 0 && E * E >= DSizeSq * ConeAngleSin * ConeAngleSin)
		{
			return DSizeSq <= SphereCenterAndRadius.W * SphereCenterAndRadius.W;
		}
		else
		{
			return true;
		}
	}

	return false;
}

/** Data structure used for adaptive refinement.  This is basically a 2d array of quadtrees. */
class FUniformHemisphereRefinementGrid
{
public:

	FUniformHemisphereRefinementGrid(int32 InNumThetaSteps, int32 InNumPhiSteps)
	{
		NumThetaSteps = InNumThetaSteps;
		NumPhiSteps = InNumPhiSteps;
		Cells.Empty(NumThetaSteps * NumPhiSteps);
		Cells.AddZeroed(NumThetaSteps * NumPhiSteps);
	}

	/** 
	 * Fetches a leaf node value at the desired fractional position.
	 * Expects a UV that is the center of the cell being searched for, not the min. 
	 */
	const FLightingAndOcclusion& GetValue(FVector2f UV)
	{
		// Theta is radius, clamp
		const int32 ThetaIndex = FMath::Clamp(FMath::FloorToInt(UV.X * NumThetaSteps), 0, NumThetaSteps - 1);
		// Phi is angle around the hemisphere axis, wrap on both ends
		const int32 PhiIndex = (FMath::FloorToInt(UV.Y * NumPhiSteps) + NumPhiSteps) % NumPhiSteps;
		const float CellU = FMath::Fractional(FMath::Clamp(UV.X, 0.0f, .9999f) * NumThetaSteps);
		const float CellV = FMath::Abs(FMath::Fractional(UV.Y * NumPhiSteps));
		const FSimpleQuadTree<FRefinementElement>& QuadTree = Cells[ThetaIndex * NumPhiSteps + PhiIndex];

		return QuadTree.GetLeafElement(CellU, CellV).Lighting;
	}

	const FLightingAndOcclusion& GetRootValue(int32 ThetaIndex, int32 PhiIndex)
	{
		return Cells[ThetaIndex * NumPhiSteps + PhiIndex].RootNode.Element.Lighting;
	}

	/** Computes the value for the requested cell by averaging all the leaves inside the cell. */
	FLightingAndOcclusion GetFilteredValue(int32 ThetaIndex, int32 PhiIndex)
	{
		return GetFilteredValueRecursive(&Cells[ThetaIndex * NumPhiSteps + PhiIndex].RootNode);
	}

	void UpdateHitPointWeights(TArray<FFinalGatherHitPoint>& FinalGatherHitPoints, int32 ThetaIndex, int32 PhiIndex, float GridCellWeight)
	{
		UpdateHitPointWeightsRecursive(FinalGatherHitPoints, &Cells[ThetaIndex * NumPhiSteps + PhiIndex].RootNode, GridCellWeight);
	}

	void SetRootElement(int32 ThetaIndex, int32 PhiIndex, const FRefinementElement& Element)
	{
		Cells[ThetaIndex * NumPhiSteps + PhiIndex].RootNode.Element = Element;
	}

	void ReturnToFreeList(TArray<FSimpleQuadTreeNode<FRefinementElement>*>& OutNodes)
	{
		for (int32 CellIndex = 0; CellIndex < Cells.Num(); CellIndex++)
		{
			Cells[CellIndex].ReturnToFreeList(OutNodes);
		}
	}

	void RefineIncomingRadiance(
		const FStaticLightingSystem& LightingSystem,
		const FStaticLightingMapping* Mapping,
		const FFullStaticLightingVertex& Vertex,
		float SampleRadius,
		int32 BounceNumber,
		EFinalGatherRayBiasMode RayBiasMode,
		EHemisphereGatherClassification GatherClassification,
		bool bGatheringForCachedDirectLighting,
		int32 NumAdaptiveRefinementLevels,
		float BrightnessThresholdScale,
		const TArray<FVector4f, TInlineAllocator<30> >& TangentImportancePhotonDirections,
		const TArray<FSphere3f>& PortalBoundingSpheres,
		FStaticLightingMappingContext& MappingContext,
		FGatherHitPoints* HitPointRecorder,
		FLMRandomStream& RandomStream,
		bool bDebugThisTexel)
	{
		FIntPoint Neighbors[8];
		Neighbors[0] = FIntPoint(1, 0);
		Neighbors[1] = FIntPoint(-1, 0);
		Neighbors[2] = FIntPoint(0, 1);
		Neighbors[3] = FIntPoint(0, -1);
		Neighbors[4] = FIntPoint(1, 1);
		Neighbors[5] = FIntPoint(1, -1);
		Neighbors[6] = FIntPoint(-1, 1);
		Neighbors[7] = FIntPoint(-1, -1);

		TArray<FRefinementTraversalContext> NodesToRefine[2];
		NodesToRefine[0].Empty(400);
		NodesToRefine[1].Empty(400);

		const int32 NumSubsamples = 2;

		TArray<FRefinementTraversalContext>* CurrentNodesToRefine = &NodesToRefine[0];
		TArray<FRefinementTraversalContext>* NextNodesToRefine = &NodesToRefine[1];

		const float InvNumHemisphereSamples = 1.0f / (NumThetaSteps * NumPhiSteps);
		float ImportanceConeAngle = LightingSystem.ImportanceTracingSettings.AdaptiveFirstBouncePhotonConeAngle;
		// Approximation for the cone angle of a root level cell
		const float RootCellAngle = PI * FMath::Sqrt((.5f / NumThetaSteps) * (.5f / NumThetaSteps) + (.5f / NumPhiSteps) * (.5f / NumPhiSteps));
		const float CosRootCellAngle = FMath::Cos(RootCellAngle);
		const float SinRootCellAngle = FMath::Sin(RootCellAngle);
		const float RootSolidAngle = 2 * PI * (1 - CosRootCellAngle);
		const float RootCombinedAngleThreshold = FMath::Cos(ImportanceConeAngle + RootCellAngle);
		const float ConeIntersectionWeight = 1.0f / TangentImportancePhotonDirections.Num();

		float BrightnessThreshold = LightingSystem.ImportanceTracingSettings.AdaptiveBrightnessThreshold * BrightnessThresholdScale;
		float SkyOcclusionThreshold = LightingSystem.ImportanceTracingSettings.AdaptiveBrightnessThreshold * BrightnessThresholdScale;
		bool bRefineForSkyOcclusion = LightingSystem.SkyLights.Num() > 0;
		float SkyVarianceThreshold = LightingSystem.ImportanceTracingSettings.AdaptiveSkyVarianceThreshold;

		// This is basically disabled, causes too much noise in worst case scenarios (all GI coming from small bright spot)
		float ConeWeightThreshold = .006f;

		// Operate on all cells at a refinement depth before going deeper
		// This is necessary for the neighbor comparisons to work right
		for (int32 RefinementDepth = 0; RefinementDepth < NumAdaptiveRefinementLevels; RefinementDepth++)
		{
			FLinearColor TotalLighting = FLinearColor::Black;

			// Recalculate total lighting based on the refined results
			for (int32 ThetaIndex = 0; ThetaIndex < NumThetaSteps; ThetaIndex++)
			{
				for (int32 PhiIndex = 0; PhiIndex < NumPhiSteps; PhiIndex++)
				{
					const FLightingAndOcclusion FilteredLighting = GetFilteredValue(ThetaIndex, PhiIndex);
					TotalLighting += FilteredLighting.Lighting + FilteredLighting.StationarySkyLighting;
				}
			}

			// Normalize by sample count
			TotalLighting *= InvNumHemisphereSamples;

			const float AverageBrightness = FMath::Max(TotalLighting.GetLuminance(), .01f);

			// At depth 0 we are operating on the 2d grid 
			if (RefinementDepth == 0)
			{
				for (int32 ThetaIndex = 0; ThetaIndex < NumThetaSteps; ThetaIndex++)
				{
					for (int32 PhiIndex = 0; PhiIndex < NumPhiSteps; PhiIndex++)
					{
						const FVector4f CellCenterTangentDirection = UniformSampleHemisphere((ThetaIndex + .5f) / (float)NumThetaSteps, (PhiIndex + .5f) / (float)NumPhiSteps);
						const FVector4f CellCenterWorldDirection = Vertex.TransformTriangleTangentVectorToWorld(CellCenterTangentDirection);
						float IntersectingImportanceConeWeight = 0;

						// Accumulate weight of intersecting photon cones
						for (int32 ImportanceDirectionIndex = 0; ImportanceDirectionIndex < TangentImportancePhotonDirections.Num(); ImportanceDirectionIndex++)
						{
							const float CosAngleBetweenCones = Dot3(TangentImportancePhotonDirections[ImportanceDirectionIndex], CellCenterTangentDirection);
								
							// Cone intersection by comparing the cosines of angles
							// In the range [0, PI], cosine is always decreasing while the input angle is increasing, so we can just flip the comparison from what we would do on the angle
							if (CosAngleBetweenCones > RootCombinedAngleThreshold)
							{
								IntersectingImportanceConeWeight += ConeIntersectionWeight;

								if (IntersectingImportanceConeWeight >= ConeWeightThreshold)
								{
									break;
								}
							}
						}

						EFinalGatherRefinementCause RefinementCause = EFinalGatherRefinementCause::None;

						if (IntersectingImportanceConeWeight >= ConeWeightThreshold)
						{
							RefinementCause = EFinalGatherRefinementCause::ImportancePhotons;
						}

						if (RefinementCause == EFinalGatherRefinementCause::None)
						{
							for (int32 PortalIndex = 0; PortalIndex < PortalBoundingSpheres.Num(); PortalIndex++)
							{
								if (SphereIntersectCone(PortalBoundingSpheres[PortalIndex], Vertex.WorldPosition, CellCenterWorldDirection, CosRootCellAngle, SinRootCellAngle))
								{
									RefinementCause = EFinalGatherRefinementCause::Portal;
									break;
								}
							}
						}

						float MaxRelativeDifference = 0;
						float MaxSkyOcclusionDifference = 0;

						// Determine maximum relative brightness difference
						if (RefinementCause == EFinalGatherRefinementCause::None)
						{
							const FLightingAndOcclusion RootElementLighting = GetRootValue(ThetaIndex, PhiIndex);
							const FLinearColor Radiance = RootElementLighting.Lighting + RootElementLighting.StationarySkyLighting;
							const float RelativeBrightness = Radiance.GetLuminance() / AverageBrightness;

							for (int32 NeighborIndex = 0; NeighborIndex < UE_ARRAY_COUNT(Neighbors); NeighborIndex++)
							{
								int32 NeighborTheta = ThetaIndex + Neighbors[NeighborIndex].X;
								// Wrap phi around, since it is the angle around the hemisphere axis
								// Add NumPhiSteps to handle negative
								int32 NeighborPhi = ((PhiIndex + Neighbors[NeighborIndex].Y) + NumPhiSteps) % NumPhiSteps;

								if (NeighborTheta >= 0 && NeighborTheta < NumThetaSteps)
								{
									const FLightingAndOcclusion NeighborLighting = GetRootValue(NeighborTheta, NeighborPhi);
									const float NeighborBrightness = (NeighborLighting.Lighting + NeighborLighting.StationarySkyLighting).GetLuminance();
									const float NeighborRelativeBrightness = NeighborBrightness / AverageBrightness;
									MaxRelativeDifference = FMath::Max(MaxRelativeDifference, FMath::Abs(RelativeBrightness - NeighborRelativeBrightness));

									MaxSkyOcclusionDifference = FMath::Max(MaxSkyOcclusionDifference, FMath::Abs(RootElementLighting.UnoccludedSkyVector.SizeSquared() - NeighborLighting.UnoccludedSkyVector.SizeSquared()));
								}
							}

							if (MaxRelativeDifference > BrightnessThreshold || (bRefineForSkyOcclusion && MaxSkyOcclusionDifference > SkyOcclusionThreshold))
							{
								RefinementCause = EFinalGatherRefinementCause::BrightnessDifference;
							}
						}

						if (RefinementCause == EFinalGatherRefinementCause::None)
						{
							float SkyVariance = LightingSystem.EvaluateSkyVariance(CellCenterWorldDirection, RootSolidAngle);

							if (SkyVariance > SkyVarianceThreshold)
							{
								RefinementCause = EFinalGatherRefinementCause::SkylightVariance;
							}
						}

						// Refine if the importance cone threshold is exceeded or there was a big enough brightness difference
						if (RefinementCause != EFinalGatherRefinementCause::None)
						{
							FSimpleQuadTreeNode<FRefinementElement>* Node = &Cells[ThetaIndex * NumPhiSteps + PhiIndex].RootNode;

							NextNodesToRefine->Add(FRefinementTraversalContext(
								Node, 
								FVector2f(ThetaIndex / (float)NumThetaSteps, PhiIndex / (float)NumPhiSteps),
								FVector2f(1 / (float)NumThetaSteps, 1 / (float)NumPhiSteps),
								RootSolidAngle,
								RefinementCause));
						}
					}
				}
			}
			// At depth > 0 we are operating on quadtree nodes
			else
			{
				// Reset output without reallocating
				NextNodesToRefine->Reset(); 

				float SubCellCombinedAngleThreshold = 0;
				float CosSubCellAngle = 0;
				float SinSubCellAngle = 0;
				float SubCellSolidAngle = 0;
				// The cell size will be the same for all cells of this depth, so calculate it once
				if (CurrentNodesToRefine->Num() > 0)
				{
					FRefinementTraversalContext NodeContext = (*CurrentNodesToRefine)[0];
					const FVector2f HalfSubCellSize = NodeContext.Size / 4;
					// Approximate the cone angle of the sub cell
					const float SubCellAngle = PI * FMath::Sqrt(HalfSubCellSize.X * HalfSubCellSize.X + HalfSubCellSize.Y * HalfSubCellSize.Y);
					SubCellCombinedAngleThreshold = FMath::Cos(ImportanceConeAngle + SubCellAngle);
					CosSubCellAngle = FMath::Cos(SubCellAngle);
					SinSubCellAngle = FMath::Sin(SubCellAngle);
					SubCellSolidAngle = 2 * PI * (1 - CosSubCellAngle);
				}

				for (int32 NodeIndex = 0; NodeIndex < CurrentNodesToRefine->Num(); NodeIndex++)
				{
					FRefinementTraversalContext NodeContext = (*CurrentNodesToRefine)[NodeIndex];
					const FVector2f HalfSubCellSize = NodeContext.Size / 4;

					for (int32 SubThetaIndex = 0; SubThetaIndex < NumSubsamples; SubThetaIndex++)
					{
						for (int32 SubPhiIndex = 0; SubPhiIndex < NumSubsamples; SubPhiIndex++)
						{
							FSimpleQuadTreeNode<FRefinementElement>* ChildNode = NodeContext.Node->Children[SubThetaIndex * NumSubsamples + SubPhiIndex];

							const FVector4f CellCenterTangentDirection = UniformSampleHemisphere(
								NodeContext.Min.X + SubThetaIndex * NodeContext.Size.X / 2 + NodeContext.Size.X / 4,
								NodeContext.Min.Y + SubPhiIndex * NodeContext.Size.Y / 2 + NodeContext.Size.Y / 4);

							const FVector4f CellCenterWorldDirection = Vertex.TransformTriangleTangentVectorToWorld(CellCenterTangentDirection);

							float IntersectingImportanceConeWeight = 0;

							// Accumulate weight of intersecting photon cones
							for (int32 ImportanceDirectionIndex = 0; ImportanceDirectionIndex < TangentImportancePhotonDirections.Num(); ImportanceDirectionIndex++)
							{
								const float CosAngleBetweenCones = Dot3(TangentImportancePhotonDirections[ImportanceDirectionIndex], CellCenterTangentDirection);

								// Cone intersection by comparing the cosines of angles
								// In the range [0, PI], cosine is always decreasing while the input angle is increasing, so we can just flip the comparison from what we would do on the angle
								if (CosAngleBetweenCones > SubCellCombinedAngleThreshold)
								{
									IntersectingImportanceConeWeight += ConeIntersectionWeight;

									if (IntersectingImportanceConeWeight >= ConeWeightThreshold)
									{
										break;
									}
								}
							}

							EFinalGatherRefinementCause RefinementCause = EFinalGatherRefinementCause::None;

							if (IntersectingImportanceConeWeight >= ConeWeightThreshold)
							{
								RefinementCause = EFinalGatherRefinementCause::ImportancePhotons;
							}

							if (RefinementCause == EFinalGatherRefinementCause::None)
							{
								for (int32 PortalIndex = 0; PortalIndex < PortalBoundingSpheres.Num(); PortalIndex++)
								{
									if (SphereIntersectCone(PortalBoundingSpheres[PortalIndex], Vertex.WorldPosition, CellCenterWorldDirection, CosSubCellAngle, SinSubCellAngle))
									{
										RefinementCause = EFinalGatherRefinementCause::Portal;
										break;
									}
								}
							}

							float MaxRelativeDifference = 0;
							float MaxSkyOcclusionDifference = 0;

							// Determine maximum relative brightness difference
							if (RefinementCause == EFinalGatherRefinementCause::None)
							{
								const FLightingAndOcclusion ChildLighting = ChildNode->Element.Lighting;
								const FLinearColor Radiance = ChildLighting.Lighting + ChildLighting.StationarySkyLighting;
								const float RelativeBrightness = Radiance.GetLuminance() / AverageBrightness;

								// Only search the axis neighbors past the first depth
								for (int32 NeighborIndex = 0; NeighborIndex < UE_ARRAY_COUNT(Neighbors) / 2; NeighborIndex++)
								{
									const float NeighborU = NodeContext.Min.X + (SubThetaIndex + Neighbors[NeighborIndex].X) * NodeContext.Size.X / 2;
									const float NeighborV = NodeContext.Min.Y + (SubPhiIndex + Neighbors[NeighborIndex].Y) * NodeContext.Size.Y / 2;

									// Query must be done on the center of the cell
									const FVector2f NeighborUV = FVector2f(NeighborU, NeighborV) + NodeContext.Size / 4;
									const FLightingAndOcclusion NeighborLighting = GetValue(NeighborUV);
									const float NeighborBrightness = (NeighborLighting.Lighting + NeighborLighting.StationarySkyLighting).GetLuminance();
									const float NeighborRelativeBrightness = NeighborBrightness / AverageBrightness;
									MaxRelativeDifference = FMath::Max(MaxRelativeDifference, FMath::Abs(RelativeBrightness - NeighborRelativeBrightness));
									MaxSkyOcclusionDifference = FMath::Max(MaxSkyOcclusionDifference, FMath::Abs(ChildLighting.UnoccludedSkyVector.SizeSquared() - NeighborLighting.UnoccludedSkyVector.SizeSquared()));
								}

								if (MaxRelativeDifference > BrightnessThreshold || (bRefineForSkyOcclusion && MaxSkyOcclusionDifference > SkyOcclusionThreshold))
								{
									RefinementCause = EFinalGatherRefinementCause::BrightnessDifference;
								}
							}

							if (RefinementCause == EFinalGatherRefinementCause::None)
							{
								float SkyVariance = LightingSystem.EvaluateSkyVariance(CellCenterWorldDirection, SubCellSolidAngle);

								if (SkyVariance > SkyVarianceThreshold)
								{
									RefinementCause = EFinalGatherRefinementCause::SkylightVariance;
								}
							}

							// Refine if the importance cone threshold is exceeded or there was a big enough brightness difference
							if (RefinementCause != EFinalGatherRefinementCause::None)
							{
								NextNodesToRefine->Add(FRefinementTraversalContext(
									ChildNode, 
									FVector2f(NodeContext.Min.X + SubThetaIndex * NodeContext.Size.X / 2, NodeContext.Min.Y + SubPhiIndex * NodeContext.Size.Y / 2),
									NodeContext.Size / 2.0f,
									SubCellSolidAngle,
									RefinementCause));
							}
						}
					}
				}
			}

			// Swap input and output for the next step
			Swap(CurrentNodesToRefine, NextNodesToRefine);

			FVector4f WorldPathDirection;
			FVector4f TangentPathDirection;
			FLightRay LightRay;
			FLightRayIntersection LightRayIntersection;

			FStaticLightingMappingStats& Stats = MappingContext.Stats;

			for (int32 NodeIndex = 0; NodeIndex < CurrentNodesToRefine->Num(); NodeIndex++)
			{
				FRefinementTraversalContext NodeContext = (*CurrentNodesToRefine)[NodeIndex];
				FLinearColor SubsampledRadiance = FLinearColor::Black;
				FLightingCacheGatherInfo SubsampleGatherInfo; 
				SubsampleGatherInfo.HitPointRecorder = HitPointRecorder;

				for (int32 SubThetaIndex = 0; SubThetaIndex < NumSubsamples; SubThetaIndex++)
				{
					for (int32 SubPhiIndex = 0; SubPhiIndex < NumSubsamples; SubPhiIndex++)
					{
						FSimpleQuadTreeNode<FRefinementElement>* FreeNode = NULL;
						
						if (MappingContext.RefinementTreeFreePool.Num() > 0)
						{
							FreeNode = MappingContext.RefinementTreeFreePool.Pop(EAllowShrinking::No);
							*FreeNode = FSimpleQuadTreeNode<FRefinementElement>();
						}
						else
						{
							FreeNode = new FSimpleQuadTreeNode<FRefinementElement>();
						}

						const FVector2f ChildMin = NodeContext.Min + FVector2f(SubThetaIndex, SubPhiIndex) * NodeContext.Size / 2;

						// Reuse the parent sample result in whatever child cell it falls in
						if (NodeContext.Node->Element.Uniforms.X >= ChildMin.X
							&& NodeContext.Node->Element.Uniforms.Y >= ChildMin.Y
							&& NodeContext.Node->Element.Uniforms.X < ChildMin.X + NodeContext.Size.X / 2
							&& NodeContext.Node->Element.Uniforms.Y < ChildMin.Y + NodeContext.Size.Y / 2)
						{
							//@todo - re-evaluate skylight which uses texture filtering dependent on SolidAngle, which has been refined
							FreeNode->Element = NodeContext.Node->Element;
							NodeContext.Node->Element.HitPointIndex = -1;
						}
						else
						{
							const float U1 = RandomStream.GetFraction();
							const float U2 = RandomStream.GetFraction();
							// Stratified sampling, pick a random position within the target cell
							const float SubStepFraction1 = (SubThetaIndex + U1) / (float)NumSubsamples;
							const float SubStepFraction2 = (SubPhiIndex + U2) / (float)NumSubsamples;
							const float Fraction1 = NodeContext.Min.X + SubStepFraction1 * NodeContext.Size.X;
							const float Fraction2 = NodeContext.Min.Y + SubStepFraction2 * NodeContext.Size.Y;

							const FVector4f SampleDirection = UniformSampleHemisphere(Fraction1, Fraction2);

							Vertex.ComputePathDirections(SampleDirection, WorldPathDirection, TangentPathDirection);

							LightingSystem.IntersectLightRay(
								Mapping,
								Vertex,
								SampleRadius,
								WorldPathDirection,
								TangentPathDirection,
								RayBiasMode,
								MappingContext,
								LightRay,
								LightRayIntersection);

							FVector3f UnoccludedSkyVector;
							FLinearColor StationarySkyLighting;
							FFinalGatherInfo SubsampleFinalGatherInfo;
							FFinalGatherHitPoint HitPoint;

							const FLinearColor SubsampleLighting = LightingSystem.FinalGatherSample(
								Mapping,
								Vertex,
								WorldPathDirection,
								TangentPathDirection,
								LightRay,
								LightRayIntersection,
								NodeContext.SolidAngle,
								BounceNumber,
								GatherClassification,
								bGatheringForCachedDirectLighting,
								bDebugThisTexel,
								MappingContext,
								RandomStream,
								SubsampleGatherInfo,
								SubsampleFinalGatherInfo,
								HitPoint,
								UnoccludedSkyVector,
								StationarySkyLighting);

							int32 StoredHitPointIndex = -1;

							if (SubsampleGatherInfo.HitPointRecorder && HitPoint.MappingSurfaceCoordinate >= 0)
							{
								StoredHitPointIndex = SubsampleGatherInfo.HitPointRecorder->GatherHitPointData.Num();
								SubsampleGatherInfo.HitPointRecorder->GatherHitPointRanges.Last().NumEntries++;
								SubsampleGatherInfo.HitPointRecorder->GatherHitPointData.Add(HitPoint);
							}

							FreeNode->Element = FRefinementElement(FLightingAndOcclusion(SubsampleLighting, UnoccludedSkyVector, StationarySkyLighting, SubsampleFinalGatherInfo.NumSamplesOccluded), FVector2f(Fraction1, Fraction2), StoredHitPointIndex);

							Stats.NumRefiningFinalGatherSamples[RefinementDepth]++;

							if (NodeContext.RefinementCause == EFinalGatherRefinementCause::BrightnessDifference)
							{
								Stats.NumRefiningSamplesDueToBrightness++;
							}
							else if (NodeContext.RefinementCause == EFinalGatherRefinementCause::ImportancePhotons)
							{
								Stats.NumRefiningSamplesDueToImportancePhotons++;
							}
							else
							{
								Stats.NumRefiningSamplesOther++;
							}
						}

						NodeContext.Node->AddChild(SubThetaIndex * NumSubsamples + SubPhiIndex, FreeNode);
					}
				}
			}

			// Tighten the refinement criteria for the next depth level
			// These have a huge impact on build time with a large depth limit
			//@todo - refine based on relative error instead of these heuristics
			ImportanceConeAngle /= 4;
			BrightnessThreshold *= 2;
			ConeWeightThreshold *= 1.5f;
			SkyOcclusionThreshold *= 16;
			SkyVarianceThreshold *= 2;
		}
	}

private:

	// Dimensions of the base 2d grid
	int32 NumThetaSteps;
	int32 NumPhiSteps;

	// 2d grid of quadtrees for refinement
	TArray<FSimpleQuadTree<FRefinementElement> > Cells;

	FLightingAndOcclusion GetFilteredValueRecursive(const FSimpleQuadTreeNode<FRefinementElement>* __restrict Parent) const
	{
		if (Parent->Children[0])
		{
			FLightingAndOcclusion FilteredValue;

			for (int32 ChildIndex = 0; ChildIndex < UE_ARRAY_COUNT(Parent->Children); ChildIndex++)
			{
				FilteredValue = FilteredValue + GetFilteredValueRecursive(Parent->Children[ChildIndex]) / 4.0f;
			}

			return FilteredValue;
		}
		else
		{
			return Parent->Element.Lighting;
		}
	}

	void UpdateHitPointWeightsRecursive(TArray<FFinalGatherHitPoint>& FinalGatherHitPoints, FSimpleQuadTreeNode<FRefinementElement>* Parent, float ParentWeight)
	{
		if (Parent->Children[0])
		{
			if (Parent->Element.HitPointIndex >= 0)
			{
				FinalGatherHitPoints[Parent->Element.HitPointIndex].Weight = 0.0f;
			}

			for (int32 ChildIndex = 0; ChildIndex < UE_ARRAY_COUNT(Parent->Children); ChildIndex++)
			{
				UpdateHitPointWeightsRecursive(FinalGatherHitPoints, Parent->Children[ChildIndex], ParentWeight / 4.0f);
			}
		}
		else
		{
			if (Parent->Element.HitPointIndex >= 0)
			{
				FinalGatherHitPoints[Parent->Element.HitPointIndex].Weight = ParentWeight;
			}
		}
	}
};

/** 
 * Final gather using adaptive sampling to estimate the incident radiance function. 
 * Adaptive refinement is done on brightness differences and anywhere that a first bounce photon determined lighting was coming from.
 */
template<class SampleType>
SampleType FStaticLightingSystem::IncomingRadianceAdaptive(
	const FStaticLightingMapping* Mapping,
	const FFullStaticLightingVertex& Vertex,
	float SampleRadius,
	bool bIntersectingSurface,
	int32 ElementIndex,
	int32 BounceNumber,
	EFinalGatherRayBiasMode RayBiasMode,
	EHemisphereGatherClassification GatherClassification,
	int32 NumAdaptiveRefinementLevels,
	float BrightnessThresholdScale,
	const TArray<FVector4f>& UniformHemisphereSamples,
	const TArray<FVector2f>& UniformHemisphereSampleUniforms,
	float MaxUnoccludedLength,
	const TArray<FVector4f>& ImportancePhotonDirections,
	FStaticLightingMappingContext& MappingContext,
	FLMRandomStream& RandomStream,
	FLightingCacheGatherInfo& GatherInfo,
	bool bGatheringForCachedDirectLighting,
	bool bDebugThisTexel) const
{
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisTexel)
	{
		int32 TempBreak = 0;
	}
#endif

	const double StartBaseTraceTime = FPlatformTime::Seconds();

	const int32 NumThetaSteps = FMath::TruncToInt(FMath::Sqrt(UniformHemisphereSamples.Num() / (float)PI) + .5f);
	const int32 NumPhiSteps = UniformHemisphereSamples.Num() / NumThetaSteps;
	checkSlow(NumThetaSteps * NumPhiSteps == UniformHemisphereSamples.Num());

	int32 NumBackfaceHits = 0;
	FUniformHemisphereRefinementGrid RefinementGrid(NumThetaSteps, NumPhiSteps);

	const float BaseGridSolidAngle = 2 * PI / (float)UniformHemisphereSamples.Num();

	FLightRay LightRay;
	FLightRayIntersection LightRayIntersection;

	// Initialize the root level of the refinement grid with lighting values
	for (int32 ThetaIndex = 0; ThetaIndex < NumThetaSteps; ThetaIndex++)
	{
		for (int32 PhiIndex = 0; PhiIndex < NumPhiSteps; PhiIndex++)
		{
			const int32 SampleIndex = ThetaIndex * NumPhiSteps + PhiIndex;
			const FVector4f TriangleTangentPathDirection = UniformHemisphereSamples[SampleIndex];
			
			FVector4f WorldPathDirection;
			FVector4f TangentPathDirection;
			Vertex.ComputePathDirections(TriangleTangentPathDirection, WorldPathDirection, TangentPathDirection);

			IntersectLightRay(
				Mapping,
				Vertex,
				SampleRadius,
				WorldPathDirection,
				TangentPathDirection,
				RayBiasMode,
				MappingContext,
				LightRay,
				LightRayIntersection);

			FVector3f UnoccludedSkyVector;
			FLinearColor StationarySkyLighting;
			FFinalGatherInfo FinalGatherInfo;
			FFinalGatherHitPoint HitPoint;

			const FLinearColor Radiance = FinalGatherSample(
				Mapping,
				Vertex,
				WorldPathDirection,
				TangentPathDirection,
				LightRay,
				LightRayIntersection,
				BaseGridSolidAngle,
				BounceNumber,
				GatherClassification,
				bGatheringForCachedDirectLighting,
				bDebugThisTexel,
				MappingContext,
				RandomStream,
				GatherInfo,
				FinalGatherInfo,
				HitPoint,
				UnoccludedSkyVector,
				StationarySkyLighting);

			int32 StoredHitPointIndex = -1;

			if (GatherInfo.HitPointRecorder && HitPoint.MappingSurfaceCoordinate >= 0)
			{
				StoredHitPointIndex = GatherInfo.HitPointRecorder->GatherHitPointData.Num();
				GatherInfo.HitPointRecorder->GatherHitPointRanges.Last().NumEntries++;
				GatherInfo.HitPointRecorder->GatherHitPointData.Add(HitPoint);
			}

			NumBackfaceHits += FinalGatherInfo.NumBackfaceHits;
			RefinementGrid.SetRootElement(ThetaIndex, PhiIndex, FRefinementElement(FLightingAndOcclusion(Radiance, UnoccludedSkyVector, StationarySkyLighting, FinalGatherInfo.NumSamplesOccluded), UniformHemisphereSampleUniforms[SampleIndex], StoredHitPointIndex));
		}
	}

	const double EndBaseTraceTime = FPlatformTime::Seconds();

	MappingContext.Stats.BaseFinalGatherSampleTime += EndBaseTraceTime - StartBaseTraceTime;
	MappingContext.Stats.NumBaseFinalGatherSamples += NumThetaSteps * NumPhiSteps;
	GatherInfo.BackfacingHitsFraction = NumBackfaceHits / (float)UniformHemisphereSamples.Num();

	// Refine if we are not hidden inside some geometry
	const bool bRefine = GatherInfo.BackfacingHitsFraction < .5f || bIntersectingSurface;

	if (bRefine)
	{
		TArray<FVector4f, TInlineAllocator<30> > TangentSpaceImportancePhotonDirections;
		TangentSpaceImportancePhotonDirections.Empty(ImportancePhotonDirections.Num());

		for (int32 PhotonIndex = 0; PhotonIndex < ImportancePhotonDirections.Num(); PhotonIndex++)
		{
			TangentSpaceImportancePhotonDirections.Add(Vertex.TransformWorldVectorToTriangleTangent(ImportancePhotonDirections[PhotonIndex]));
		}

		RefinementGrid.RefineIncomingRadiance(
			*this,
			Mapping,
			Vertex,
			SampleRadius,
			BounceNumber,
			RayBiasMode,
			GatherClassification,
			bGatheringForCachedDirectLighting,
			NumAdaptiveRefinementLevels,
			BrightnessThresholdScale,
			TangentSpaceImportancePhotonDirections,
			Scene.Portals,
			MappingContext,
			GatherInfo.HitPointRecorder,
			RandomStream,
			bDebugThisTexel);
	}

	const double EndRefiningTime = FPlatformTime::Seconds();

	MappingContext.Stats.RefiningFinalGatherSampleTime += EndRefiningTime - EndBaseTraceTime;

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisTexel)
	{
		int32 TempBreak = 0;
	}
#endif

	SampleType IncomingRadiance;
	FVector3f CombinedSkyUnoccludedDirection(0);
	float NumSamplesOccluded = 0;

	// Accumulate lighting from all samples
	for (int32 ThetaIndex = 0; ThetaIndex < NumThetaSteps; ThetaIndex++)
	{
		for (int32 PhiIndex = 0; PhiIndex < NumPhiSteps; PhiIndex++)
		{
			const int32 SampleIndex = ThetaIndex * NumPhiSteps + PhiIndex;

			const FVector4f TriangleTangentPathDirection = UniformHemisphereSamples[SampleIndex];
			checkSlow(TriangleTangentPathDirection.Z >= 0.0f);
			checkSlow(TriangleTangentPathDirection.IsUnit3());

			const FVector4f WorldPathDirection = Vertex.TransformTriangleTangentVectorToWorld(TriangleTangentPathDirection);
			checkSlow(WorldPathDirection.IsUnit3());

			const FVector4f TangentPathDirection = Vertex.TransformWorldVectorToTangent(WorldPathDirection);
			checkSlow(TangentPathDirection.IsUnit3());

			const float UniformPDF = 1.0f / (2.0f * (float)PI);
			const float SampleWeight = 1.0f / (UniformPDF * UniformHemisphereSamples.Num());

			if (GatherInfo.HitPointRecorder)
			{				
				RefinementGrid.UpdateHitPointWeights(GatherInfo.HitPointRecorder->GatherHitPointData, ThetaIndex, PhiIndex, SampleWeight * FMath::Max(TangentPathDirection.Z, 0.0f));
			}
			
			const FLightingAndOcclusion FilteredLighting = RefinementGrid.GetFilteredValue(ThetaIndex, PhiIndex);
			// Get the filtered lighting from the leaves of the refinement trees
			const FLinearColor Radiance = FilteredLighting.Lighting;
			CombinedSkyUnoccludedDirection += FilteredLighting.UnoccludedSkyVector;

			IncomingRadiance.AddIncomingRadiance(Radiance, SampleWeight, TangentPathDirection, WorldPathDirection);
			checkSlow(IncomingRadiance.AreFloatsValid());
			NumSamplesOccluded += FilteredLighting.NumSamplesOccluded;
		}
	}

	// Calculate the fraction of samples which were occluded
	const float MaterialElementFullyOccludedSamplesFraction = Mapping ? Mapping->Mesh->GetFullyOccludedSamplesFraction(ElementIndex) : 1.0f;
	const float OcclusionFraction = FMath::Min(NumSamplesOccluded / (AmbientOcclusionSettings.FullyOccludedSamplesFraction * MaterialElementFullyOccludedSamplesFraction * UniformHemisphereSamples.Num()), 1.0f);
	// Constant which maintains an integral of .5 for the unclamped exponential function applied to occlusion below
	// An integral of .5 is important because it makes an image with a uniform distribution of occlusion values stay the same brightness with different exponents.
	// As a result, OcclusionExponent just controls contrast and doesn't affect brightness.
	const float NormalizationConstant = .5f * (AmbientOcclusionSettings.OcclusionExponent + 1);
	IncomingRadiance.SetOcclusion(FMath::Clamp(NormalizationConstant * FMath::Pow(OcclusionFraction, AmbientOcclusionSettings.OcclusionExponent), 0.0f, 1.0f)); 

	const FVector3f BentNormal = CombinedSkyUnoccludedDirection / (MaxUnoccludedLength * UniformHemisphereSamples.Num());
	IncomingRadiance.SetSkyOcclusion(BentNormal);

	RefinementGrid.ReturnToFreeList(MappingContext.RefinementTreeFreePool);

	return IncomingRadiance;
}

/** Calculates irradiance gradients for a sample position that will be cached. */
void FStaticLightingSystem::CalculateIrradianceGradients(
	int32 BounceNumber,
	const FLightingCacheGatherInfo& GatherInfo,
	FVector4f& RotationalGradient,
	FVector4f& TranslationalGradient) const
{
	// Calculate rotational and translational gradients as described in the paper "Irradiance Gradients" by Greg Ward and Paul Heckbert
	FVector4f AccumulatedRotationalGradient(0,0,0);
	FVector4f AccumulatedTranslationalGradient(0,0,0);
	if (IrradianceCachingSettings.bUseIrradianceGradients)
	{
		// Extract Theta and Phi steps from the number of hemisphere samples requested
		const float NumThetaStepsFloat = FMath::Sqrt(GetNumUniformHemisphereSamples(BounceNumber) / (float)PI);
		const int32 NumThetaSteps = FMath::TruncToInt(NumThetaStepsFloat);
		// Using PI times more Phi steps as Theta steps
		const int32 NumPhiSteps = FMath::TruncToInt(NumThetaStepsFloat * (float)PI);
		checkSlow(NumThetaSteps > 0 && NumPhiSteps > 0);

		// Calculate the rotational gradient
		for (int32 PhiIndex = 0; PhiIndex < NumPhiSteps; PhiIndex++)
		{
			FVector4f InnerSum(0,0,0);
			for (int32 ThetaIndex = 0; ThetaIndex < NumThetaSteps; ThetaIndex++)
			{
				const int32 SampleIndex = ThetaIndex * NumPhiSteps + PhiIndex;
				const FLinearColor& IncidentRadiance = GatherInfo.PreviousIncidentRadiances[SampleIndex];
				// Note: These equations need to be re-derived from the paper for a non-uniform PDF
				const float TangentTerm = -FMath::Tan(ThetaIndex / (float)NumThetaSteps);
				InnerSum += TangentTerm * FVector4f(IncidentRadiance);
			}
			const float CurrentPhi = 2.0f * (float)PI * PhiIndex / (float)NumPhiSteps;
			// Vector in the tangent plane perpendicular to the current Phi
			const FVector4f BasePlaneVector = FVector2f((float)HALF_PI, FMath::Fmod(CurrentPhi + (float)HALF_PI, 2.0f * (float)PI)).SphericalToUnitCartesian();
			AccumulatedRotationalGradient += InnerSum * BasePlaneVector;
		}
		// Normalize the sum
		AccumulatedRotationalGradient *= (float)PI / (NumThetaSteps * NumPhiSteps);

		// Calculate the translational gradient
		for (int32 PhiIndex = 0; PhiIndex < NumPhiSteps; PhiIndex++)
		{
			FVector4f PolarWallContribution(0,0,0);
			// Starting from 1 since Theta doesn't wrap around (unlike Phi)
			for (int32 ThetaIndex = 1; ThetaIndex < NumThetaSteps; ThetaIndex++)
			{
				const float CurrentTheta = ThetaIndex / (float)NumThetaSteps;
				const float CosCurrentTheta = FMath::Cos(CurrentTheta);
				const int32 SampleIndex = ThetaIndex * NumPhiSteps + PhiIndex;
				const int32 PreviousThetaSampleIndex = (ThetaIndex - 1) * NumPhiSteps + PhiIndex;
				const float& PreviousThetaDistance = GatherInfo.PreviousDistances[PreviousThetaSampleIndex];
				const float& CurrentThetaDistance = GatherInfo.PreviousDistances[SampleIndex];
				const float MinDistance = FMath::Min(PreviousThetaDistance, CurrentThetaDistance);
				checkSlow(MinDistance > 0);
				const FLinearColor IncomingRadianceDifference = GatherInfo.PreviousIncidentRadiances[SampleIndex] - GatherInfo.PreviousIncidentRadiances[PreviousThetaSampleIndex];
				PolarWallContribution += FMath::Sin(CurrentTheta) * CosCurrentTheta * CosCurrentTheta / MinDistance * FVector4f(IncomingRadianceDifference);
				checkSlow(!PolarWallContribution.ContainsNaN());
			}

			// Wrap Phi around for the first Phi index
			const int32 PreviousPhiIndex = PhiIndex == 0 ? NumPhiSteps - 1 : PhiIndex - 1;
			FVector4f RadialWallContribution(0,0,0);
			for (int32 ThetaIndex = 0; ThetaIndex < NumThetaSteps; ThetaIndex++)
			{
				const float CurrentTheta = FMath::Acos(ThetaIndex / (float)NumThetaSteps);
				const float NextTheta = FMath::Acos((ThetaIndex + 1) / (float)NumThetaSteps);
				const int32 SampleIndex = ThetaIndex * NumPhiSteps + PhiIndex;
				const int32 PreviousPhiSampleIndex = ThetaIndex * NumPhiSteps + PreviousPhiIndex;
				const float& PreviousPhiDistance = GatherInfo.PreviousDistances[PreviousPhiSampleIndex];
				const float& CurrentPhiDistance = GatherInfo.PreviousDistances[SampleIndex];
				const float MinDistance = FMath::Min(PreviousPhiDistance, CurrentPhiDistance);
				checkSlow(MinDistance > 0);
				const FLinearColor IncomingRadianceDifference = GatherInfo.PreviousIncidentRadiances[SampleIndex] - GatherInfo.PreviousIncidentRadiances[PreviousPhiSampleIndex];
				RadialWallContribution += (FMath::Sin(NextTheta) - FMath::Sin(CurrentTheta)) / MinDistance * FVector4f(IncomingRadianceDifference);
				checkSlow(!RadialWallContribution.ContainsNaN());
			}

			const float CurrentPhi = 2.0f * (float)PI * PhiIndex / (float)NumPhiSteps;
			// Vector in the tangent plane in the direction of the current Phi
			const FVector4f PhiDirection = SphericalToUnitCartesian(FVector2f((float)HALF_PI, CurrentPhi));
			// Vector in the tangent plane perpendicular to the current Phi
			const FVector4f PerpendicularPhiDirection = FVector2f((float)HALF_PI, FMath::Fmod(CurrentPhi + (float)HALF_PI, 2.0f * (float)PI)).SphericalToUnitCartesian();

			PolarWallContribution = PhiDirection * 2.0f * (float)PI / (float)NumPhiSteps * PolarWallContribution;
			RadialWallContribution = PerpendicularPhiDirection * RadialWallContribution;
			AccumulatedTranslationalGradient += PolarWallContribution + RadialWallContribution;
		}
	}
	RotationalGradient = AccumulatedRotationalGradient;
	TranslationalGradient = AccumulatedTranslationalGradient;
}

/** 
 * Interpolates incoming radiance from the lighting cache if possible,
 * otherwise estimates incoming radiance for this sample point and adds it to the cache. 
 */
FFinalGatherSample FStaticLightingSystem::CachePointIncomingRadiance(
	const FStaticLightingMapping* Mapping,
	const FFullStaticLightingVertex& Vertex,
	int32 ElementIndex,
	float TexelRadius,
	float SampleRadius,
	bool bIntersectingSurface,
	FStaticLightingMappingContext& MappingContext,
	FLMRandomStream& RandomStream,
	bool bDebugThisTexel
	) const
{
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisTexel)
	{
		int32 TempBreak = 0;
	}
#endif

	const int32 BounceNumber = 1;
	FFinalGatherSample IndirectLighting;
	FFinalGatherSample UnusedSecondLighting;
	float UnusedBackfacingHitsFraction;
	// Attempt to interpolate incoming radiance from the lighting cache
	if (!IrradianceCachingSettings.bAllowIrradianceCaching || !MappingContext.FirstBounceCache.InterpolateLighting(Vertex, true, bDebugThisTexel, 1.0f , IndirectLighting, UnusedSecondLighting, UnusedBackfacingHitsFraction, MappingContext.DebugCacheRecords))
	{
		// If final gathering is disabled, all indirect lighting will be estimated using photon mapping.
		// This is really only useful for debugging since it requires an excessive number of indirect photons to get indirect shadows for the first bounce.
		if (PhotonMappingSettings.bUsePhotonMapping 
			&& GeneralSettings.NumIndirectLightingBounces > 0
			&& !PhotonMappingSettings.bUseFinalGathering)
		{
			// Use irradiance photons for indirect lighting
			if (PhotonMappingSettings.bUseIrradiancePhotons)
			{
				FLinearColor Irradiance = FLinearColor::Black;

				if (PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces)
				{
					// Trace a ray into the texel to get a good representation of what the final gather will see,
					// Instead of just calculating lightmap UV's from the current texel's position.
					// Speed does not matter here since !bUseFinalGathering is only used for debugging.
					const FLightRay TexelRay(
						Vertex.WorldPosition + Vertex.WorldTangentZ * SampleRadius,
						Vertex.WorldPosition - Vertex.WorldTangentZ * SampleRadius,
						Mapping,
						NULL
						);

					FLightRayIntersection Intersection;
					AggregateMesh->IntersectLightRay(TexelRay, true, false, false, MappingContext.RayCache, Intersection);
					FStaticLightingVertex CurrentVertex = Vertex;
					// Use the intersection's UV's if found, otherwise use the passed in UV's
					if (Intersection.bIntersects && Mapping == Intersection.Mapping)
					{
						CurrentVertex = Intersection.IntersectionVertex;
					}

					Irradiance = Mapping->GetSurfaceCacheLighting(CurrentVertex);

					const bool bTranslucent = Mapping->Mesh->IsTranslucent(ElementIndex);
					const FLinearColor Reflectance = (bTranslucent ? FLinearColor::Black : Mapping->Mesh->EvaluateTotalReflectance(CurrentVertex, ElementIndex)) * (float)INV_PI;

					// Undo reflectance scale applied to surface cache lighting
					Irradiance /= Reflectance;
				}
				else
				{
					const FIrradiancePhoton* NearestPhoton = NULL;

					TArray<FIrradiancePhoton*> TempIrradiancePhotons;
					// Search the irradiance photon map for the nearest one
					NearestPhoton = FindNearestIrradiancePhoton(Vertex, MappingContext, TempIrradiancePhotons, false, bDebugThisTexel);
					Irradiance = NearestPhoton ? NearestPhoton->GetIrradiance() : FLinearColor::Black;
				}
				
				// Convert irradiance (which is incident radiance over all directions for a point) to incident radiance with the approximation 
				// That the irradiance is actually incident radiance along the surface normal.  This will only be correct for simple lightmaps.
				IndirectLighting.AddWeighted(FGatheredLightSampleUtil::AmbientLight<2>(Irradiance), 1.0f);
			}
			else
			{
				// Use the photons deposited on surfaces to estimate indirect lighting
				const bool bDebugFirstBouncePhotonGather = bDebugThisTexel && GeneralSettings.ViewSingleBounceNumber == BounceNumber;
				const FGatheredLightSample FirstBounceLighting = CalculatePhotonIncidentRadiance(FirstBouncePhotonMap, NumPhotonsEmittedFirstBounce, PhotonMappingSettings.IndirectPhotonSearchDistance, Vertex, bDebugFirstBouncePhotonGather);
				if (GeneralSettings.ViewSingleBounceNumber < 0 || GeneralSettings.ViewSingleBounceNumber == BounceNumber)
				{
					IndirectLighting.AddWeighted(FirstBounceLighting, 1.0f);
				}
				
				if (GeneralSettings.NumIndirectLightingBounces > 1)
				{
					const bool bDebugSecondBouncePhotonGather = bDebugThisTexel && GeneralSettings.ViewSingleBounceNumber > BounceNumber;
					const FGatheredLightSample SecondBounceLighting = CalculatePhotonIncidentRadiance(SecondBouncePhotonMap, NumPhotonsEmittedSecondBounce, PhotonMappingSettings.IndirectPhotonSearchDistance, Vertex, bDebugSecondBouncePhotonGather);
					if (GeneralSettings.ViewSingleBounceNumber < 0 || GeneralSettings.ViewSingleBounceNumber > BounceNumber)
					{
						IndirectLighting.AddWeighted(SecondBounceLighting, 1.0f);
					}
				}
			}
		}
		else if (DynamicObjectSettings.bVisualizeVolumeLightInterpolation
			&& GeneralSettings.NumIndirectLightingBounces > 0)
		{
			const FGatheredLightSample VolumeLighting = InterpolatePrecomputedVolumeIncidentRadiance(Vertex, SampleRadius, MappingContext.RayCache, bDebugThisTexel);
			IndirectLighting.AddWeighted(VolumeLighting, 1.0f);
		}
		else
		{
			// Using final gathering with photon mapping, hemisphere gathering without photon mapping, path tracing and/or just calculating ambient occlusion
			TArray<FVector4f> ImportancePhotonDirections;

			if (GeneralSettings.NumIndirectLightingBounces > 0)
			{
				if (PhotonMappingSettings.bUsePhotonMapping)
				{
					LIGHTINGSTAT(FScopedRDTSCTimer PhotonGatherTimer(MappingContext.Stats.ImportancePhotonGatherTime));
					TArray<FPhoton> FoundPhotons;
					// Gather nearby first bounce photons, which give an estimate of the first bounce incident radiance function,
					// Which we can use to importance sample the real first bounce incident radiance function.
					// See the "Extended Photon Map Implementation" paper.
					FFindNearbyPhotonStats DummyStats;
					FindNearbyPhotonsIterative(
						FirstBouncePhotonMap, 
						Vertex.WorldPosition, 
						Vertex.TriangleNormal, 
						PhotonMappingSettings.NumImportanceSearchPhotons, 
						PhotonMappingSettings.MinImportancePhotonSearchDistance, 
						PhotonMappingSettings.MaxImportancePhotonSearchDistance, 
						bDebugThisTexel, 
						false,
						FoundPhotons,
						DummyStats);

					MappingContext.Stats.TotalFoundImportancePhotons += FoundPhotons.Num();

					ImportancePhotonDirections.Empty(FoundPhotons.Num());
					for (int32 PhotonIndex = 0; PhotonIndex < FoundPhotons.Num(); PhotonIndex++)
					{
						const FPhoton& CurrentPhoton = FoundPhotons[PhotonIndex];
						// Calculate the direction from the current position to the photon's source
						// Using the photon's incident direction unmodified produces artifacts proportional to the distance to that photon
						const FVector4f NewDirection = CurrentPhoton.GetPosition() + CurrentPhoton.GetIncidentDirection() * CurrentPhoton.GetDistance() - Vertex.WorldPosition;
						// Only use the direction if it is in the hemisphere of the normal
						// FindNearbyPhotons only returns photons whose incident directions lie in this hemisphere, but the recalculated direction might not.
						if (Dot3(NewDirection, Vertex.TriangleNormal) > 0.0f)
						{
							ImportancePhotonDirections.Add(NewDirection.GetUnsafeNormal3());
						}
					}
				}
			}
			
			FLightingCacheGatherInfo GatherInfo;
			FFinalGatherSample UniformSampledIncomingRadiance = IncomingRadianceAdaptive<FFinalGatherSample>(
					Mapping, 
					Vertex, 
					SampleRadius, 
					bIntersectingSurface,
					ElementIndex, 
					BounceNumber, 
					RBM_ConstantNormalOffset,
					GLM_FinalGather,
					ImportanceTracingSettings.NumAdaptiveRefinementLevels,
					1.0f,
					CachedHemisphereSamples,
					CachedHemisphereSampleUniforms,
					CachedSamplesMaxUnoccludedLength,
					ImportancePhotonDirections, 
					MappingContext, 
					RandomStream, 
					GatherInfo,
					false,
					bDebugThisTexel);

			IndirectLighting.AddWeighted(UniformSampledIncomingRadiance, 1.0f);

			const bool bInsideGeometry = GatherInfo.BackfacingHitsFraction > .5f && !bIntersectingSurface;

			if (IrradianceCachingSettings.bAllowIrradianceCaching)
			{
				FVector4f RotationalGradient;
				FVector4f TranslationalGradient;
				CalculateIrradianceGradients(BounceNumber, GatherInfo, RotationalGradient, TranslationalGradient);

				float OverrideRadius = 0;

				if (GeneralSettings.bAccountForTexelSize)
				{
					// Make the irradiance cache sample radius very small for texels whose radius is close to the minimum, 
					// Since those texels are usually in corners and not representative of their neighbors.
					if (SampleRadius < SceneConstants.SmallestTexelRadius * 2.0f)
					{
						OverrideRadius = SceneConstants.SmallestTexelRadius;
					}
					else if (GatherInfo.MinDistance > SampleRadius)
					{
						// When uniform final gather rays are offset from the center of the texel, 
						// It's possible for a perpendicular surface to intersect the center of the texel and none of the final gather rays detect it.
						// The lighting cache sample will be assigned a large radius and the artifact will be interpolated a large distance.
						// Trace a ray from one corner of the texel to the other to detect this edge case, 
						// And set the record radius to the minimum to contain the error.
						// Center of the texel offset along the normal
						const FVector4f TexelCenterOffset = Vertex.WorldPosition + Vertex.TriangleNormal * SampleRadius * SceneConstants.VisibilityNormalOffsetSampleRadiusScale;
						// Vector from the center to one of the corners of the texel
						// The FMath::Sqrt(.5f) is to normalize (Vertex.TriangleTangentX + Vertex.TriangleTangentY), which are orthogonal unit vectors.
						const FVector4f CornerOffset = FMath::Sqrt(.5f) * (Vertex.TriangleTangentX + Vertex.TriangleTangentY) * SampleRadius * SceneConstants.VisibilityTangentOffsetSampleRadiusScale;
						const FLightRay TexelRay(
							TexelCenterOffset + CornerOffset,
							TexelCenterOffset - CornerOffset,
							NULL,
							NULL
							);

						FLightRayIntersection Intersection;
						AggregateMesh->IntersectLightRay(TexelRay, false, false, false, MappingContext.RayCache, Intersection);
						if (Intersection.bIntersects)
						{
							OverrideRadius = SampleRadius;
						}
	#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
						if (bDebugThisTexel 
							&& GeneralSettings.ViewSingleBounceNumber == BounceNumber
							&& (!PhotonMappingSettings.bUsePhotonMapping || !PhotonMappingSettings.bVisualizePhotonImportanceSamples))
						{
							FDebugStaticLightingRay DebugRay(TexelRay.Start, TexelRay.End, Intersection.bIntersects, false);
							if (Intersection.bIntersects)
							{
								DebugRay.End = Intersection.IntersectionVertex.WorldPosition;
							}

							// CachePointIncomingRadiance can be called from multiple threads
							FScopeLock DebugOutputLock(&DebugOutputSync);
							MappingContext.DebugOutput->PathRays.Add(DebugRay);
						}
	#endif
					}
				}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisTexel)
				{
					int32 TempBreak = 0;
				}
#endif

				TLightingCache<FFinalGatherSample>::FRecord<FFinalGatherSample> NewRecord(
					Vertex,
					ElementIndex,
					GatherInfo,
					BounceNumber == 1 ? TexelRadius : SampleRadius,
					OverrideRadius,
					IrradianceCachingSettings,
					GeneralSettings,
					IndirectLighting,
					RotationalGradient,
					TranslationalGradient
					);

				// Add the incident radiance sample to the first bounce lighting cache.
				MappingContext.FirstBounceCache.AddRecord(NewRecord, bInsideGeometry, true);

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (IrradianceCachingSettings.bVisualizeIrradianceSamples && Mapping == Scene.DebugMapping && GeneralSettings.ViewSingleBounceNumber == BounceNumber)
				{
					const float DistanceToDebugTexelSq = FVector3f(Scene.DebugInput.Position - Vertex.WorldPosition).SizeSquared();
					FDebugLightingCacheRecord TempRecord;
					TempRecord.bNearSelectedTexel = DistanceToDebugTexelSq < NewRecord.BoundingRadius * NewRecord.BoundingRadius;
					TempRecord.Radius = NewRecord.Radius;
					TempRecord.Vertex.VertexPosition = Vertex.WorldPosition;
					TempRecord.Vertex.VertexNormal = Vertex.WorldTangentZ;
					TempRecord.RecordId = NewRecord.Id;

					MappingContext.DebugCacheRecords.Add(TempRecord);
				}
#endif
			}
		}
	}

	return IndirectLighting;
}

} //namespace Lightmass

