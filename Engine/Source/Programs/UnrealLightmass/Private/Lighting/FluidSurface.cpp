// Copyright Epic Games, Inc. All Rights Reserved.

#include "FluidSurface.h"
#include "Importer.h"
#include "LightingMesh.h"

namespace Lightmass
{

	/** Represents the fluid surface mesh to the static lighting system. */
	static void GetStaticLightingVertex(
		const FVector4f* QuadCorners,
		const FVector4f* QuadUVCorners,
		uint32 VertexIndex,
		const FMatrix44f& LocalToWorld,
		const FMatrix44f& LocalToWorldInverseTranspose,
		FStaticLightingVertex& OutVertex
		)
	{
		OutVertex.WorldPosition = LocalToWorld.TransformPosition(QuadCorners[VertexIndex]);
		OutVertex.WorldTangentX = LocalToWorld.TransformVector(FVector4f(1, 0, 0, 1)).GetSafeNormal();
		OutVertex.WorldTangentY = LocalToWorld.TransformVector(FVector4f(0, 1, 0, 1)).GetSafeNormal();
		OutVertex.WorldTangentZ = LocalToWorldInverseTranspose.TransformVector(FVector4f(0, 0, 1, 1)).GetSafeNormal();

		for(uint32 UVIndex = 0; UVIndex < 1; UVIndex++)
		{
			OutVertex.TextureCoordinates[UVIndex] = FVector2f(QuadUVCorners[VertexIndex].X, QuadUVCorners[VertexIndex].Y);
		}
	}

	// FStaticLightingMesh interface.
	void FFluidSurfaceStaticLightingMesh::GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const
	{
		GetStaticLightingVertex(QuadCorners,QuadUVCorners,QuadIndices[TriangleIndex * 3 + 0],LocalToWorld,LocalToWorldInverseTranspose,OutV0);
		GetStaticLightingVertex(QuadCorners,QuadUVCorners,QuadIndices[TriangleIndex * 3 + 1],LocalToWorld,LocalToWorldInverseTranspose,OutV1);
		GetStaticLightingVertex(QuadCorners,QuadUVCorners,QuadIndices[TriangleIndex * 3 + 2],LocalToWorld,LocalToWorldInverseTranspose,OutV2);
		ElementIndex = 0;
	}

	void FFluidSurfaceStaticLightingMesh::GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const
	{
		OutI0 = QuadIndices[TriangleIndex * 3 + 0];
		OutI1 = QuadIndices[TriangleIndex * 3 + 1];
		OutI2 = QuadIndices[TriangleIndex * 3 + 2];
	}

	void FFluidSurfaceStaticLightingMesh::Import( class FLightmassImporter& Importer )
	{
		// import super class
		FStaticLightingMesh::Import( Importer );
		Importer.ImportData((FFluidSurfaceStaticLightingMeshData*) this);
		check(MaterialElements.Num() > 0);
	}

	void FFluidSurfaceStaticLightingTextureMapping::Import( class FLightmassImporter& Importer )
	{
		FStaticLightingTextureMapping::Import(Importer);

		// Can't use the FStaticLightingMapping Import functionality for this
		// as it only looks in the StaticMeshInstances map...
		Mesh = Importer.GetFluidMeshInstances().FindRef(Guid);
		check(Mesh);
	}

} //namespace Lightmass
