// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LightingMesh.h"
#include "Mappings.h"

namespace Lightmass
{

	/** Represents the triangles of a fluid surface primitive to the static lighting system. */
	class FFluidSurfaceStaticLightingMesh : public FStaticLightingMesh, public FFluidSurfaceStaticLightingMeshData
	{
	public:
		// FStaticLightingMesh interface.
		virtual void GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const;
		virtual void GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const;

		virtual void Import( class FLightmassImporter& Importer );
	};

	/** Represents a fluid surface primitive with texture mapped static lighting. */
	class FFluidSurfaceStaticLightingTextureMapping : public FStaticLightingTextureMapping
	{
	public:
		virtual void Import( class FLightmassImporter& Importer );
	};

} //namespace Lightmass
