// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ImportExport.h"
#include "Math/SHMath.h"

namespace Lightmass
{

/** 
 * The light incident for a point on a surface, in the representation used when gathering lighting. 
 * This representation is additive, and allows for accumulating lighting contributions in-place. 
 */
template <int32 SHOrder>
class TGatheredLightSample
{
public:	

	/** World space incident lighting. */
	TSHVectorRGB<SHOrder> SHVector;

	/** Incident lighting including dot(N, L) where N is the smoothed vertex normal. */
	FLinearColor IncidentLighting;

	/** Correction factor to force SH as applied to a flat normal map to be 1 to get purely directional data. */
	float SHCorrection;

	/** Sky bent normal, points toward the most unoccluded direction, and the length is the visibility amount (0 = occluded, 1 = visible). */
	FVector3f SkyOcclusion;

	float AOMaterialMask;

	/** Initialization constructor. */
	TGatheredLightSample()
	{
		SHCorrection = 0.0f;
		IncidentLighting = FLinearColor(0, 0, 0, 0);
		SkyOcclusion = FVector3f(0);
		AOMaterialMask = 0;
	}

	TGatheredLightSample(EForceInit)
	{
		SHCorrection = 0.0f;
		IncidentLighting = FLinearColor(0, 0, 0, 0);
		SkyOcclusion = FVector3f(0);
		AOMaterialMask = 0;
	}	

	/**
	 * Adds a weighted light sample to this light sample.
	 * @param OtherSample - The sample to add.
	 * @param Weight - The weight to multiply the other sample by before addition.
	 */
	void AddWeighted(const TGatheredLightSample& OtherSample, float Weight);

	void ApplyOcclusion(float Occlusion);

	inline void SetSkyOcclusion(FVector3f InSkyOcclusion)
	{
		SkyOcclusion = InSkyOcclusion;
	}

	bool AreFloatsValid() const;

	TGatheredLightSample operator*(float Scalar) const
	{
		TGatheredLightSample Result;
		Result.SHVector = SHVector * Scalar;
		Result.SHCorrection = SHCorrection * Scalar;
		Result.IncidentLighting = IncidentLighting * Scalar;
		Result.SkyOcclusion = SkyOcclusion * Scalar;
		Result.AOMaterialMask = AOMaterialMask * Scalar;
		return Result;
	}

	TGatheredLightSample operator+(const TGatheredLightSample& SampleB) const
	{
		TGatheredLightSample Result;
		Result.SHVector = SHVector + SampleB.SHVector;
		Result.SHCorrection = SHCorrection + SampleB.SHCorrection;
		Result.IncidentLighting = IncidentLighting + SampleB.IncidentLighting;
		Result.SkyOcclusion = SkyOcclusion + SampleB.SkyOcclusion;
		Result.AOMaterialMask = AOMaterialMask + SampleB.AOMaterialMask;
		return Result;
	}
};

class FGatheredLightSampleUtil
{
public:

	/**
	* Constructs a light sample representing an ambient light.
	* Note: Lighting contributed through this method won't have the same final brightness as PointLightWorldSpace, because of the dot(N, L)
	* @param Color - The color/intensity of the light at the sample point.
	*/
	template <int32 SHOrder>
	static TGatheredLightSample<SHOrder> AmbientLight(const FLinearColor& Color);

	/**
	* Constructs a light sample representing a point light.
	* @param Color - The color/intensity of the light at the sample point.
	* @param Direction - The direction toward the light at the sample point.
	*/
	template <int32 SHOrder>
	static TGatheredLightSample<SHOrder> PointLightWorldSpace(const FLinearColor& Color, const FVector4f& TangentDirection, const FVector4f& WorldDirection);
};

typedef TGatheredLightSample<2> FGatheredLightSample;
typedef TGatheredLightSample<3> FGatheredLightSample3;

class FGatheredLightMapSample
{
public:
	FGatheredLightSample	HighQuality;
	FGatheredLightSample	LowQuality;
	
	/** True if this sample maps to a valid point on a triangle.  This is only meaningful for texture lightmaps. */
	bool					bIsMapped;

	FGatheredLightMapSample()
	: bIsMapped(false)
	{}

	FGatheredLightMapSample(const FGatheredLightSample& Sample)
	: HighQuality(Sample)
	, LowQuality(Sample)
	, bIsMapped(false)
	{}

	FGatheredLightMapSample& operator=(const FGatheredLightSample& Sample)
	{
		HighQuality = Sample;
		LowQuality = Sample;
		return *this;
	}

	/**
	 * Adds a weighted light sample to this light sample.
	 * @param OtherSample - The sample to add.
	 * @param Weight - The weight to multiply the other sample by before addition.
	 */
	void AddWeighted(const FGatheredLightSample& OtherSample, float Weight)
	{
		HighQuality.AddWeighted( OtherSample, Weight );
		LowQuality.AddWeighted( OtherSample, Weight );
	}

	void ApplyOcclusion(float Occlusion)
	{
		HighQuality.ApplyOcclusion(Occlusion);
		LowQuality.ApplyOcclusion(Occlusion);
	}

	/** Converts an FGatheredLightMapSample into a FLightSample. */
	FLightSample ConvertToLightSample(bool bDebugThisSample) const;
};

/** The lighting information gathered for one final gather sample */
template <int32 SHOrder>
class TFinalGatherSample : public TGatheredLightSample<SHOrder>
{
public:

	/** Occlusion factor of the sample, 0 is completely unoccluded, 1 is completely occluded. */
	float Occlusion;

	/** 
	 * A light sample for sky lighting.  This has to be stored separately to support stationary sky lights only contributing to low quality lightmaps.
	 */
	TGatheredLightSample<SHOrder> StationarySkyLighting;

	/** Initialization constructor. */
	TFinalGatherSample() :
		TGatheredLightSample<SHOrder>(),
		Occlusion(0.0f)
	{}

	TFinalGatherSample(EForceInit) :
		TGatheredLightSample<SHOrder>(ForceInit),
		Occlusion(0.0f)
	{}

	/**
	 * Adds a weighted light sample to this light sample.
	 * @param OtherSample - The sample to add.
	 * @param Weight - The weight to multiply the other sample by before addition.
	 */
	void AddWeighted(const TFinalGatherSample& OtherSample, float Weight);

	/**
	 * Adds a weighted light sample to this light sample.
	 * @param OtherSample - The sample to add.
	 * @param Weight - The weight to multiply the other sample by before addition.
	 */
	template<int32 OtherOrder>
	inline void AddWeighted(const TGatheredLightSample<OtherOrder>& OtherSample, float Weight)
	{
		TGatheredLightSample<SHOrder>::AddWeighted(OtherSample, Weight);
	}

	inline void SetOcclusion(float InOcclusion)
	{
		Occlusion = InOcclusion;
	}

	inline void AddIncomingRadiance(const FLinearColor& IncomingRadiance, float Weight, const FVector4f& TangentSpaceDirection, const FVector4f& WorldSpaceDirection)
	{
		AddWeighted(FGatheredLightSampleUtil::PointLightWorldSpace<SHOrder>(IncomingRadiance, TangentSpaceDirection, WorldSpaceDirection), Weight);
	}

	bool AreFloatsValid() const;

	TFinalGatherSample operator*(float Scalar) const
	{
		TFinalGatherSample Result;
		(TGatheredLightSample<SHOrder>&)Result = (const TGatheredLightSample<SHOrder>&)(*this) * Scalar;
		Result.Occlusion = Occlusion * Scalar;
		return Result;
	}

	TFinalGatherSample operator+(const TFinalGatherSample& SampleB) const
	{
		TFinalGatherSample Result;
		(TGatheredLightSample<SHOrder>&)Result = (const TGatheredLightSample<SHOrder>&)(*this) + (const TGatheredLightSample<SHOrder>&)SampleB;
		Result.Occlusion = Occlusion + SampleB.Occlusion;
		return Result;
	}
};

typedef TFinalGatherSample<2> FFinalGatherSample;
typedef TFinalGatherSample<3> FFinalGatherSample3;

} //namespace Lightmass

