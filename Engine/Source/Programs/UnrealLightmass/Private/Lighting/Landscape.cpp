// Copyright Epic Games, Inc. All Rights Reserved.

#include "Landscape.h"
#include "Importer.h"

namespace Lightmass
{
	// Accessors from FLandscapeDataInterface
	void FLandscapeStaticLightingMesh::VertexIndexToXY(int32 VertexIndex, int32& OutX, int32& OutY) const
	{
		OutX = VertexIndex % NumVertices;
		OutY = VertexIndex / NumVertices;
	}

	void FLandscapeStaticLightingMesh::QuadIndexToXY(int32 QuadIndex, int32& OutX, int32& OutY) const
	{
		OutX = QuadIndex % NumQuads;
		OutY = QuadIndex / NumQuads;
	}

	const FColor* FLandscapeStaticLightingMesh::GetHeightData( int32 LocalX, int32 LocalY ) const
	{
		return &HeightMap[LocalX + LocalY * NumVertices ];
	}

	void FLandscapeStaticLightingMesh::GetTriangleIndices(int32 QuadIndex,int32 TriNum,int32& OutI0,int32& OutI1,int32& OutI2) const
	{
		//OutI0 = 0; OutI1 = 1; OutI2 = 2;
		int32 QuadX, QuadY;
		QuadIndexToXY( QuadIndex, QuadX, QuadY );
		switch(TriNum)
		{
		case 0:
			OutI0 = (QuadX + 0) + (QuadY + 0) * NumVertices;
			OutI1 = (QuadX + 1) + (QuadY + 1) * NumVertices;
			OutI2 = (QuadX + 1) + (QuadY + 0) * NumVertices;
			break;
		case 1:
			OutI0 = (QuadX + 0) + (QuadY + 0) * NumVertices;
			OutI1 = (QuadX + 0) + (QuadY + 1) * NumVertices;
			OutI2 = (QuadX + 1) + (QuadY + 1) * NumVertices;
			break;
		}

		if (bReverseWinding)
		{
			Swap(OutI1, OutI2);
		}
	}

	// from FStaticLightMesh....
	void FLandscapeStaticLightingMesh::GetStaticLightingVertex(int32 VertexIndex, FStaticLightingVertex& OutVertex) const
	{
		int32 X, Y;
		VertexIndexToXY(VertexIndex, X, Y);

		const int32 LocalX = X - ExpandQuadsX;
		const int32 LocalY = Y - ExpandQuadsY;

		const FColor* Data = GetHeightData( X, Y );

		OutVertex.WorldTangentZ.X = 2.0f / 255.f * (float)Data->B - 1.0f;
		OutVertex.WorldTangentZ.Y = 2.0f / 255.f * (float)Data->A - 1.0f;
		OutVertex.WorldTangentZ.Z = FMath::Sqrt(FMath::Max(1.0f - (FMath::Square(OutVertex.WorldTangentZ.X) + FMath::Square(OutVertex.WorldTangentZ.Y)), 0.f));
		OutVertex.WorldTangentX = FVector4f(OutVertex.WorldTangentZ.Z, 0.0f, -OutVertex.WorldTangentZ.X);
		OutVertex.WorldTangentY = OutVertex.WorldTangentZ ^ OutVertex.WorldTangentX;

		// Copied (vaguely) from FLandscapeComponentDataInterface::GetWorldPositionTangents to fix bad lighting when rotated
		const FMatrix44f LtWNoScale = LocalToWorld.GetMatrixWithoutScale();
		OutVertex.WorldTangentX = LtWNoScale.TransformVector(OutVertex.WorldTangentX);
		OutVertex.WorldTangentY = LtWNoScale.TransformVector(OutVertex.WorldTangentY);
		OutVertex.WorldTangentZ = LtWNoScale.TransformVector(OutVertex.WorldTangentZ);

		const uint16 Height = (Data->R << 8) + Data->G;
		OutVertex.WorldPosition = LocalToWorld.TransformPosition( FVector4f( LocalX, LocalY, ((float)Height - 32768.f) * LANDSCAPE_ZSCALE ) );
		//UE_LOGF(LogLightmass, Log, "%d, %d, %d, %d, %d, %d, X:%f, Y:%f, Z:%f ", SectionBaseX + LocalX - ExpandQuadsX, SectionBaseY + LocalY - ExpandQuadsY, ClampedLocalX, ClampedLocalY, SectionBaseX, SectionBaseY, WorldPos.X, WorldPos.Y, WorldPos.Z);

		const int32 LightmapUVIndex = 1;

		OutVertex.TextureCoordinates[0] = FVector2f((float)X / NumVertices, (float)Y / NumVertices); 
		OutVertex.TextureCoordinates[LightmapUVIndex].X = X * UVFactor;
		OutVertex.TextureCoordinates[LightmapUVIndex].Y = Y * UVFactor;
	}

	// FStaticLightingMesh interface.
	void FLandscapeStaticLightingMesh::GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const
	{
		int32 I0, I1, I2;
		//GetTriangleIndices(TriangleIndex,I0, I1, I2);
		I0 = I1 = I2 = 0;
		int32 QuadIndex = TriangleIndex >> 1;
		int32 QuadTriIndex = TriangleIndex & 1;

		GetTriangleIndices(QuadIndex, QuadTriIndex, I0, I1, I2);
		GetStaticLightingVertex(I0,OutV0);
		GetStaticLightingVertex(I1,OutV1);
		GetStaticLightingVertex(I2,OutV2);

		ElementIndex = 0;
	}

	void FLandscapeStaticLightingMesh::GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const
	{
		//OutI0 = OutI1 = OutI2 = 0;
		int32 QuadIndex = TriangleIndex >> 1;
		int32 QuadTriIndex = TriangleIndex & 1;

		GetTriangleIndices(QuadIndex, QuadTriIndex, OutI0, OutI1, OutI2);
	}

	void FLandscapeStaticLightingMesh::Import( class FLightmassImporter& Importer )
	{
		// import super class
		FStaticLightingMesh::Import( Importer );
		Importer.ImportData((FLandscapeStaticLightingMeshData*) this);

		// we have the guid for the mesh, now hook it up to the actual static mesh
		int32 ReadSize = FMath::Square(ComponentSizeQuads + 2*ExpandQuadsX + 1);
		checkf(ReadSize > 0, TEXT("Failed to import Landscape Heightmap data!"));
		Importer.ImportArray(HeightMap, ReadSize);
		check(HeightMap.Num() == ReadSize);

		NumVertices = ComponentSizeQuads + 2*ExpandQuadsX + 1;
		NumQuads = NumVertices - 1;
		UVFactor = LightMapRatio / NumVertices;
		bReverseWinding = (LocalToWorld.RotDeterminant() < 0.0f);
	}

	void FLandscapeStaticLightingTextureMapping::Import( class FLightmassImporter& Importer )
	{
		FStaticLightingTextureMapping::Import(Importer);

		// Can't use the FStaticLightingMapping Import functionality for this
		// as it only looks in the StaticMeshInstances map...
		Mesh = Importer.GetLandscapeMeshInstances().FindRef(Guid);
		check(Mesh);
	}

} //namespace Lightmass
