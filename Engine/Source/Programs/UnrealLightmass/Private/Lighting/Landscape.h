// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LightingMesh.h"
#include "Mappings.h"

#define LANDSCAPE_ZSCALE	(1.0f/128.0f)

namespace Lightmass
{
	/** Represents the triangles of a Landscape primitive to the static lighting system. */
	class FLandscapeStaticLightingMesh : public FStaticLightingMesh, public FLandscapeStaticLightingMeshData
	{
	public:	
		FORCEINLINE void GetStaticLightingVertex(int32 VertexIndex, FStaticLightingVertex& OutVertex) const;
		// FStaticLightingMesh interface.
		virtual void GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const;
		virtual void GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const;

		virtual void Import( class FLightmassImporter& Importer );

		// Accessors from FLandscapeDataInterface
		FORCEINLINE void VertexIndexToXY(int32 VertexIndex, int32& OutX, int32& OutY) const;
		FORCEINLINE void QuadIndexToXY(int32 QuadIndex, int32& OutX, int32& OutY) const;
		FORCEINLINE const FColor*	GetHeightData( int32 LocalX, int32 LocalY ) const;
		FORCEINLINE void GetTriangleIndices(int32 QuadIndex,int32 TriNum,int32& OutI0,int32& OutI1,int32& OutI2) const;

		// We always want to compute visibility for landscape meshes, regardless of material blend mode
		virtual bool IsAlwaysOpaqueForVisibility() const override { return true; }

	private:
		TArray<FColor> HeightMap;
		// Cache
		int32 NumVertices;
		int32 NumQuads;
		float UVFactor;
		bool bReverseWinding;
	};

	/** Represents a landscape primitive with texture mapped static lighting. */
	class FLandscapeStaticLightingTextureMapping : public FStaticLightingTextureMapping
	{
	public:
		virtual void Import( class FLightmassImporter& Importer );
	};

	/** 
	 * A mapping that represents an object which is going to use the global volumetric lightmap.
	 * Hack: currently represented as a texture lightmap to Lightmass for the purposes of surface caching of light (radiosity + direct lighting).
	 */
	class FLandscapeStaticLightingGlobalVolumeMapping : public FLandscapeStaticLightingTextureMapping
	{
	public:

		FLandscapeStaticLightingGlobalVolumeMapping() 
		{}

		virtual class FLandscapeStaticLightingGlobalVolumeMapping* GetLandscapeVolumeMapping() override
		{
			return this;
		}

		virtual const FLandscapeStaticLightingGlobalVolumeMapping* GetLandscapeVolumeMapping() const override
		{
			return this;
		}
	};


} //namespace Lightmass
