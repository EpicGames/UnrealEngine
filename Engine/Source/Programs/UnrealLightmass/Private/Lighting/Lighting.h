// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LMCore.h"

/** 
 * Set to 1 to allow selecting lightmap texels by holding down T and left clicking in the editor,
 * And having debug information about that texel tracked during subsequent lighting rebuilds.
 * Have to enable 'r.TexelDebugging' in the editor as well.
 */
#define ALLOW_LIGHTMAP_SAMPLE_DEBUGGING	0

#if USE_EMBREE
#if USE_EMBREE_MAJOR_VERSION >= 4
	#include <embree4/rtcore.h>
	#include <embree4/rtcore_ray.h>
#elif USE_EMBREE_MAJOR_VERSION == 3
	#include <embree3/rtcore.h>
	#include <embree3/rtcore_ray.h>
#else
	#include <embree2/rtcore.h>
	#include <embree2/rtcore_ray.h>
#endif // USE_EMBREE_MAJOR_VERSION >= 4
#else
typedef void* RTCDevice;
#endif

#include "LightmassScene.h"
#include "Mesh.h"
#include "Texture.h"
#include "Material.h"
#include "LightingMesh.h"
#include "BuildOptions.h"
#include "LightmapData.h"
#include "Mappings.h"
#include "Collision.h"
#include "Embree.h"

namespace Lightmass
{
#if USE_EMBREE
	typedef FStaticLightingAggregateMesh FStaticLightingAggregateMeshType;
#else
	typedef FDefaultAggregateMesh FStaticLightingAggregateMeshType;
#endif
}

#include "BSP.h"
#include "StaticMesh.h"
#include "FluidSurface.h"
#include "Landscape.h"

namespace Lightmass
{
	extern double GStartupTime;
}

// LWC_TODO: Lightmass definition of WORLD_MAX is no longer in sync with EngineDefines.h. Is this acceptable?
#define WORLD_MAX			2097152.0			/* Maximum size of the world */
#define HALF_WORLD_MAX		( WORLD_MAX*0.5f )	/* Half the maximum size of the world */


