// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightingCache.h"
#include "LightingSystem.h"

namespace Lightmass
{

FLightingCacheBase::FLightingCacheBase(const FStaticLightingSystem& InSystem, int32 InBounceNumber) :
	MinCosPointBehindPlane(FMath::Cos((InSystem.IrradianceCachingSettings.PointBehindRecordMaxAngle + 90.0f) * (float)PI / 180.0f)),
	DistanceSmoothFactor(FMath::Max(InSystem.IrradianceCachingSettings.DistanceSmoothFactor * InSystem.GeneralSettings.IndirectLightingSmoothness, 1.0f)),
	bUseIrradianceGradients(InSystem.IrradianceCachingSettings.bUseIrradianceGradients),
	bShowGradientsOnly(InSystem.IrradianceCachingSettings.bShowGradientsOnly),
	bVisualizeIrradianceSamples(InSystem.IrradianceCachingSettings.bVisualizeIrradianceSamples),
	BounceNumber(InBounceNumber),
	NextRecordId(0),
	System(InSystem)
{
	InterpolationAngleNormalization = 1.0f / FMath::Sqrt(1.0f - FMath::Cos(InSystem.IrradianceCachingSettings.InterpolationMaxAngle * (float)PI / 180.0f));

	const float AngleScale = FMath::Max(InSystem.IrradianceCachingSettings.AngleSmoothFactor * InSystem.GeneralSettings.IndirectLightingSmoothness, 1.0f);
	InterpolationAngleNormalizationSmooth = 1.0f / FMath::Sqrt(1.0f - FMath::Cos(AngleScale * InSystem.IrradianceCachingSettings.InterpolationMaxAngle * (float)PI / 180.0f));
}

} //namespace Lightmass
