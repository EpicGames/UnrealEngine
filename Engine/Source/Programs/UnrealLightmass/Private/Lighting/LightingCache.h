// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LightingMesh.h"
#include "LMOctree.h"

namespace Lightmass
{

class FIrradianceCacheStats
{
public:

	uint64 NumCacheLookups;
	uint64 NumRecords;
	uint64 NumInsideGeometry;

	FIrradianceCacheStats() :
		NumCacheLookups(0),
		NumRecords(0),
		NumInsideGeometry(0)
	{}

	FIrradianceCacheStats& operator+=(const FIrradianceCacheStats& B)
	{
		NumCacheLookups += B.NumCacheLookups;
		NumRecords += B.NumRecords;
		NumInsideGeometry += B.NumInsideGeometry;
		return *this;
	}
};

// max absolute error 9.0x10^-3
// Eberly's polynomial degree 1 - respect bounds
// input [-1, 1] and output [0, PI]
inline float acosFast(float inX) 
{
    float x = FMath::Abs(inX);
    float res = -0.156583f * x + (0.5 * PI);
    res *= sqrt(1.0f - x);
    return (inX >= 0) ? res : PI - res;
}

class FQuantizedHemisphereDirection
{
public:

	FQuantizedHemisphereDirection() :
		QuantizedTheta(0),
		QuantizedPhi(0)
	{}

	FQuantizedHemisphereDirection(const FVector4f& UnitTangentSpaceDirection)
	{
		checkSlow(UnitTangentSpaceDirection.IsUnit3());
		checkSlow(UnitTangentSpaceDirection.Z >= 0 && UnitTangentSpaceDirection.Z <= 1);
		const float ThetaFloat = acosFast(UnitTangentSpaceDirection.Z);
		const float PhiFloat = FMath::Atan2(UnitTangentSpaceDirection.Y, UnitTangentSpaceDirection.X);
		const float NormalizedTheta = ThetaFloat / PI;
		const float NormalizedPhi = PhiFloat / (2 * PI) + .5f;

		QuantizedTheta = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(NormalizedTheta * (MAX_uint8 - 1)), 0, MAX_uint8 - 1);
		QuantizedPhi = (uint8)FMath::Clamp<int32>(FMath::RoundToInt(NormalizedPhi * (MAX_uint8 - 1)), 0, MAX_uint8 - 1);
	}

	FVector4f GetDirection() const
	{
		const float Uint8RangeScale = 1.0f / (float)(MAX_uint8 - 1);
		const float RescaledTheta = QuantizedTheta * Uint8RangeScale * PI;
		const float RescaledPhi = (QuantizedPhi * Uint8RangeScale - .5f) * 2 * PI;

		const float SinTheta = FMath::Sin(RescaledTheta);
		return FVector3f(SinTheta * FMath::Cos(RescaledPhi), SinTheta * FMath::Sin(RescaledPhi), FMath::Cos(RescaledTheta));
	}

private:
	uint8 QuantizedTheta;
	uint8 QuantizedPhi;
};

class FFinalGatherHitPoint
{
public:

	FFinalGatherHitPoint() :
		MappingIndex(-1),
		MappingSurfaceCoordinate(-1),
		Weight(0.0f)
	{}

	int32 MappingIndex;
	int32 MappingSurfaceCoordinate;
	FFloat16 Weight;
};





/** The information needed by the lighting cache from a uniform sampled integration of the hemisphere in order to create a lighting record at that point. */
class FLightingCacheGatherInfo
{
public:
	float MinDistance;
	float BackfacingHitsFraction;
	/** Incident radiance and distance from each hemisphere sample. */
	TArray<FLinearColor> PreviousIncidentRadiances;
	TArray<float> PreviousDistances;
	class FGatherHitPoints* HitPointRecorder;

	FLightingCacheGatherInfo() :
		MinDistance(FLT_MAX),
		BackfacingHitsFraction(0),
		HitPointRecorder(NULL)
	{}

	inline void UpdateOnHit(float IntersectionDistance)
	{
		MinDistance = FMath::Min(MinDistance, IntersectionDistance);
	}
};

class FLightingCacheBase
{
public:
	/** See FIrradianceCachingSettings for descriptions of these or the variables they are based on. */
	float InterpolationAngleNormalization;
	float InterpolationAngleNormalizationSmooth;
	const float MinCosPointBehindPlane;
	const float DistanceSmoothFactor;
	const bool bUseIrradianceGradients;
	const bool bShowGradientsOnly;
	const bool bVisualizeIrradianceSamples;
	const int32 BounceNumber;
	int32 NextRecordId;
	mutable FIrradianceCacheStats Stats;
	const class FStaticLightingSystem& System;

	/** Initialization constructor. */
	FLightingCacheBase(const FStaticLightingSystem& InSystem, int32 InBounceNumber);
};

/** A lighting cache. */
template<class SampleType>
class TLightingCache : public FLightingCacheBase
{
public:

	/** The irradiance for a single static lighting vertex. */
	template<class RecordSampleType>
	class FRecord
	{
	public:

		/** The static lighting vertex the irradiance record was computed for. */
		FFullStaticLightingVertex Vertex;

		int32 ElementIndex;

		/** Largest radius that the sample will ever have, used for insertion into spatial data structures. */
		float BoundingRadius;

		/** Radius of this irradiance cache record in the cache pass. */
		float Radius;

		/** Radius of this irradiance cache record in the interpolation pass. */
		float InterpolationRadius;

		/** The lighting incident on an infinitely small surface at WorldPosition facing along WorldNormal. */
		RecordSampleType Lighting;

		/** The rotational gradient along the vector perpendicular to both the record normal and the normal of the vertex being interpolated to, used for higher order interpolation. */
		FVector4f RotationalGradient;

		/** The translational gradient from the record to the point being interpolated to, used for higher order interpolation. */
		FVector4f TranslationalGradient;

		/** For debugging */
		int32 Id;

		float BackfacingHitsFraction;

		/** Initialization constructor. */
		FRecord(const FFullStaticLightingVertex& InVertex,int32 InElementIndex,const FLightingCacheGatherInfo& GatherInfo,float SampleRadius,float InOverrideRadius,const FIrradianceCachingSettings& IrradianceCachingSettings,const FStaticLightingSettings& GeneralSettings,const RecordSampleType& InLighting, const FVector4f& InRotGradient, const FVector4f& InTransGradient):
			Vertex(InVertex),
			ElementIndex(InElementIndex),
			Lighting(InLighting),
			RotationalGradient(InRotGradient),
			TranslationalGradient(InTransGradient),
			Id(-1)
		{
			// Clamp to be larger than the texel
			Radius = FMath::Clamp(GatherInfo.MinDistance, SampleRadius, IrradianceCachingSettings.MaxRecordRadius) * IrradianceCachingSettings.RecordRadiusScale;
			// Use a larger radius to interpolate, which smooths the error
			InterpolationRadius = Radius * FMath::Max(IrradianceCachingSettings.DistanceSmoothFactor * GeneralSettings.IndirectLightingSmoothness, 1.0f);

			BoundingRadius = FMath::Max(Radius, InterpolationRadius);

			BackfacingHitsFraction = GatherInfo.BackfacingHitsFraction;
		}
	};

	struct FRecordOctreeSemantics;

	/** The type of lighting cache octree nodes. */
	typedef TOctree<FRecord<SampleType>,FRecordOctreeSemantics> LightingOctreeType;
	
	/** The octree semantics for irradiance records. */
	struct FRecordOctreeSemantics
	{
		enum { MaxElementsPerLeaf = 4 };
		enum { MaxNodeDepth = 24 };
		enum { LoosenessDenominator = 16 };

		typedef TInlineAllocator<MaxElementsPerLeaf * 8> ElementAllocator;

		static FBoxCenterAndExtent GetBoundingBox(const FRecord<SampleType>& LightingRecord)
		{
			return FBoxCenterAndExtent(
				LightingRecord.Vertex.WorldPosition,
				FVector4f(LightingRecord.BoundingRadius, LightingRecord.BoundingRadius, LightingRecord.BoundingRadius)
				);
		}
	};

	TLightingCache(const FBox3f& InBoundingBox, const FStaticLightingSystem& System, int32 InBounceNumber) :
		FLightingCacheBase(System, InBounceNumber),
		Octree(InBoundingBox.GetCenter(),InBoundingBox.GetExtent().GetMax())
	{}

	/** Adds a lighting record to the cache. */
	void AddRecord(FRecord<SampleType>& Record, bool bInsideGeometry, bool bAddToStats)
	{
		Record.Id = NextRecordId;
		NextRecordId++;
		Octree.AddElement(Record);

		if (bAddToStats)
		{
			Stats.NumRecords++;

			if (bInsideGeometry)
			{
				Stats.NumInsideGeometry++;
			}
		}
	}

	void GetAllRecords(TArray<FRecord<SampleType> >& Records) const
	{
		// Gather an array of samples from the octree
		for(typename LightingOctreeType::template TConstIterator<> NodeIt(Octree); NodeIt.HasPendingNodes(); NodeIt.Advance())
		{
			const typename LightingOctreeType::FNode& CurrentNode = NodeIt.GetCurrentNode();

			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if(CurrentNode.HasChild(ChildRef))
				{
					NodeIt.PushChild(ChildRef);
				}
			}

			for (typename LightingOctreeType::ElementConstIt ElementIt(CurrentNode.GetConstElementIt()); ElementIt; ++ElementIt)
			{
				const FRecord<SampleType>& Sample = *ElementIt;
				Records.Add(Sample);
			}
		}
	}

	LightingOctreeType& GetOctree() { return Octree; }

	/**
	 * Interpolates nearby lighting records for a vertex.
	 * @param Vertex - The vertex to interpolate the lighting for.
	 * @param OutLighting - If true is returned, contains the blended lighting records that were found near the point.
	 * @return true if nearby records were found with enough relevance to interpolate this point's lighting.
	 */
	bool InterpolateLighting(
		const FFullStaticLightingVertex& Vertex, 
		bool bFirstPass, 
		bool bDebugThisSample, 
		float SecondInterpolationSmoothnessReduction,
		SampleType& OutLighting,
		SampleType& OutSecondLighting,
		float& OutBackfacingHitsFraction,
		TArray<FDebugLightingCacheRecord>& DebugCacheRecords,
		class FInfluencingRecordCollector* RecordCollector = NULL) const;

private:

	/** The lighting cache octree. */
	LightingOctreeType Octree;
};

class FInfluencingRecord
{
public:
	int32 RecordIndex;
	FFloat16 RecordWeight;

	FInfluencingRecord(int32 InRecordIndex, FFloat16 InRecordWeight) :
		RecordIndex(InRecordIndex),
		RecordWeight(InRecordWeight)
	{}
};

class FArrayRange
{
public:

	FArrayRange(int32 InStartIndex) :
		StartIndex(InStartIndex),
		NumEntries(0)
	{}

	int32 StartIndex;
	int32 NumEntries;
};

class FInfluencingRecords
{
public:

	TArray<FArrayRange> Ranges;
	TArray<FInfluencingRecord> Data;

	size_t GetAllocatedSize() const { return Ranges.GetAllocatedSize() + Data.GetAllocatedSize(); }
};

class FInfluencingRecordCollector
{
public:

	FInfluencingRecordCollector(FInfluencingRecords& InInfluencingRecords, int32 InCurrentRangeIndex) :
		CurrentRangeIndex(InCurrentRangeIndex),
		InfluencingRecords(InInfluencingRecords)
	{}

	void AddInfluencingRecord(int32 RecordId, float Weight)
	{
		InfluencingRecords.Ranges[CurrentRangeIndex].NumEntries++;
		InfluencingRecords.Data.Add(FInfluencingRecord(RecordId, Weight));
	}

	int32 CurrentRangeIndex;
	FInfluencingRecords& InfluencingRecords;
};

/**
 * Interpolates nearby lighting records for a vertex.
 * @param Vertex - The vertex to interpolate the lighting for.
 * @param OutLighting - If true is returned, contains the blended lighting records that were found near the point.
 * @return true if nearby records were found with enough relevance to interpolate this point's lighting.
 */
template<class SampleType>
bool TLightingCache<SampleType>::InterpolateLighting(
	const FFullStaticLightingVertex& Vertex, 
	bool bFirstPass, 
	bool bDebugThisSample, 
	float SecondInterpolationSmoothnessReduction,
	SampleType& OutLighting,
	SampleType& OutSecondLighting,
	float& OutBackfacingHitsFraction,
	TArray<FDebugLightingCacheRecord>& DebugCacheRecords,
	FInfluencingRecordCollector* RecordCollector) const
{
	if (bFirstPass)
	{
		Stats.NumCacheLookups++;
	}
	const float AngleNormalization = bFirstPass ? InterpolationAngleNormalization : InterpolationAngleNormalizationSmooth;
	// Initialize the sample to zero
	SampleType AccumulatedLighting(ForceInit);
	float TotalWeight = 0.0f;
	SampleType SecondAccumulatedLighting(ForceInit);
	float SecondTotalWeight = 0.0f;
	float AccumulatedBackfacingHitsFraction = 0.0f;

	// Iterate over the octree nodes containing the query point.
	for( typename LightingOctreeType::template TConstElementBoxIterator<> OctreeIt(
		Octree,
		FBoxCenterAndExtent(Vertex.WorldPosition, FVector4f(0,0,0))
		);
		OctreeIt.HasPendingElements();
		OctreeIt.Advance())
	{
		const FRecord<SampleType>& LightingRecord = OctreeIt.GetCurrentElement();

		// Check whether the query point is farther than the record's intersection distance for the direction to the query point.
		const float DistanceSquared = (LightingRecord.Vertex.WorldPosition - Vertex.WorldPosition).SizeSquared3();
		if (DistanceSquared > FMath::Square(LightingRecord.BoundingRadius))
		{
			continue;
		}

		const float Distance = FMath::Sqrt(DistanceSquared);

		// Don't use a lighting record if it's in front of the query point.
		// Query points behind the lighting record may have nearby occluders that the lighting record does not see.
		const FVector4f RecordToVertexVector = Vertex.WorldPosition - LightingRecord.Vertex.WorldPosition;
		// Use the average normal to handle surfaces with constant concavity
		const FVector4f AverageNormal = (LightingRecord.Vertex.TriangleNormal + Vertex.TriangleNormal).GetSafeNormal();
		const float PlaneDistance = Dot3(AverageNormal, RecordToVertexVector.GetSafeNormal());
		// Setup an error metric that goes from 0 if the points are coplanar, to 1 if the point being shaded is at the angle corresponding to MinCosPointBehindPlane behind the plane
		const float PointBehindPlaneError = FMath::Max(PlaneDistance / MinCosPointBehindPlane, 0.0f);

		const float NormalDot = Dot3(LightingRecord.Vertex.WorldTangentZ, Vertex.WorldTangentZ);

		const float NonGradientLighting = bShowGradientsOnly ? 0.0f : 1.0f;
		float RotationalGradientContribution = 0.0f;
		float TranslationalGradientContribution = 0.0f;

		if (bUseIrradianceGradients)
		{
			// Calculate the gradient's contribution
			RotationalGradientContribution = Dot3((LightingRecord.Vertex.WorldTangentZ ^ Vertex.WorldTangentZ), LightingRecord.RotationalGradient);
			TranslationalGradientContribution = Dot3((Vertex.WorldPosition - LightingRecord.Vertex.WorldPosition), LightingRecord.TranslationalGradient);
		}

		// Error metric from "An Approximate Global Illumination System for Computer Generated Films",
		// This error metric has the advantages (over Ward's original metric from "A Ray Tracing Solution to Diffuse Interreflection")
		// That it goes to 0 at the record's radius, which avoids discontinuities,
		// And it is finite at the record's center, which allows filtering the records to be more effective.
		const float EffectiveRadius = bFirstPass ? LightingRecord.Radius : LightingRecord.InterpolationRadius;

		{
			const float DistanceRatio = Distance / EffectiveRadius;
			const float NormalRatio = AngleNormalization * FMath::Sqrt(FMath::Max(1.0f - NormalDot, 0.0f));
			// The total error is the max of the distance, normal and plane errors
			float RecordError = FMath::Max(DistanceRatio, NormalRatio);
			RecordError = FMath::Max(RecordError, PointBehindPlaneError);

			if (RecordError < 1)
			{
				const float RecordWeight = 1.0f - RecordError;

				//@todo - Rotate the record's lighting into this vertex's tangent basis.  We are linearly combining incident lighting in different coordinate spaces.
				AccumulatedLighting = AccumulatedLighting + LightingRecord.Lighting * RecordWeight * (NonGradientLighting + RotationalGradientContribution + TranslationalGradientContribution);
				AccumulatedBackfacingHitsFraction += LightingRecord.BackfacingHitsFraction * RecordWeight * (NonGradientLighting + RotationalGradientContribution + TranslationalGradientContribution);
				// Accumulate the weight of all records
				TotalWeight += RecordWeight;

				if (RecordCollector)
				{
					RecordCollector->AddInfluencingRecord(LightingRecord.Id, RecordWeight);
				}

				if (bVisualizeIrradianceSamples && bDebugThisSample && BounceNumber == 1)
				{
					for (int32 i = 0; i < DebugCacheRecords.Num(); i++)
					{
						FDebugLightingCacheRecord& CurrentRecord =  DebugCacheRecords[i];

						if (CurrentRecord.RecordId == LightingRecord.Id)
						{
							CurrentRecord.bAffectsSelectedTexel = true;
						}
					}
				}
			}
		}

		// Accumulate a second interpolation with reduced smoothness
		// This is useful for lighting components like AO and sky shadowing where less smoothing is needed to hide noise
		// This interpolation is done in the same pass to prevent another traversal of the octree
		{
			const float DistanceRatio = Distance / FMath::Lerp(LightingRecord.Radius, EffectiveRadius, SecondInterpolationSmoothnessReduction);
			const float SecondAngleNormalization = FMath::Lerp(InterpolationAngleNormalization, AngleNormalization, SecondInterpolationSmoothnessReduction);
			const float NormalRatio = SecondAngleNormalization * FMath::Sqrt(FMath::Max(1.0f - NormalDot, 0.0f));
			// The total error is the max of the distance, normal and plane errors
			float RecordError = FMath::Max(DistanceRatio, NormalRatio);
			RecordError = FMath::Max(RecordError, PointBehindPlaneError);

			if (RecordError < 1)
			{
				const float RecordWeight = 1.0f - RecordError;

				//@todo - Rotate the record's lighting into this vertex's tangent basis.  We are linearly combining incident lighting in different coordinate spaces.
				SecondAccumulatedLighting = SecondAccumulatedLighting + LightingRecord.Lighting * RecordWeight * (NonGradientLighting + RotationalGradientContribution + TranslationalGradientContribution);
				// Accumulate the weight of all records
				SecondTotalWeight += RecordWeight;
			}
		}

		if (bVisualizeIrradianceSamples && bDebugThisSample && BounceNumber == 1)
		{
			for (int32 i = 0; i < DebugCacheRecords.Num(); i++)
			{
				FDebugLightingCacheRecord& CurrentRecord =  DebugCacheRecords[i];

				if (CurrentRecord.RecordId == LightingRecord.Id)
				{
					CurrentRecord.bAffectsSelectedTexel = true;
				}
			}
		}
	}

	if (TotalWeight > DELTA)
	{
		// Normalize the accumulated lighting and return success.
		const float InvTotalWeight = 1.0f / TotalWeight;
		OutLighting = OutLighting + AccumulatedLighting * InvTotalWeight;
		OutSecondLighting = OutSecondLighting + SecondAccumulatedLighting * (1.0f / SecondTotalWeight);
		OutBackfacingHitsFraction = AccumulatedBackfacingHitsFraction * InvTotalWeight;
		return true;
	}
	else
	{
		// Irradiance for the query vertex couldn't be interpolated from the cache
		return false;
	}
}

} //namespace Lightmass
