// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightingMesh.h"
#include "Importer.h"
#include "MonteCarlo.h"
#include "LightingSystem.h"

namespace Lightmass
{

/** 
 * Map from FStaticLightingMesh to the index given to uniquely identify all instances of the same primitive component.
 * This is used to give all LOD's of the same primitive component the same mesh index.
 */
TMap<FStaticLightingMesh*, int32> FStaticLightingMesh::MeshToIndexMap;

// Currently disabled due to lack of robustness
bool bAllowMeshAreaLights = false;

/** Evaluates the mesh's Bidirectional Reflectance Distribution Function. */
FLinearColor FStaticLightingMesh::EvaluateBRDF(
	const FStaticLightingVertex& Vertex, 
	int32 ElementIndex,
	const FVector4f& IncomingDirection, 
	const FVector4f& OutgoingDirection) const
{
	checkSlow(Vertex.WorldTangentZ.IsUnit3());
	checkSlow(IncomingDirection.IsUnit3());
	checkSlow(OutgoingDirection.IsUnit3());
	const FVector4f ReflectedIncomingVector = IncomingDirection.Reflect3(Vertex.WorldTangentZ);
	const float OutgoingDotReflected = FMath::Max(Dot3(OutgoingDirection, ReflectedIncomingVector), 0.0f);
	const FLinearColor Diffuse = EvaluateDiffuse(Vertex.TextureCoordinates[0], ElementIndex);
	return Diffuse / (float)PI;
}

/** Generates an outgoing direction sample and evaluates the BRDF for that direction. */
FLinearColor FStaticLightingMesh::SampleBRDF(
	const FStaticLightingVertex& Vertex, 
	int32 ElementIndex,
	const FVector4f& IncomingDirection, 
	FVector4f& OutgoingDirection,
	float& DirectionPDF,
	FLMRandomStream& RandomStream
	) const
{
	checkSlow(Vertex.WorldTangentZ.IsUnit3());
	checkSlow(IncomingDirection.IsUnit3());

	const FVector4f ReflectedIncomingVector = IncomingDirection.Reflect3(Vertex.WorldTangentZ);
	const FVector4f TangentReflectedIncomingVector = Vertex.TransformWorldVectorToTangent(ReflectedIncomingVector);

	const FLinearColor Diffuse = EvaluateDiffuse(Vertex.TextureCoordinates[0], ElementIndex);

	const float DiffuseIntensity = Diffuse.GetLuminance();
	
	// Generate a direction based on the cosine lobe
	FVector4f TangentPathDirection = GetCosineHemisphereVector(RandomStream);

	const float CosTheta = FMath::Max(Dot3(IncomingDirection, Vertex.WorldTangentZ), 0.0f);
	const float CosPDF = CosTheta / (float)PI;
	checkSlow(CosPDF > 0.0f);
	DirectionPDF = CosPDF;

	checkSlow(TangentPathDirection.Z >= 0.0f);
	checkSlow(TangentPathDirection.IsUnit3());
	OutgoingDirection = Vertex.TransformTangentVectorToWorld(TangentPathDirection);
	checkSlow(OutgoingDirection.IsUnit3());

	FLinearColor BRDF = Diffuse / (float)PI;
	// So we can compare against FLinearColor::Black
	BRDF.A = 1.0f;
	return BRDF;
}

bool FStaticLightingMesh::DoesMeshBelongToLOD0() const
{
	const uint32 GeoMeshLODIndex = GetLODIndices() & 0xFFFF;
	const uint32 GeoHLODTreeIndex = (GetLODIndices() & 0xFFFF0000) >> 16;
	const uint32 GeoHLODRange = GetHLODRange();
	const uint32 GeoHLODRangeStart = GeoHLODRange & 0xFFFF;
	const uint32 GeoHLODRangeEnd = (GeoHLODRange & 0xFFFF0000) >> 16;

	bool bMeshBelongsToLOD0 = GeoMeshLODIndex == 0;

	if (GeoHLODTreeIndex > 0)
	{
		bMeshBelongsToLOD0 = GeoHLODRangeStart == GeoHLODRangeEnd;
	}

	return bMeshBelongsToLOD0;
}

void FStaticLightingMesh::SetDebugMaterial(bool bInUseDebugMaterial, FLinearColor InDiffuse)
{
	bUseDebugMaterial = bInUseDebugMaterial;
	DebugDiffuse = InDiffuse;
}

void FStaticLightingMesh::Import( FLightmassImporter& Importer )
{
	// Import into a temporary struct and manually copy settings over,
	// Since the import will overwrite padding in FStaticLightingMeshInstanceData which is actual data in derived classes.
	FStaticLightingMeshInstanceData TempData;
	Importer.ImportData(&TempData);
	Guid = TempData.Guid;
	NumTriangles = TempData.NumTriangles;
	NumShadingTriangles = TempData.NumShadingTriangles;
	NumVertices = TempData.NumVertices;
	NumShadingVertices = TempData.NumShadingVertices;
	MeshIndex = TempData.MeshIndex;
	LevelGuid = TempData.LevelGuid;
	TextureCoordinateIndex = TempData.TextureCoordinateIndex;
	LightingFlags = TempData.LightingFlags;
	bCastShadowAsTwoSided = TempData.bCastShadowAsTwoSided;
	bMovable = TempData.bMovable;
	NumRelevantLights = TempData.NumRelevantLights;
	BoundingBox = TempData.BoundingBox;
	Importer.ImportGuidArray( RelevantLights, NumRelevantLights, Importer.GetLights() );

	int32 NumVisibilityIds = 0;
	Importer.ImportData(&NumVisibilityIds);
	Importer.ImportArray(VisibilityIds, NumVisibilityIds);

	int32 NumMaterialElements = 0;
	Importer.ImportData(&NumMaterialElements);
	check(NumMaterialElements > 0);
	MaterialElements.Empty(NumMaterialElements);
	MaterialElements.AddUninitialized(NumMaterialElements);
	for (int32 MtrlIdx = 0; MtrlIdx < NumMaterialElements; MtrlIdx++)
	{
		FMaterialElement& CurrentMaterialElement = MaterialElements[MtrlIdx];
		FMaterialElementData METempData;
		Importer.ImportData(&METempData);
		CurrentMaterialElement.MaterialHash = METempData.MaterialHash;
		CurrentMaterialElement.bUseTwoSidedLighting = METempData.bUseTwoSidedLighting;
		CurrentMaterialElement.bShadowIndirectOnly = METempData.bShadowIndirectOnly;
		CurrentMaterialElement.bUseEmissiveForStaticLighting = METempData.bUseEmissiveForStaticLighting;
		CurrentMaterialElement.bUseVertexNormalForHemisphereGather = METempData.bUseVertexNormalForHemisphereGather;
		// Validating data here instead of in Unreal since EmissiveLightFalloffExponent is used in so many different object types
		CurrentMaterialElement.EmissiveLightFalloffExponent = FMath::Max(METempData.EmissiveLightFalloffExponent, 0.0f);
		CurrentMaterialElement.EmissiveLightExplicitInfluenceRadius = FMath::Max(METempData.EmissiveLightExplicitInfluenceRadius, 0.0f);
		CurrentMaterialElement.EmissiveBoost = METempData.EmissiveBoost;
		CurrentMaterialElement.DiffuseBoost = METempData.DiffuseBoost;
		CurrentMaterialElement.FullyOccludedSamplesFraction = METempData.FullyOccludedSamplesFraction;
		CurrentMaterialElement.Material = Importer.ConditionalImportObject<FMaterial>(CurrentMaterialElement.MaterialHash, LM_MATERIAL_VERSION, LM_MATERIAL_EXTENSION, LM_MATERIAL_CHANNEL_FLAGS, Importer.GetMaterials());
		checkf(CurrentMaterialElement.Material, TEXT("Failed to import material with Hash %s"), *CurrentMaterialElement.MaterialHash.ToString());

		const bool bMasked = CurrentMaterialElement.Material->BlendMode == BLEND_Masked;

		CurrentMaterialElement.bIsMasked = bMasked && CurrentMaterialElement.Material->TransmissionSize > 0;
		CurrentMaterialElement.bIsTwoSided = CurrentMaterialElement.Material->bTwoSided;
		CurrentMaterialElement.bIsThinSurface = CurrentMaterialElement.Material->bIsThinSurface;
		CurrentMaterialElement.bTranslucent = !CurrentMaterialElement.bIsMasked && CurrentMaterialElement.Material->TransmissionSize > 0;
		CurrentMaterialElement.bCastShadowAsMasked = CurrentMaterialElement.Material->bCastShadowAsMasked;
		CurrentMaterialElement.bSurfaceDomain = CurrentMaterialElement.Material->bSurfaceDomain;
	}
	bColorInvalidTexels = true;
	bUseDebugMaterial = false;
	DebugDiffuse = FLinearColor::Black;
}

/** Determines whether two triangles overlap each other's AABB's. */
static bool AxisAlignedTriangleIntersectTriangle2d(
	const FVector2f& V0, const FVector2f& V1, const FVector2f& V2, 
	const FVector2f& OtherV0, const FVector2f& OtherV1, const FVector2f& OtherV2)
{
	const FVector2f MinFirst = FVector2f::Min(V0, FVector2f::Min(V1, V2));
	const FVector2f MaxFirst = FVector2f::Max(V0, FVector2f::Max(V1, V2));
	const FVector2f MinSecond = FVector2f::Min(OtherV0, FVector2f::Min(OtherV1, OtherV2));
	const FVector2f MaxSecond = FVector2f::Max(OtherV0, FVector2f::Max(OtherV1, OtherV2));

	return !(MinFirst.X > MaxSecond.X 
		|| MinSecond.X > MaxFirst.X 
		|| MinFirst.Y > MaxSecond.Y 
		|| MinSecond.Y > MaxFirst.Y);
}

static const int32 UnprocessedIndex = -1;
static const int32 PendingProcessingIndex = -2;
static const int32 NotEmissiveIndex = -3;

/** Allows the mesh to create mesh area lights from its emissive contribution */
void FStaticLightingMesh::CreateMeshAreaLights(
	const FStaticLightingSystem& LightingSystem, 
	const FScene& Scene,
	TIndirectArray<FMeshAreaLight>& MeshAreaLights) const
{
	bool bAnyElementsUseEmissiveForLighting = false;
	for (int32 MaterialIndex = 0; MaterialIndex < MaterialElements.Num(); MaterialIndex++)
	{
		if (bAllowMeshAreaLights && 
			MaterialElements[MaterialIndex].bUseEmissiveForStaticLighting &&
			MaterialElements[MaterialIndex].Material->EmissiveSize > 0)
		{
			bAnyElementsUseEmissiveForLighting = true;
			break;
		}
	}

	if (!bAnyElementsUseEmissiveForLighting)
	{
		// Exit if none of the mesh's elements use emissive for lighting
		return;
	}

	// Emit warnings for meshes with lots of triangles, since the mesh area light creation is O(N^2) on the number of triangles
	if (NumTriangles > 3000 && NumTriangles <= 5000)
	{
		GSwarm->SendAlertMessage(NSwarm::ALERT_LEVEL_WARNING, Guid, SOURCEOBJECTTYPE_Mapping, TEXT("LightmassError_EmissiveMeshHighPolyCount"));
	}
	else if (NumTriangles > 5000)
	{
		GSwarm->SendAlertMessage(NSwarm::ALERT_LEVEL_ERROR, Guid, SOURCEOBJECTTYPE_Mapping, TEXT("LightmassError_EmissiveMeshExtremelyHighPolyCount"));
		// This mesh will take a very long time to create mesh area lights for, so skip it
		return;
	}
	
	TArray<FStaticLightingVertex> MeshVertices;
	MeshVertices.Empty(NumTriangles * 3);
	MeshVertices.AddZeroed(NumTriangles * 3);

	TArray<int32> ElementIndices;
	ElementIndices.Empty(NumTriangles);
	ElementIndices.AddZeroed(NumTriangles);

	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		// Query the mesh for the triangle's vertices.
		GetTriangle(TriangleIndex, MeshVertices[TriangleIndex * 3 + 0], MeshVertices[TriangleIndex * 3 + 1], MeshVertices[TriangleIndex * 3 + 2], ElementIndices[TriangleIndex]);
	}

	TArray<TArray<int32> > LayeredGroupTriangles;
	// Split the mesh into layers whose UVs do not overlap, maintaining adjacency in world space position and UVs.
	// This way meshes with tiling emissive textures are handled correctly, all instances of the emissive texels will emit light.
	CalculateUniqueLayers(MeshVertices, ElementIndices, LayeredGroupTriangles);

	// get Min/MaxUV on the mesh for the triangles
	FVector2f MinUV(FLT_MAX, FLT_MAX), MaxUV(-FLT_MAX, -FLT_MAX);
	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		for (int32 VIndex = 0; VIndex < 3; VIndex++)
		{
			const FStaticLightingVertex& CurrentVertex = MeshVertices[TriangleIndex * 3 + VIndex];
			MinUV.X = FMath::Min(MinUV.X, CurrentVertex.TextureCoordinates[TextureCoordinateIndex].X);
			MaxUV.X = FMath::Max(MaxUV.X, CurrentVertex.TextureCoordinates[TextureCoordinateIndex].X);
			MinUV.Y = FMath::Min(MinUV.Y, CurrentVertex.TextureCoordinates[TextureCoordinateIndex].Y);
			MaxUV.Y = FMath::Max(MaxUV.Y, CurrentVertex.TextureCoordinates[TextureCoordinateIndex].Y);
		}
	}

	// figure out many iterations of the texture we need (enough integer repetitions to cover the entire UV range used)
	// we floor the min and max because we need to see which integer wrap of UVs it falls into
	// so, if we had range .2 to .8, the floors would both go to 0, then add 1 to account for that one
	// if we have range -.2 to .3, we need space for the -1 .. 0 wrap, and the 0 to 1 wrap (ie 2 iterations)
	// @todo UE5: Actually, if the Min was used to modify the UV when looping through the Corners array,
	// we wouldn't need full integer ranges
	const int32 NumIterationsX = (FMath::FloorToInt(MaxUV.X) - FMath::FloorToInt(MinUV.X)) + 1;
	const int32 NumIterationsY = (FMath::FloorToInt(MaxUV.Y) - FMath::FloorToInt(MinUV.Y)) + 1;

	// calculate the bias and scale needed to map the random UV range into 0 .. NumIterations when rasterizing
	// into the TexelToCornersMap
	const FVector2f UVBias(-FMath::FloorToFloat(MinUV.X), -FMath::FloorToFloat(MinUV.Y));
	const FVector2f UVScale(1.0f / NumIterationsX, 1.0f / NumIterationsY);

	for (int32 MaterialIndex = 0; MaterialIndex < MaterialElements.Num(); MaterialIndex++)
	{
		const FMaterial& CurrentMaterial = *MaterialElements[MaterialIndex].Material;
		if (bAllowMeshAreaLights &&
			MaterialElements[MaterialIndex].bUseEmissiveForStaticLighting &&
			CurrentMaterial.EmissiveSize > 0)
		{
			// Operate on each layer independently
			for (int32 GroupIndex = 0; GroupIndex < LayeredGroupTriangles.Num(); GroupIndex++)
			{
				// Allocate a map from texel to the corners of that texel, giving enough space for all of the possible integer wraps
				FTexelToCornersMap TexelToCornersMap(NumIterationsX * CurrentMaterial.EmissiveSize, NumIterationsY * CurrentMaterial.EmissiveSize);
				LightingSystem.CalculateTexelCorners(LayeredGroupTriangles[GroupIndex], MeshVertices, TexelToCornersMap, ElementIndices, MaterialIndex, TextureCoordinateIndex, false, UVBias, UVScale);

				for (int32 Y = 0; Y < TexelToCornersMap.GetSizeY(); Y++)
				{
					for (int32 X = 0; X < TexelToCornersMap.GetSizeX(); X++)
					{
						FTexelToCorners& CurrentTexelCorners = TexelToCornersMap(X, Y);
						// Normals need to be unit as their dot product will be used in comparisons later
						CurrentTexelCorners.WorldTangentZ = CurrentTexelCorners.WorldTangentZ.SizeSquared3() > DELTA ? CurrentTexelCorners.WorldTangentZ.GetUnsafeNormal3() : FVector4f(0,0,1);
					}
				}

				TArray<int32> LightIndices;
				// Allocate an array of light indices, one for each texel, indexed by Y * SizeX + X
				LightIndices.AddZeroed(TexelToCornersMap.GetSizeX() * TexelToCornersMap.GetSizeY());
				// Initialize light indices to unprocessed
				FMemory::Memset(LightIndices.GetData(), UnprocessedIndex, LightIndices.Num() * LightIndices.GetTypeSize());
				int32 NextLightIndex = 0;
				// The temporary stack of texels that need to be processed
				TArray<FIntPoint> TexelsInCurrentLight;
				// Iterate over all texels and assign a light index to each one
				for (int32 Y = 0; Y < TexelToCornersMap.GetSizeY(); Y++)
				{
					for (int32 X = 0; X < TexelToCornersMap.GetSizeX(); X++)
					{
						// Push the current texel onto the stack if it is emissive and hasn't been processed yet
						AddLightTexel(TexelToCornersMap, MaterialIndex, LightIndices, X, Y, Scene.MeshAreaLightSettings.EmissiveIntensityThreshold, TexelsInCurrentLight, CurrentMaterial.EmissiveSize, CurrentMaterial.EmissiveSize);
						if (TexelsInCurrentLight.Num() > 0)
						{
							// This is the first texel in a new light group
							const int32 CurrentLightIndex = NextLightIndex;
							// Update the next light index
							NextLightIndex++;
							// Flood fill neighboring emissive texels with CurrentLightIndex
							// This is done with a temporary stack instead of recursion since the recursion depth can be very deep and overflow the stack
							while (TexelsInCurrentLight.Num() > 0)
							{
								// Remove the last texel in the stack
								const FIntPoint NextTexel = TexelsInCurrentLight.Pop();
								// Mark it as belonging to the current light
								LightIndices[NextTexel.Y * TexelToCornersMap.GetSizeX() + NextTexel.X] = CurrentLightIndex;
								// Push all of the texel's emissive, unprocessed neighbors onto the stack
								AddLightTexel(TexelToCornersMap, MaterialIndex, LightIndices, NextTexel.X - 1, NextTexel.Y, Scene.MeshAreaLightSettings.EmissiveIntensityThreshold, TexelsInCurrentLight, CurrentMaterial.EmissiveSize, CurrentMaterial.EmissiveSize);
								AddLightTexel(TexelToCornersMap, MaterialIndex, LightIndices, NextTexel.X + 1, NextTexel.Y, Scene.MeshAreaLightSettings.EmissiveIntensityThreshold, TexelsInCurrentLight, CurrentMaterial.EmissiveSize, CurrentMaterial.EmissiveSize);
								AddLightTexel(TexelToCornersMap, MaterialIndex, LightIndices, NextTexel.X, NextTexel.Y - 1, Scene.MeshAreaLightSettings.EmissiveIntensityThreshold, TexelsInCurrentLight, CurrentMaterial.EmissiveSize, CurrentMaterial.EmissiveSize);
								AddLightTexel(TexelToCornersMap, MaterialIndex, LightIndices, NextTexel.X, NextTexel.Y + 1, Scene.MeshAreaLightSettings.EmissiveIntensityThreshold, TexelsInCurrentLight, CurrentMaterial.EmissiveSize, CurrentMaterial.EmissiveSize);
							}
						}
					}
				}

				TArray<int32> PrimitiveIndices;
				PrimitiveIndices.AddZeroed(TexelToCornersMap.GetSizeX() * TexelToCornersMap.GetSizeY());
				FMemory::Memset(PrimitiveIndices.GetData(), UnprocessedIndex, PrimitiveIndices.Num() * PrimitiveIndices.GetTypeSize());
				int32 NextPrimitiveIndex = 0;
				const float DistanceThreshold = FBoxSphereBounds3f(BoundingBox).SphereRadius * Scene.MeshAreaLightSettings.MeshAreaLightSimplifyMeshBoundingRadiusFractionThreshold;
				// The temporary stack of texels that need to be processed
				TArray<FIntPoint> PendingTexels;
				// Iterate over all texels and assign a primitive index to each one
				// This effectively simplifies the mesh area light by reducing the number of primitives that are needed to represent the light
				for (int32 Y = 0; Y < TexelToCornersMap.GetSizeY(); Y++)
				{
					for (int32 X = 0; X < TexelToCornersMap.GetSizeX(); X++)
					{
						const int32 LightIndex = LightIndices[Y * TexelToCornersMap.GetSizeX() + X];
						// Every texel should have a valid light index or be marked not emissive by this pass
						checkSlow(LightIndex != UnprocessedIndex);
						checkSlow(LightIndex != PendingProcessingIndex);
						const FTexelToCorners& CurrentTexelCorners = TexelToCornersMap(X, Y);

						FVector4f PrimitiveCenter(0,0,0);
						for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
						{
							PrimitiveCenter += CurrentTexelCorners.Corners[CornerIndex].WorldPosition / (float)NumTexelCorners;
						}

						// Push the current texel onto the stack if it can be merged into the same primitive and hasn't been processed yet
						AddPrimitiveTexel(TexelToCornersMap, CurrentTexelCorners, LightIndex, PrimitiveCenter, PrimitiveIndices, LightIndices, X, Y, PendingTexels, Scene, DistanceThreshold);

						if (PendingTexels.Num() > 0) 
						{
							const int32 CurrentPrimitiveIndex = NextPrimitiveIndex;
							// This is the first texel to go into a new primitive
							NextPrimitiveIndex++;
							// Flood fill neighboring texels with CurrentPrimitiveIndex
							// This is done with a temporary stack instead of recursion since the recursion depth can be very deep and overflow the stack
							while (PendingTexels.Num() > 0)
							{
								// Remove the last texel in the stack
								const FIntPoint NextTexel = PendingTexels.Pop();
								// Mark it as belonging to the current primitive
								PrimitiveIndices[NextTexel.Y * TexelToCornersMap.GetSizeX() + NextTexel.X] = CurrentPrimitiveIndex;
								const FTexelToCorners& NextTexelCorners = TexelToCornersMap(NextTexel.X, NextTexel.Y);
								const int32 NextTexelLightIndex = LightIndices[NextTexel.Y * TexelToCornersMap.GetSizeX() + NextTexel.X];
								// Push all of the texel's neighbors onto the stack if they should be merged into the current primitive
								AddPrimitiveTexel(TexelToCornersMap, NextTexelCorners, NextTexelLightIndex, PrimitiveCenter, PrimitiveIndices, LightIndices, NextTexel.X - 1, NextTexel.Y, PendingTexels, Scene, DistanceThreshold);
								AddPrimitiveTexel(TexelToCornersMap, NextTexelCorners, NextTexelLightIndex, PrimitiveCenter, PrimitiveIndices, LightIndices, NextTexel.X + 1, NextTexel.Y, PendingTexels, Scene, DistanceThreshold);
								AddPrimitiveTexel(TexelToCornersMap, NextTexelCorners, NextTexelLightIndex, PrimitiveCenter, PrimitiveIndices, LightIndices, NextTexel.X, NextTexel.Y - 1, PendingTexels, Scene, DistanceThreshold);
								AddPrimitiveTexel(TexelToCornersMap, NextTexelCorners, NextTexelLightIndex, PrimitiveCenter, PrimitiveIndices, LightIndices, NextTexel.X, NextTexel.Y + 1, PendingTexels, Scene, DistanceThreshold);
							}
						}
					}
				}

				// An array of mesh light primitives for each light
				TArray<TArray<FMeshLightPrimitive>> EmissivePrimitives;
				// Allocate for the number of light indices that were assigned
				EmissivePrimitives.Empty(NextLightIndex);
				EmissivePrimitives.AddZeroed(NextLightIndex);
				for (int32 LightIndex = 0; LightIndex < NextLightIndex; LightIndex++)
				{
					// Allocate for the number of primitives that were assigned
					EmissivePrimitives[LightIndex].Empty(NextPrimitiveIndex);
					EmissivePrimitives[LightIndex].AddZeroed(NextPrimitiveIndex);
				}

				for (int32 Y = 0; Y < TexelToCornersMap.GetSizeY(); Y++)
				{
					const float YFraction = Y / (float)CurrentMaterial.EmissiveSize;
					for (int32 X = 0; X < TexelToCornersMap.GetSizeX(); X++)
					{
						const int32 LightIndex = LightIndices[Y * TexelToCornersMap.GetSizeX() + X];
						// Every texel should have a valid light index or be marked not emissive by this pass
						checkSlow(LightIndex != UnprocessedIndex);
						checkSlow(LightIndex != PendingProcessingIndex);
						if (LightIndex >= 0)
						{
							const FTexelToCorners& CurrentTexelCorners = TexelToCornersMap(X, Y);
							const int32 PrimitiveIndex = PrimitiveIndices[Y * TexelToCornersMap.GetSizeX() + X];
							// Every texel should have a valid primitive index or be marked not emissive by this pass
							checkSlow(PrimitiveIndex != UnprocessedIndex);
							checkSlow(PrimitiveIndex != PendingProcessingIndex);
							if (PrimitiveIndex >= 0)
							{
								// Calculate the texel's center
								FVector4f TexelCenter(0,0,0);
								bool bAllCornersValid = true;
								for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
								{
									TexelCenter += CurrentTexelCorners.Corners[CornerIndex].WorldPosition / (float)NumTexelCorners;
									bAllCornersValid = bAllCornersValid && CurrentTexelCorners.bValid[CornerIndex];
								}
								checkSlow(bAllCornersValid);

								// Calculate the texel's bounding radius
								float TexelBoundingRadiusSquared = 0.0f;
								for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
								{
									const float CurrentRadiusSquared = (TexelCenter - CurrentTexelCorners.Corners[CornerIndex].WorldPosition).SizeSquared3();
									if (CurrentRadiusSquared > TexelBoundingRadiusSquared)
									{
										TexelBoundingRadiusSquared = CurrentRadiusSquared;
									}
								}

								const float XFraction = X / (float)CurrentMaterial.EmissiveSize;
								const FLinearColor CurrentEmissive = EvaluateEmissive(FVector2f(XFraction, YFraction), MaterialIndex);
								checkSlow(CurrentEmissive.R > Scene.MeshAreaLightSettings.EmissiveIntensityThreshold 
									|| CurrentEmissive.G > Scene.MeshAreaLightSettings.EmissiveIntensityThreshold 
									|| CurrentEmissive.B > Scene.MeshAreaLightSettings.EmissiveIntensityThreshold);

								// Add a new primitive representing this texel to the light the texel was assigned to in the first pass
								EmissivePrimitives[LightIndex][PrimitiveIndex].AddSubPrimitive(
									CurrentTexelCorners, 
									FIntPoint(X, Y), 
									CurrentEmissive, 
									// Offset the light primitives by a fraction of the texel's bounding radius to avoid incorrect self-occlusion,
									// Since the surface of the light is actually a mesh
									FMath::Sqrt(TexelBoundingRadiusSquared) * Scene.SceneConstants.VisibilityNormalOffsetSampleRadiusScale);
							}
						}
					}
				}

				TArray<TArray<FMeshLightPrimitive>> TrimmedEmissivePrimitives;
				TrimmedEmissivePrimitives.Empty(EmissivePrimitives.Num());
				TrimmedEmissivePrimitives.AddZeroed(EmissivePrimitives.Num());
				for (int32 LightIndex = 0; LightIndex < EmissivePrimitives.Num(); LightIndex++)
				{
					TrimmedEmissivePrimitives[LightIndex].Empty(EmissivePrimitives[LightIndex].Num());
					for (int32 PrimitiveIndex = 0; PrimitiveIndex < EmissivePrimitives[LightIndex].Num(); PrimitiveIndex++)
					{
						if (EmissivePrimitives[LightIndex][PrimitiveIndex].NumSubPrimitives > 0)
						{
							// Only copy over primitives containing one or more sub primitives
							TrimmedEmissivePrimitives[LightIndex].Add(EmissivePrimitives[LightIndex][PrimitiveIndex]);
							TrimmedEmissivePrimitives[LightIndex].Last().Finalize();

							if (Scene.MeshAreaLightSettings.bVisualizeMeshAreaLightPrimitives)
							{
								// Draw 4 lines between the primitive corners for debugging
								// Currently hijacking ShadowRays
								//LightingSystem.DebugOutput.ShadowRays.Add(FDebugStaticLightingRay(EmissivePrimitives[LightIndex][PrimitiveIndex].Corners[0].WorldPosition - FVector4f(0,0,.1f), EmissivePrimitives[LightIndex][PrimitiveIndex].Corners[1].WorldPosition - FVector4f(0,0,.1f), true, false));
								//LightingSystem.DebugOutput.ShadowRays.Add(FDebugStaticLightingRay(EmissivePrimitives[LightIndex][PrimitiveIndex].Corners[1].WorldPosition - FVector4f(0,0,.1f), EmissivePrimitives[LightIndex][PrimitiveIndex].Corners[3].WorldPosition - FVector4f(0,0,.1f), true, true));
								//LightingSystem.DebugOutput.ShadowRays.Add(FDebugStaticLightingRay(EmissivePrimitives[LightIndex][PrimitiveIndex].Corners[3].WorldPosition - FVector4f(0,0,.1f), EmissivePrimitives[LightIndex][PrimitiveIndex].Corners[2].WorldPosition - FVector4f(0,0,.1f), true, false));
								//LightingSystem.DebugOutput.ShadowRays.Add(FDebugStaticLightingRay(EmissivePrimitives[LightIndex][PrimitiveIndex].Corners[2].WorldPosition - FVector4f(0,0,.1f), EmissivePrimitives[LightIndex][PrimitiveIndex].Corners[0].WorldPosition - FVector4f(0,0,.1f), true, true));
							}
						}
					}
				}

				// Create mesh area lights from each group of primitives that were gathered
				for (int32 LightIndex = 0; LightIndex < TrimmedEmissivePrimitives.Num(); LightIndex++)
				{
					if (TrimmedEmissivePrimitives[LightIndex].Num() > 0)
					{
						// Initialize all of the mesh area light's unused properties to 0
						FMeshAreaLight* NewLight = new FMeshAreaLight(ForceInit);
						NewLight->LightFlags = GI_LIGHT_HASSTATICLIGHTING | GI_LIGHT_CASTSHADOWS | GI_LIGHT_CASTSTATICSHADOWS;
						NewLight->SetPrimitives(
							TrimmedEmissivePrimitives[LightIndex], 
							MaterialElements[MaterialIndex].EmissiveLightFalloffExponent, 
							MaterialElements[MaterialIndex].EmissiveLightExplicitInfluenceRadius,
							Scene.MeshAreaLightSettings.MeshAreaLightGridSize,
							LevelGuid);
						MeshAreaLights.Add(NewLight);
					}
				}
			}
		}
	}
}

/** Splits a mesh into layers with non-overlapping UVs, maintaining adjacency in world space and UVs. */
void FStaticLightingMesh::CalculateUniqueLayers(
	const TArray<FStaticLightingVertex>& MeshVertices, 
	const TArray<int32>& ElementIndices, 
	TArray<TArray<int32> >& LayeredGroupTriangles) const
{
	// Indices of adjacent triangles in world space, 3 indices for each triangle
	TArray<int32> WorldSpaceAdjacentTriangles;
	WorldSpaceAdjacentTriangles.Empty(NumTriangles * 3);
	WorldSpaceAdjacentTriangles.AddZeroed(NumTriangles * 3);
	// Adjacency for the mesh's triangles compared in texture space
	TArray<int32> TextureSpaceAdjacentTriangles;
	TextureSpaceAdjacentTriangles.Empty(NumTriangles * 3);
	TextureSpaceAdjacentTriangles.AddZeroed(NumTriangles * 3);

	// Initialize all triangles to having no adjacent triangles
	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		WorldSpaceAdjacentTriangles[TriangleIndex * 3 + 0] = INDEX_NONE;
		WorldSpaceAdjacentTriangles[TriangleIndex * 3 + 1] = INDEX_NONE;
		WorldSpaceAdjacentTriangles[TriangleIndex * 3 + 2] = INDEX_NONE;
		TextureSpaceAdjacentTriangles[TriangleIndex * 3 + 0] = INDEX_NONE;
		TextureSpaceAdjacentTriangles[TriangleIndex * 3 + 1] = INDEX_NONE;
		TextureSpaceAdjacentTriangles[TriangleIndex * 3 + 2] = INDEX_NONE;
	}

	// Generate world space and texture space adjacency
	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		for (int32 OtherTriangleIndex = TriangleIndex + 1; OtherTriangleIndex < NumTriangles; OtherTriangleIndex++)
		{
			for (int32 EdgeIndex = 0; EdgeIndex < 3; EdgeIndex++)
			{
				if (WorldSpaceAdjacentTriangles[TriangleIndex * 3 + EdgeIndex] == INDEX_NONE)
				{
					for (int32 OtherEdgeIndex = 0; OtherEdgeIndex < 3; OtherEdgeIndex++)
					{
						if (WorldSpaceAdjacentTriangles[OtherTriangleIndex * 3 + OtherEdgeIndex] == INDEX_NONE)
						{
							const FStaticLightingVertex& V0 = MeshVertices[TriangleIndex * 3 + EdgeIndex];
							const FStaticLightingVertex& V1 = MeshVertices[TriangleIndex * 3 + (EdgeIndex + 1) % 3];
							const FStaticLightingVertex& OtherV0 = MeshVertices[OtherTriangleIndex * 3 + OtherEdgeIndex];
							const FStaticLightingVertex& OtherV1 = MeshVertices[OtherTriangleIndex * 3 + (OtherEdgeIndex + 1) % 3];
							// Triangles are adjacent if they share one edge in world space
							if ((V0.WorldPosition - OtherV1.WorldPosition).IsNearlyZero3(KINDA_SMALL_NUMBER * 100.0f)
								&& (V1.WorldPosition - OtherV0.WorldPosition).IsNearlyZero3(KINDA_SMALL_NUMBER * 100.0f))
							{
								WorldSpaceAdjacentTriangles[TriangleIndex * 3 + EdgeIndex] = OtherTriangleIndex;
								WorldSpaceAdjacentTriangles[OtherTriangleIndex * 3 + OtherEdgeIndex] = TriangleIndex;
								break;
							}
						}
					}
				}

				if (TextureSpaceAdjacentTriangles[TriangleIndex * 3 + EdgeIndex] == INDEX_NONE)
				{
					for (int32 OtherEdgeIndex = 0; OtherEdgeIndex < 3; OtherEdgeIndex++)
					{
						if (TextureSpaceAdjacentTriangles[OtherTriangleIndex * 3 + OtherEdgeIndex] == INDEX_NONE)
						{
							const FStaticLightingVertex& V0 = MeshVertices[TriangleIndex * 3 + EdgeIndex];
							const FStaticLightingVertex& V1 = MeshVertices[TriangleIndex * 3 + (EdgeIndex + 1) % 3];
							const FStaticLightingVertex& OtherV0 = MeshVertices[OtherTriangleIndex * 3 + OtherEdgeIndex];
							const FStaticLightingVertex& OtherV1 = MeshVertices[OtherTriangleIndex * 3 + (OtherEdgeIndex + 1) % 3];
							// Triangles are adjacent if they share one edge in texture space
							if ((V0.TextureCoordinates[TextureCoordinateIndex] - OtherV1.TextureCoordinates[TextureCoordinateIndex]).IsNearlyZero(KINDA_SMALL_NUMBER * 100.0f)
								&& (V1.TextureCoordinates[TextureCoordinateIndex] - OtherV0.TextureCoordinates[TextureCoordinateIndex]).IsNearlyZero(KINDA_SMALL_NUMBER * 100.0f))
							{
								TextureSpaceAdjacentTriangles[TriangleIndex * 3 + EdgeIndex] = OtherTriangleIndex;
								TextureSpaceAdjacentTriangles[OtherTriangleIndex * 3 + OtherEdgeIndex] = TriangleIndex;
								break;
							}
						}
					}
				}
			}
		}
	}

	TArray<int32> TriangleGroups;
	TriangleGroups.Empty(NumTriangles);
	TriangleGroups.AddZeroed(NumTriangles);
	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		TriangleGroups[TriangleIndex] = INDEX_NONE;
	}

	TArray<int32> PendingTriangles;
	int32 NextGroupIndex = 0;
	// Arrange adjacent triangles in texture and world space together into groups
	// Assign a group index to each triangle
	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		if (TriangleGroups[TriangleIndex] == INDEX_NONE)
		{
			// Push the current triangle
			PendingTriangles.Add(TriangleIndex);
			const int32 CurrentGroupIndex = NextGroupIndex;
			NextGroupIndex++;
			while (PendingTriangles.Num() > 0)
			{
				// Pop the next pending triangle and process it
				const int32 NeighborTriangleIndex = PendingTriangles.Pop();
				// Assign the group index
				TriangleGroups[NeighborTriangleIndex] = CurrentGroupIndex;
				// Flood fill all adjacent triangles with the same group index
				for (int32 NeighborIndex = 0; NeighborIndex < 3; NeighborIndex++)
				{
					const int32 WorldSpaceNeighbor = WorldSpaceAdjacentTriangles[NeighborTriangleIndex * 3 + NeighborIndex];
					const int32 TextureSpaceNeighbor = TextureSpaceAdjacentTriangles[NeighborTriangleIndex * 3 + NeighborIndex];
					if (WorldSpaceNeighbor != INDEX_NONE 
						&& WorldSpaceNeighbor == TextureSpaceNeighbor
						&& ElementIndices[TriangleIndex] == ElementIndices[NeighborTriangleIndex]
						&& TriangleGroups[WorldSpaceNeighbor] == INDEX_NONE)
					{
						PendingTriangles.Add(WorldSpaceNeighbor);
					}
				}
			}
		}
	}

	TArray<TArray<int32> > GroupedTriangles;
	GroupedTriangles.Empty(NextGroupIndex);
	GroupedTriangles.AddZeroed(NextGroupIndex);
	for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
	{
		const int32 GroupIndex = TriangleGroups[TriangleIndex];
		GroupedTriangles[GroupIndex].Add(TriangleIndex);
	} 

	LayeredGroupTriangles.Add(GroupedTriangles[0]);

	// At this point many meshes will have hundreds of groups, depending on how many UV charts they have
	// Merge these groups into the same layer to be processed together if they share the same material and are not overlapping in UV space
	for (int32 GroupIndex = 1; GroupIndex < GroupedTriangles.Num(); GroupIndex++)
	{
		const int32 GroupElementIndex = ElementIndices[GroupedTriangles[GroupIndex][0]];
		bool bMergedGroup = false;
		// Search through the merged groups for one that the current group can be merged into
		for (int32 LayeredGroupIndex = 0; LayeredGroupIndex < LayeredGroupTriangles.Num() && !bMergedGroup; LayeredGroupIndex++)
		{
			const int32 LayerGroupElementIndex = ElementIndices[LayeredGroupTriangles[LayeredGroupIndex][0]];
			if (GroupElementIndex == LayerGroupElementIndex)
			{
				bool bOverlapping = false;
				for (int32 TriangleIndex = 0; TriangleIndex < GroupedTriangles[GroupIndex].Num() && !bOverlapping; TriangleIndex++)
				{
					for (int32 OtherTriangleIndex = 0; OtherTriangleIndex < LayeredGroupTriangles[LayeredGroupIndex].Num(); OtherTriangleIndex++)
					{
						const FVector2f& V0 = MeshVertices[GroupedTriangles[GroupIndex][TriangleIndex] * 3 + 0].TextureCoordinates[TextureCoordinateIndex];
						const FVector2f& V1 = MeshVertices[GroupedTriangles[GroupIndex][TriangleIndex] * 3 + 1].TextureCoordinates[TextureCoordinateIndex];
						const FVector2f& V2 = MeshVertices[GroupedTriangles[GroupIndex][TriangleIndex] * 3 + 2].TextureCoordinates[TextureCoordinateIndex];

						const FVector2f& OtherV0 = MeshVertices[LayeredGroupTriangles[LayeredGroupIndex][OtherTriangleIndex] * 3 + 0].TextureCoordinates[TextureCoordinateIndex];
						const FVector2f& OtherV1 = MeshVertices[LayeredGroupTriangles[LayeredGroupIndex][OtherTriangleIndex] * 3 + 1].TextureCoordinates[TextureCoordinateIndex];
						const FVector2f& OtherV2 = MeshVertices[LayeredGroupTriangles[LayeredGroupIndex][OtherTriangleIndex] * 3 + 2].TextureCoordinates[TextureCoordinateIndex];

						if (AxisAlignedTriangleIntersectTriangle2d(V0, V1, V2, OtherV0, OtherV1, OtherV2))
						{
							bOverlapping = true;
							break;
						}
					}
				}

				
				if (!bOverlapping)
				{
					// The current group was the same element index as the current merged group and they did not overlap in texture space, merge them
					bMergedGroup = true;
					LayeredGroupTriangles[LayeredGroupIndex].Append(GroupedTriangles[GroupIndex]);
				}
			}
		}
		if (!bMergedGroup)
		{
			// The current group did not get merged into any layers, add a new layer
			LayeredGroupTriangles.Add(GroupedTriangles[GroupIndex]);
		}
	}
}

/** Adds an entry to Texels if the given texel passes the emissive criteria. */
void FStaticLightingMesh::AddLightTexel(
	const FTexelToCornersMap& TexelToCornersMap, 
	int32 ElementIndex,
	TArray<int32>& LightIndices, 
	int32 X, int32 Y, 
	float EmissiveThreshold,
	TArray<FIntPoint>& Texels,
	int32 TexSizeX, 
	int32 TexSizeY) const
{
	if (X >= 0
		&& X < TexelToCornersMap.GetSizeX()
		&& Y >= 0
		&& Y < TexelToCornersMap.GetSizeY()
		// Only continue if this texel hasn't already been processed
		&& LightIndices[Y * TexelToCornersMap.GetSizeX() + X] == UnprocessedIndex)
	{
		const FTexelToCorners& CurrentTexelCorners = TexelToCornersMap(X, Y);
		bool bAllCornersValid = true;
		for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
		{
			bAllCornersValid = bAllCornersValid && CurrentTexelCorners.bValid[CornerIndex];
		}

		//@todo - handle partial texels
		if (bAllCornersValid)
		{
			const float XFraction = X / (float)TexSizeX;
			const float YFraction = Y / (float)TexSizeY;
			const FLinearColor CurrentEmissive = EvaluateEmissive(FVector2f(XFraction, YFraction), ElementIndex);
			if (CurrentEmissive.R > EmissiveThreshold || CurrentEmissive.G > EmissiveThreshold || CurrentEmissive.B > EmissiveThreshold)
			{
				Texels.Add(FIntPoint(X, Y));
				// Mark the texel as pending so it doesn't get added to Texels again
				LightIndices[Y * TexelToCornersMap.GetSizeX() + X] = PendingProcessingIndex;
				return;
			}
		}
		// Mark the texel as not emissive so we won't process it again
		LightIndices[Y * TexelToCornersMap.GetSizeX() + X] = NotEmissiveIndex;
	}
}

/** Adds an entry to Texels if the given texel passes the primitive simplifying criteria. */
void FStaticLightingMesh::AddPrimitiveTexel(
	const FTexelToCornersMap& TexelToCornersMap, 
	const FTexelToCorners& ComparisonTexel,
	int32 ComparisonTexelLightIndex,
	const FVector4f& PrimitiveOrigin,
	TArray<int32>& PrimitiveIndices, 
	const TArray<int32>& LightIndices, 
	int32 X, int32 Y, 
	TArray<FIntPoint>& Texels,
	const FScene& Scene,
	float DistanceThreshold) const
{
	if (X >= 0
		&& X < TexelToCornersMap.GetSizeX()
		&& Y >= 0
		&& Y < TexelToCornersMap.GetSizeY()
		// Only continue if this texel hasn't already been processed
		&& PrimitiveIndices[Y * TexelToCornersMap.GetSizeX() + X] == UnprocessedIndex)
	{
		const int32 LightIndex = LightIndices[Y * TexelToCornersMap.GetSizeX() + X];
		if (LightIndex == NotEmissiveIndex)
		{
			// Mark the texel as not emissive so we won't process it again
			PrimitiveIndices[Y * TexelToCornersMap.GetSizeX() + X] = NotEmissiveIndex;
		}
		// Only assign this texel to the primitive if its light index matches the primitive's light index
		else if (LightIndex == ComparisonTexelLightIndex)
		{
			const FTexelToCorners& CurrentTexelCorners = TexelToCornersMap(X, Y);
			FVector4f PrimitiveCenter(0,0,0);
			for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
			{
				PrimitiveCenter += CurrentTexelCorners.Corners[CornerIndex].WorldPosition / (float)NumTexelCorners;
			}

			const float NormalsDot = Dot3(CurrentTexelCorners.WorldTangentZ, ComparisonTexel.WorldTangentZ);
			const float DistanceToPrimitiveOriginSq = (PrimitiveCenter - PrimitiveOrigin).SizeSquared3();
			// Only merge into the simplified primitive if this texel's normal is similar and it is within a distance threshold
			if (NormalsDot > Scene.MeshAreaLightSettings.MeshAreaLightSimplifyNormalCosAngleThreshold && DistanceToPrimitiveOriginSq < FMath::Square(DistanceThreshold))
			{
				bool bAnyCornersMatch = false;
				for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners && !bAnyCornersMatch; CornerIndex++)
				{
					const FVector4f CurrentPosition = CurrentTexelCorners.Corners[CornerIndex].WorldPosition;
					for (int32 OtherCornerIndex = 0; OtherCornerIndex < NumTexelCorners; OtherCornerIndex++)
					{
						if ((CurrentPosition - ComparisonTexel.Corners[OtherCornerIndex].WorldPosition).SizeSquared3() < FMath::Square(Scene.MeshAreaLightSettings.MeshAreaLightSimplifyCornerDistanceThreshold))
						{
							bAnyCornersMatch = true;
							break;
						}
					}
				}

				// Only merge into the simplified primitive if any corner of this texel has the same position as the neighboring texel in the primitive
				if (bAnyCornersMatch)
				{
					Texels.Add(FIntPoint(X, Y));
					// Mark the texel as pending so it doesn't get added to Texels again
					PrimitiveIndices[Y * TexelToCornersMap.GetSizeX() + X] = PendingProcessingIndex;
					return;
				}
			}
		}
	}
}

} //namespace Lightmass
