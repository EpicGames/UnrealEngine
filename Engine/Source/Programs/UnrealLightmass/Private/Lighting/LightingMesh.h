// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ImportExport.h"
#include "Material.h"
#include "Templates/RefCounting.h"
#include "LMMath.h"
#include "Containers/IndirectArray.h"

namespace Lightmass
{
class FLight;
class FScene;
class FMeshAreaLight;
class FStaticMeshStaticLightingMesh;

/** The vertex data used to build static lighting. */
struct FStaticLightingVertex: public FStaticLightingVertexData
{
	FStaticLightingVertex() {}

	FStaticLightingVertex(const FMinimalStaticLightingVertex& InVertex)
	{
		WorldPosition = InVertex.WorldPosition;
		WorldTangentZ = InVertex.WorldTangentZ;

		for (int32 i = 0; i < UE_ARRAY_COUNT(TextureCoordinates); i++)
		{
			TextureCoordinates[i] = InVertex.TextureCoordinates[i];
		}

		GenerateVertexTangents();
	}

	/** Transforms a world space vector into the tangent space of this vertex. */
	inline FVector4f TransformWorldVectorToTangent(const FVector4f& WorldVector) const
	{
		const FVector4f TangentVector(
			Dot3(WorldTangentX, WorldVector),
			Dot3(WorldTangentY, WorldVector),
			Dot3(WorldTangentZ, WorldVector)
			);
		return TangentVector;
	}

	/** Transforms a vector in the tangent space of this vertex into world space. */
	inline FVector4f TransformTangentVectorToWorld(const FVector4f& TangentVector) const
	{
		checkSlow(TangentVector.IsUnit3());
		// Assuming the transpose of the tangent basis is also the inverse
		const FVector4f WorldTangentRow0(WorldTangentX.X, WorldTangentY.X, WorldTangentZ.X);
		const FVector4f WorldTangentRow1(WorldTangentX.Y, WorldTangentY.Y, WorldTangentZ.Y);
		const FVector4f WorldTangentRow2(WorldTangentX.Z, WorldTangentY.Z, WorldTangentZ.Z);
		const FVector4f WorldVector(
			Dot3(WorldTangentRow0, TangentVector),
			Dot3(WorldTangentRow1, TangentVector),
			Dot3(WorldTangentRow2, TangentVector)
			);
		checkSlow(WorldVector.IsUnit3());
		return WorldVector;
	}

	/** Generates WorldTangentX and WorldTangentY from WorldTangentZ such that the tangent basis is orthonormal. */
	inline void GenerateVertexTangents()
	{
		checkSlow(WorldTangentZ.IsUnit3());
		// Use the vector perpendicular to the normal and the negative Y axis as the TangentX.  
		// A WorldTangentZ of (0,0,1) will generate WorldTangentX of (1,0,0) and WorldTangentY of (0,1,0) which can be useful for debugging tangent space issues.
		const FVector4f TangentXCandidate = WorldTangentZ ^ FVector4f(0,-1,0);
		if (TangentXCandidate.SizeSquared3() < KINDA_SMALL_NUMBER)
		{
			// The normal was nearly equal to the Y axis, use the X axis instead
			WorldTangentX = (WorldTangentZ ^ FVector4f(1,0,0)).GetUnsafeNormal3();
		}
		else
		{
			WorldTangentX = TangentXCandidate.GetUnsafeNormal3();
		}
		WorldTangentY = WorldTangentZ ^ WorldTangentX;
		checkSlow(WorldTangentY.IsUnit3());
	}

	// Operators used for linear combinations of static lighting vertices.
	friend FStaticLightingVertex operator+(const FStaticLightingVertex& A,const FStaticLightingVertex& B)
	{
		FStaticLightingVertex Result;
		Result.WorldPosition =	A.WorldPosition + B.WorldPosition;
		Result.WorldTangentX =	A.WorldTangentX + B.WorldTangentX;
		Result.WorldTangentY =	A.WorldTangentY + B.WorldTangentY;
		Result.WorldTangentZ =	A.WorldTangentZ + B.WorldTangentZ;
		for(int32 CoordinateIndex = 0;CoordinateIndex < MAX_TEXCOORDS;CoordinateIndex++)
		{
			Result.TextureCoordinates[CoordinateIndex] = A.TextureCoordinates[CoordinateIndex] + B.TextureCoordinates[CoordinateIndex];
		}
		return Result;
	}

	friend FStaticLightingVertex operator-(const FStaticLightingVertex& A,const FStaticLightingVertex& B)
	{
		FStaticLightingVertex Result;
		Result.WorldPosition =	A.WorldPosition - B.WorldPosition;
		Result.WorldTangentX =	A.WorldTangentX - B.WorldTangentX;
		Result.WorldTangentY =	A.WorldTangentY - B.WorldTangentY;
		Result.WorldTangentZ =	A.WorldTangentZ - B.WorldTangentZ;
		for(int32 CoordinateIndex = 0;CoordinateIndex < MAX_TEXCOORDS;CoordinateIndex++)
		{
			Result.TextureCoordinates[CoordinateIndex] = A.TextureCoordinates[CoordinateIndex] - B.TextureCoordinates[CoordinateIndex];
		}
		return Result;
	}

	friend FStaticLightingVertex operator*(const FStaticLightingVertex& A,float B)
	{
		FStaticLightingVertex Result;
		Result.WorldPosition =	A.WorldPosition * B;
		Result.WorldTangentX =	A.WorldTangentX * B;
		Result.WorldTangentY =	A.WorldTangentY * B;
		Result.WorldTangentZ =	A.WorldTangentZ * B;
		for(int32 CoordinateIndex = 0;CoordinateIndex < MAX_TEXCOORDS;CoordinateIndex++)
		{
			Result.TextureCoordinates[CoordinateIndex] = A.TextureCoordinates[CoordinateIndex] * B;
		}
		return Result;
	}

	friend FStaticLightingVertex operator/(const FStaticLightingVertex& A,float B)
	{
		const float InvB = 1.0f / B;

		FStaticLightingVertex Result;
		Result.WorldPosition =	A.WorldPosition * InvB;
		Result.WorldTangentX =	A.WorldTangentX * InvB;
		Result.WorldTangentY =	A.WorldTangentY * InvB;
		Result.WorldTangentZ =	A.WorldTangentZ * InvB;
		for(int32 CoordinateIndex = 0;CoordinateIndex < MAX_TEXCOORDS;CoordinateIndex++)
		{
			Result.TextureCoordinates[CoordinateIndex] = A.TextureCoordinates[CoordinateIndex] * InvB;
		}
		return Result;
	}
};

/** 
 * A vertex for static lighting that contains a tangent space around the triangle normal. 
 * This is useful for generating rays from a tangent space sample set, because the smoothed normal will produce samples that self-intersect even on a plane.
 */
struct FFullStaticLightingVertex : public FStaticLightingVertex
{
	FVector4f TriangleTangentX;
	FVector4f TriangleTangentY;
	FVector4f TriangleNormal;

	/** Transforms a world space vector into the tangent space of this triangle. */
	inline FVector4f TransformWorldVectorToTriangleTangent(const FVector4f& WorldVector) const
	{
		const FVector4f TangentVector(
			Dot3(TriangleTangentX, WorldVector),
			Dot3(TriangleTangentY, WorldVector),
			Dot3(TriangleNormal, WorldVector)
			);
		return TangentVector;
	}

	/** Transforms a vector in the tangent space of this triangle into world space. */
	inline FVector4f TransformTriangleTangentVectorToWorld(const FVector4f& TriangleTangentVector) const
	{
		checkSlow(TriangleTangentVector.IsUnit3());
		// Assuming the transpose of the tangent basis is also the inverse
		const FVector4f WorldTangentRow0(TriangleTangentX.X, TriangleTangentY.X, TriangleNormal.X);
		const FVector4f WorldTangentRow1(TriangleTangentX.Y, TriangleTangentY.Y, TriangleNormal.Y);
		const FVector4f WorldTangentRow2(TriangleTangentX.Z, TriangleTangentY.Z, TriangleNormal.Z);
		const FVector4f WorldVector(
			Dot3(WorldTangentRow0, TriangleTangentVector),
			Dot3(WorldTangentRow1, TriangleTangentVector),
			Dot3(WorldTangentRow2, TriangleTangentVector)
			);
		checkSlow(WorldVector.IsUnit3());
		return WorldVector;
	}

	/** Generates TriangleTangentX and TriangleTangentY from TriangleNormal such that the tangent basis is orthonormal. */
	inline void GenerateTriangleTangents()
	{
		checkSlow(TriangleNormal.IsUnit3());
		// Use the vector perpendicular to the normal and the negative Y axis as the TangentX.  
		// A TriangleNormal of (0,0,1) will generate TriangleTangentX of (1,0,0) and TriangleTangentY of (0,1,0) which can be useful for debugging tangent space issues.
		const FVector4f TangentXCandidate = TriangleNormal ^ FVector4f(0,-1,0);
		if (TangentXCandidate.SizeSquared3() < KINDA_SMALL_NUMBER)
		{
			// The normal was nearly equal to the Y axis, use the X axis instead
			TriangleTangentX = (TriangleNormal ^ FVector4f(1,0,0)).GetUnsafeNormal3();
		}
		else
		{
			TriangleTangentX = TangentXCandidate.GetUnsafeNormal3();
		}
		TriangleTangentY = TriangleNormal ^ TriangleTangentX;
		checkSlow(TriangleTangentY.IsUnit3());
	}

	void ApplyVertexModifications(int32 ElementIndex, bool bUseNormalMapsForLighting, const class FStaticLightingMesh* Mesh);

	inline void ComputePathDirections(const FVector4f& TriangleTangentPathDirection, FVector4f& WorldPathDirection, FVector4f& TangentPathDirection) const
	{
		checkSlow(TriangleTangentPathDirection.Z >= 0.0f);
		checkSlow(TriangleTangentPathDirection.IsUnit3());

		// Generate the uniform hemisphere samples from a hemisphere based around the triangle normal, not the smoothed vertex normal
		// This is important for cases where the smoothed vertex normal is very different from the triangle normal, in which case
		// Using the smoothed vertex normal would cause self-intersection even on a plane
		WorldPathDirection = TransformTriangleTangentVectorToWorld(TriangleTangentPathDirection);
		checkSlow(WorldPathDirection.IsUnit3());

		TangentPathDirection = TransformWorldVectorToTangent(WorldPathDirection);
		checkSlow(TangentPathDirection.IsUnit3());
	}
};

/** The result of an intersection between a light ray and the scene. */
class FLightRayIntersection
{
public:

	/** true if the light ray intersected opaque scene geometry. */
	uint32 bIntersects : 1;

	/** The differential geometry which the light ray intersected with, only valid if the ray intersected. */
	FMinimalStaticLightingVertex IntersectionVertex;

	/** Transmission of the ray, valid whether the ray intersected or not as long as Transmission was requested from FStaticLightingAggregateMesh::IntersectLightRay. */
	FLinearColor Transmission;

	/** The mesh that was intersected by the ray, only valid if the ray intersected. */
	const class FStaticLightingMesh* Mesh;

	/** The mapping that was intersected by the ray, only valid if the ray intersected. */
	const class FStaticLightingMapping* Mapping;

	/** Primitive type specific element index associated with the triangle that was hit, only valid if the ray intersected. */
	int32 ElementIndex;

	/** Dummy constructor, not initializing any members. */
	FLightRayIntersection()
	{}

	/** Initialization constructor. */
	FLightRayIntersection(
		bool bInIntersects, 
		const FMinimalStaticLightingVertex& InIntersectionVertex, 
		const FStaticLightingMesh* InMesh, 
		const FStaticLightingMapping* InMapping,
		int32 InElementIndex)
		:
		bIntersects(bInIntersects),
		IntersectionVertex(InIntersectionVertex),
		Mesh(InMesh),
		Mapping(InMapping),
		ElementIndex(InElementIndex)
	{
		checkSlow(!bInIntersects || (InMesh && /*InMapping &&*/ ElementIndex >= 0));
	}

	/** No intersection constructor. */
	static FLightRayIntersection None() { return FLightRayIntersection(false,FMinimalStaticLightingVertex(),NULL,NULL,INDEX_NONE); }
};

/** Stores information about an element of the mesh which can have its own material. */
class FMaterialElement : public FMaterialElementData
{
public:
	/** Whether Material has transmission, cached here to avoid dereferencing Material. */
	bool bTranslucent;
	/** Whether Material is Masked, cached here to avoid dereferencing Material. */
	bool bIsMasked;
	/** 
	 * Whether Material is TwoSided, cached here to avoid dereferencing Material. 
	 * This is different from FMaterialElementData::bUseTwoSidedLighting, because a two sided material may still want to use one sided lighting for the most part.
	 * It just indicates whether backfaces will be visible, and therefore artifacts on backfaces should be avoided.
	 */
	bool bIsTwoSided;
	/** Whether Material is a thin surface or not. */
	bool bIsThinSurface;
	/** Whether Material wants to cast shadows as masked, cached here to avoid dereferencing Material. */
	bool bCastShadowAsMasked;
	bool bSurfaceDomain;
	/** The material associated with this element.  After import, Material is always valid (non-null and points to an FMaterial). */
	FMaterial* Material;

	FMaterialElement() :
		bTranslucent(false),
		bIsMasked(false),
		bIsTwoSided(false),
		bIsThinSurface(false),
		bCastShadowAsMasked(false),
		bSurfaceDomain(true),
		Material(NULL)
	{}
};

/** A mesh which is used for computing static lighting. */
class FStaticLightingMesh : public virtual FRefCountedObject, public FStaticLightingMeshInstanceData
{
public:
	/** The lights which affect the mesh's primitive. */
	TArray<FLight*> RelevantLights;

	/** 
	 * Visibility Id's corresponding to this static lighting mesh.  
	 * Has to be an array because BSP exports FStaticLightingMesh's per combined group of surfaces that should be lit together, 
	 * Instead of per-component geometry that should be visibility culled together.
	 */
	TArray<int32> VisibilityIds;

protected:

	/** Whether to color texels whose lightmap UV's are invalid. */
	bool bColorInvalidTexels;

	/** Indicates whether DebugDiffuse should override the materials associated with this mesh. */
	bool bUseDebugMaterial;
	FLinearColor DebugDiffuse;

	/** 
	 * Materials used by the mesh, guaranteed to contain at least one. 
	 * These are indexed by the primitive type specific ElementIndex.
	 */
	TArray<FMaterialElement, TInlineAllocator<5>> MaterialElements;

	/** 
	 * Map from FStaticLightingMesh to the index given to uniquely identify all instances of the same primitive component.
	 * This is used to give all LOD's of the same primitive component the same mesh index.
	 */
	static TMap<FStaticLightingMesh*, int32> MeshToIndexMap;

public:

	virtual FStaticMeshStaticLightingMesh* GetInstanceableStaticMesh() { return nullptr; }

	virtual const FStaticMeshStaticLightingMesh* GetInstanceableStaticMesh() const { return nullptr; }

	/** Virtual destructor. */
	virtual ~FStaticLightingMesh() {}

	/** Returns whether the given element index is translucent. */
	inline bool IsTranslucent(int32 ElementIndex) const { return MaterialElements[ElementIndex].bTranslucent; }
	/** Returns whether the given element index is masked. */
	inline bool IsMasked(int32 ElementIndex) const { return MaterialElements[ElementIndex].bIsMasked; }
	/** Whether samples using the given element accept lighting from both sides of the triangle. */
	inline bool UsesTwoSidedLighting(int32 ElementIndex) const { return MaterialElements[ElementIndex].bUseTwoSidedLighting; }
	/** Whether samples using the given element are going to have backfaces visible, and therefore artifacts on backfaces should be avoided. */
	inline bool IsTwoSided(int32 ElementIndex) const { return MaterialElements[ElementIndex].bIsTwoSided || MaterialElements[ElementIndex].bUseTwoSidedLighting; }
	inline bool IsThinSurface(int32 ElementIndex) const { return MaterialElements[ElementIndex].bIsThinSurface; }
	inline bool IsCastingShadowsAsMasked(int32 ElementIndex) const { return MaterialElements[ElementIndex].bCastShadowAsMasked; }
	inline bool IsSurfaceDomain(int32 ElementIndex) const { return MaterialElements[ElementIndex].bSurfaceDomain; }
	inline bool IsCastingShadowAsTwoSided() const { return bCastShadowAsTwoSided; }
	inline bool IsEmissive(int32 ElementIndex) const { return MaterialElements[ElementIndex].bUseEmissiveForStaticLighting; }
	inline bool IsIndirectlyShadowedOnly(int32 ElementIndex) const { return MaterialElements[ElementIndex].bShadowIndirectOnly; }
	inline float GetFullyOccludedSamplesFraction(int32 ElementIndex) const { return MaterialElements[ElementIndex].FullyOccludedSamplesFraction; }
	inline int32 GetNumElements() const { return MaterialElements.Num(); }
	inline bool ShouldColorInvalidTexels() const { return bColorInvalidTexels; }
	inline bool HasImportedNormal(int32 ElementIndex) const { return MaterialElements[ElementIndex].Material->NormalSize > 0; }
	inline bool UseVertexNormalForHemisphereGather(int32 ElementIndex) const { return MaterialElements[ElementIndex].bUseVertexNormalForHemisphereGather; }

	/**
	 *	Returns the Guid for the object associated with this lighting mesh.
	 *	Ie, for a StaticMeshStaticLightingMesh, it would return the Guid of the source static mesh.
	 *	The GetObjectType function should also be used to determine the TypeId of the source object.
	 */
	virtual FGuid GetObjectGuid() const { return FGuid(0,0,0,0); }

	/**
	 *	Returns the SourceObject type id.
	 */
	virtual ESourceObjectType GetObjectType() const { return SOURCEOBJECTTYPE_Unknown; }

	/**
	 * Accesses a triangle for visibility testing.
	 * @param TriangleIndex - The triangle to access, valid range is [0, NumTriangles).
	 * @param OutV0 - Upon return, should contain the first vertex of the triangle.
	 * @param OutV1 - Upon return, should contain the second vertex of the triangle.
	 * @param OutV2 - Upon return, should contain the third vertex of the triangle.
	 * @param ElementIndex - Indicates the element index of the triangle.
     */
	virtual void GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const = 0;

	/**
	 * Accesses a triangle for shading.
	 * @param TriangleIndex - The triangle to access, valid range is [0, NumShadingTriangles).
	 * @param OutV0 - Upon return, should contain the first vertex of the triangle.
	 * @param OutV1 - Upon return, should contain the second vertex of the triangle.
	 * @param OutV2 - Upon return, should contain the third vertex of the triangle.
	 * @param ElementIndex - Indicates the element index of the triangle.
     */
	virtual void GetShadingTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const
	{
		checkSlow(NumTriangles == NumShadingTriangles);
		// By default the geometry used for shading is the same as the geometry used for visibility testing.
		GetTriangle(TriangleIndex, OutV0, OutV1, OutV2, ElementIndex);
	}

	/**
	 * Accesses a triangle's vertex indices for visibility testing.
	 * @param TriangleIndex - The triangle to access, valid range is [0, NumTriangles).
	 * @param OutI0 - Upon return, should contain the first vertex index of the triangle.
	 * @param OutI1 - Upon return, should contain the second vertex index of the triangle.
	 * @param OutI2 - Upon return, should contain the third vertex index of the triangle.
	 */
	virtual void GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const = 0;

	/**
	 * Accesses a triangle's vertex indices for shading.
	 * @param TriangleIndex - The triangle to access, valid range is [0, NumShadingTriangles).
	 * @param OutI0 - Upon return, should contain the first vertex index of the triangle.
	 * @param OutI1 - Upon return, should contain the second vertex index of the triangle.
	 * @param OutI2 - Upon return, should contain the third vertex index of the triangle.
	 */
	virtual void GetShadingTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const
	{ 
		checkSlow(NumTriangles == NumShadingTriangles);
		// By default the geometry used for shading is the same as the geometry used for visibility testing.
		GetTriangleIndices(TriangleIndex, OutI0, OutI1, OutI2);
	}

	virtual bool IsElementCastingShadow(int32 ElementIndex) const 
	{ return true; }

	/** Returns the LOD of this instance. */
	virtual uint32 GetLODIndices() const { return 0; }
	virtual uint32 GetHLODRange() const { return 0; }
	bool DoesMeshBelongToLOD0() const;

	/** For debugging */
	virtual void SetDebugMaterial(bool bUseDebugMaterial, FLinearColor Diffuse);

	/** 
	 * Whether mesh is always opaque for visibility calculations, 
	 * otherwise opaque property will be checked for each triangle 
	 */
	virtual bool IsAlwaysOpaqueForVisibility() const { return false; }

	/** Evaluates the mesh's Bidirectional Reflectance Distribution Function. */
	FLinearColor EvaluateBRDF(
		const FStaticLightingVertex& Vertex, 
		int32 ElementIndex,
		const FVector4f& IncomingDirection, 
		const FVector4f& OutgoingDirection) const;

	/** Generates an outgoing direction sample and evaluates the BRDF for that direction. */
	FLinearColor SampleBRDF(
		const FStaticLightingVertex& Vertex, 
		int32 ElementIndex,
		const FVector4f& IncomingDirection, 
		FVector4f& OutgoingDirection,
		float& DirectionPDF,
		FLMRandomStream& RandomStream
		) const;

	/** Evaluates the mesh's emissive at the given UVs */
	inline FLinearColor EvaluateEmissive(const FVector2f& UVs, int32 ElementIndex) const
	{
		checkSlow(IsEmissive(ElementIndex)); 
		FLinearColor Emissive(FLinearColor::Black);
		float MaterialEmissiveBoost;
		const FMaterialElement& MaterialElement = MaterialElements[ElementIndex];
		MaterialElement.Material->SampleEmissive(UVs, Emissive, MaterialEmissiveBoost);
		FLinearColor EmissiveXYZ = FLinearColorUtils::LinearRGBToXYZ(Emissive);
		FLinearColor EmissivexyzY = FLinearColorUtils::XYZToxyzY(EmissiveXYZ);
		// Apply EmissiveBoost to the emissive brightness, which is Y in xyzY
		// Modifying brightness in xyzY to be consistent with DiffuseBoost
		EmissivexyzY.A = EmissivexyzY.A * MaterialEmissiveBoost * MaterialElement.EmissiveBoost;
		EmissiveXYZ = FLinearColorUtils::xyzYToXYZ(EmissivexyzY);
		Emissive = FLinearColorUtils::XYZToLinearRGB(EmissiveXYZ);
		return Emissive;
	}

	/** Evaluates the mesh's diffuse at the given UVs */
	inline FLinearColor EvaluateDiffuse(const FVector2f& UVs, int32 ElementIndex) const
	{
		checkSlow(!IsTranslucent(ElementIndex));
		FLinearColor Diffuse(DebugDiffuse);
		if (!bUseDebugMaterial)
		{
			float MaterialDiffuseBoost;
			const FMaterialElement& MaterialElement = MaterialElements[ElementIndex];
			MaterialElement.Material->SampleDiffuse(UVs, Diffuse, MaterialDiffuseBoost);
			Diffuse.R = FMath::Max(Diffuse.R, 0.0f);
			Diffuse.G = FMath::Max(Diffuse.G, 0.0f);
			Diffuse.B = FMath::Max(Diffuse.B, 0.0f);
			FLinearColor DiffuseXYZ = FLinearColorUtils::LinearRGBToXYZ(Diffuse);
			FLinearColor DiffusexyzY = FLinearColorUtils::XYZToxyzY(DiffuseXYZ);
			// Apply DiffuseBoost to the diffuse brightness, which is Y in xyzY
			// Using xyzY allows us to modify the brightness of the color without changing the hue
			// Clamp diffuse to be physically valid for the modified Phong lighting model
			DiffusexyzY.A = FMath::Min(DiffusexyzY.A * MaterialDiffuseBoost * MaterialElement.DiffuseBoost, 1.0f);
			DiffuseXYZ = FLinearColorUtils::xyzYToXYZ(DiffusexyzY);
			Diffuse = FLinearColorUtils::XYZToLinearRGB(DiffuseXYZ);
		}
		return Diffuse;
	}

	/** Evaluates the mesh's transmission at the given UVs */
	inline FLinearColor EvaluateTransmission(const FVector2f& UVs, int32 ElementIndex) const
	{
		checkSlow(IsTranslucent(ElementIndex));
		FLinearColor Transmission = MaterialElements[ElementIndex].Material->SampleTransmission(UVs);
		Transmission.R = FMath::Max(Transmission.R, 0.0f);
		Transmission.G = FMath::Max(Transmission.G, 0.0f);
		Transmission.B = FMath::Max(Transmission.B, 0.0f);
		return Transmission;
	}

	/** Evaluates the mesh's transmission at the given UVs */
	inline bool EvaluateMaskedCollision(const FVector2f& UVs, int32 ElementIndex) const
	{
		checkSlow(IsMasked(ElementIndex) || IsCastingShadowsAsMasked(ElementIndex));
		const FMaterialElement& MaterialElement = MaterialElements[ElementIndex];
		const float MaskClipValue = MaterialElement.Material->OpacityMaskClipValue;
		const float OpacityMask = MaterialElement.Material->SampleTransmission(UVs).R;
		return OpacityMask > MaskClipValue;
	}

	/** Evaluates the mesh's tangent space normal at the given UVs */
	inline FVector4f EvaluateNormal(const FVector2f& UVs, int32 ElementIndex) const
	{
		FVector4f Normal( 0, 0, 1.0f, 0.0 );
		const FMaterialElement& MaterialElement = MaterialElements[ElementIndex];
		if( MaterialElement.Material->NormalSize > 0 )
		{
			MaterialElement.Material->SampleNormal(UVs, Normal);
		}
		return Normal;
	}

	/** 
	 * Returns the hemispherical-hemispherical reflectance, 
	 * Which is the fraction of light that is reflected in any direction when the incident light is constant over all directions of the hemisphere.
	 * This value is used to calculate exitant radiance, which is 1 / PI * HemisphericalHemisphericalReflectance * Irradiance, disregarding directional variation.
     */
	inline FLinearColor EvaluateTotalReflectance(const FMinimalStaticLightingVertex& Vertex, int32 ElementIndex) const
	{
		return EvaluateDiffuse(Vertex.TextureCoordinates[0], ElementIndex);
	}

	virtual void Import( class FLightmassImporter& Importer );

	/** Allows the mesh to create mesh area lights from its emissive contribution */
	void CreateMeshAreaLights(const class FStaticLightingSystem& LightingSystem, const FScene& Scene, TIndirectArray<FMeshAreaLight>& MeshAreaLights) const;

private:

	/** Splits a mesh into layers with non-overlapping UVs, maintaining adjacency in world space and UVs. */
	void CalculateUniqueLayers(const TArray<FStaticLightingVertex>& MeshVertices, const TArray<int32>& ElementIndices, TArray<TArray<int32> >& LayeredGroupTriangles) const;

	/** Adds an entry to Texels if the given texel passes the emissive criteria. */
	void AddLightTexel(
		const class FTexelToCornersMap& TexelToCornersMap, 
		int32 ElementIndex,
		TArray<int32>& LightIndices, 
		int32 X, int32 Y, 
		float EmissiveThreshold,
		TArray<FIntPoint>& Texels,
		int32 TexSizeX,
		int32 TexSizeY) const;

	/** Adds an entry to Texels if the given texel passes the primitive simplifying criteria. */
	void AddPrimitiveTexel(
		const FTexelToCornersMap& TexelToCornersMap, 
		const struct FTexelToCorners& ComparisonTexel,
		int32 ComparisonTexelLightIndex,
		const FVector4f& PrimitiveOrigin,
		TArray<int32>& PrimitiveIndices, 
		const TArray<int32>& LightIndices, 
		int32 X, int32 Y, 
		TArray<FIntPoint>& Texels,
		const FScene& Scene,
		float DistanceThreshold) const;
};


} //namespace Lightmass
