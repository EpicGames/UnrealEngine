// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightingSystem.h"
#include "Exporter.h"
#include "LightmassSwarm.h"
#include "CPUSolver.h"
#include "MonteCarlo.h"
#include "Misc/ScopeLock.h"
#include "UnrealLightmass.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/ExceptionHandling.h"
#if USE_LOCAL_SWARM_INTERFACE
#include "IMessagingModule.h"
#include "Async/TaskGraphInterfaces.h"
#endif

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
	#include <psapi.h>
#include "Windows/HideWindowsPlatformTypes.h"

#pragma comment(lib, "psapi.lib")
#endif

namespace Lightmass
{
static void ConvertToLightSampleHelper(const FGatheredLightSample& InGatheredLightSample, float OutCoefficients[2][3])
{
	// SHCorrection is SHVector sampled with the normal
	float DirCorrection = 1.0f / FMath::Max( 0.0001f, InGatheredLightSample.SHCorrection );
	float DirLuma[4];
	for( int32 i = 0; i < 4; i++ )
	{
		DirLuma[i]  = 0.30f * InGatheredLightSample.SHVector.R.V[i];
		DirLuma[i] += 0.59f * InGatheredLightSample.SHVector.G.V[i];
		DirLuma[i] += 0.11f * InGatheredLightSample.SHVector.B.V[i];

		// Lighting is already in IncidentLighting. Force directional SH as applied to a flat normal map to be 1 to get purely directional data.
		DirLuma[i] *= DirCorrection / PI;
	}

	// Scale directionality so that DirLuma[0] == 1. Then scale color to compensate and toss DirLuma[0].
	float DirScale = 1.0f / FMath::Max( 0.0001f, DirLuma[0] );
	float ColorScale = DirLuma[0];

	// IncidentLighting is ground truth for a representative direction, the vertex normal
	OutCoefficients[0][0] = ColorScale * InGatheredLightSample.IncidentLighting.R;
	OutCoefficients[0][1] = ColorScale * InGatheredLightSample.IncidentLighting.G;
	OutCoefficients[0][2] = ColorScale * InGatheredLightSample.IncidentLighting.B;

	// Will force DirLuma[0] to 0.282095f
	OutCoefficients[1][0] = -0.325735f * DirLuma[1] * DirScale;
	OutCoefficients[1][1] =  0.325735f * DirLuma[2] * DirScale;
	OutCoefficients[1][2] = -0.325735f * DirLuma[3] * DirScale;
}

FLightSample FGatheredLightMapSample::ConvertToLightSample(bool bDebugThisSample) const
{
	FLightSample NewSample;
	NewSample.bIsMapped = bIsMapped;

	ConvertToLightSampleHelper(HighQuality, &NewSample.Coefficients[0]);
	ConvertToLightSampleHelper(LowQuality, &NewSample.Coefficients[ LM_LQ_LIGHTMAP_COEF_INDEX ]);

	NewSample.SkyOcclusion[0] = HighQuality.SkyOcclusion.X;
	NewSample.SkyOcclusion[1] = HighQuality.SkyOcclusion.Y;
	NewSample.SkyOcclusion[2] = HighQuality.SkyOcclusion.Z;

	NewSample.AOMaterialMask = HighQuality.AOMaterialMask;

	return NewSample;
}

FLightMapData2D* FGatheredLightMapData2D::ConvertToLightmap2D(bool bDebugThisMapping, int32 PaddedDebugX, int32 PaddedDebugY) const
{
	FLightMapData2D* ConvertedLightMap = new FLightMapData2D(SizeX, SizeY);
	ConvertedLightMap->Lights = Lights;
	ConvertedLightMap->bHasSkyShadowing = bHasSkyShadowing;

	for (int32 SampleIndex = 0; SampleIndex < Data.Num(); SampleIndex++)
	{
		const bool bDebugThisSample = bDebugThisMapping && SampleIndex == PaddedDebugY * SizeX + PaddedDebugX;
		(*ConvertedLightMap)(SampleIndex, 0) = Data[SampleIndex].ConvertToLightSample(bDebugThisSample);
	}
	return ConvertedLightMap;
}

FStaticLightingMappingContext::FStaticLightingMappingContext(const FStaticLightingMesh* InSubjectMesh, FStaticLightingSystem& InSystem, FDebugLightingOutput* InDebugOutput) :
	FirstBounceCache(InSubjectMesh ? InSubjectMesh->BoundingBox : FBox3f::BuildAABB(FVector4f(0,0,0), FVector4f(HALF_WORLD_MAX)), InSystem, 1),
	System(InSystem),
	DebugOutput(InDebugOutput)
{}

FStaticLightingMappingContext::~FStaticLightingMappingContext()
{
	{
		// Update the main threads stats with the stats from this mapping
		FScopeLock Lock(&System.Stats.StatsSync);
		System.Stats.Cache[0] += FirstBounceCache.Stats;
		System.Stats += Stats;
		System.Stats.NumFirstHitRaysTraced += RayCache.NumFirstHitRaysTraced;
		System.Stats.NumBooleanRaysTraced += RayCache.NumBooleanRaysTraced;
		System.Stats.FirstHitRayTraceThreadTime += RayCache.FirstHitRayTraceTime;
		System.Stats.BooleanRayTraceThreadTime += RayCache.BooleanRayTraceTime;
	}

	for (int32 EntryIndex = 0; EntryIndex < RefinementTreeFreePool.Num(); EntryIndex++)
	{
		// Delete on the main thread to avoid a TBB inefficiency deleting many same-sized allocations on different threads
		delete RefinementTreeFreePool[EntryIndex];
	}
}

/**
 * Initializes this static lighting system, and builds static lighting based on the provided options.
 * @param InOptions		- The static lighting build options.
 * @param InScene		- The scene containing all the lights and meshes
 * @param InExporter	- The exporter used to send completed data back to UE5
 * @param InNumThreads	- Number of concurrent threads to use for lighting building
 */
FStaticLightingSystem::FStaticLightingSystem(const FLightingBuildOptions& InOptions, FScene& InScene, FLightmassSolverExporter& InExporter, int32 InNumThreads)
:	Options(InOptions)
,	GeneralSettings(InScene.GeneralSettings)
,	SceneConstants(InScene.SceneConstants)
,	MaterialSettings(InScene.MaterialSettings)
,	MeshAreaLightSettings(InScene.MeshAreaLightSettings)
,	DynamicObjectSettings(InScene.DynamicObjectSettings)
,	VolumetricLightmapSettings(InScene.VolumetricLightmapSettings)
,	PrecomputedVisibilitySettings(InScene.PrecomputedVisibilitySettings)
,	VolumeDistanceFieldSettings(InScene.VolumeDistanceFieldSettings)
,	AmbientOcclusionSettings(InScene.AmbientOcclusionSettings)
,	ShadowSettings(InScene.ShadowSettings)
,	ImportanceTracingSettings(InScene.ImportanceTracingSettings)
,	PhotonMappingSettings(InScene.PhotonMappingSettings)
,	IrradianceCachingSettings(InScene.IrradianceCachingSettings)
,	TasksInProgressThatWillNeedHelp(0)
,	NextVolumeSampleTaskIndex(-1)
,	NumVolumeSampleTasksOutstanding(0)
,	bShouldExportVolumeSampleData(false)
,	VolumeLightingInterpolationOctree(FVector4f(0,0,0), HALF_WORLD_MAX)
,	bShouldExportMeshAreaLightData(false)
,	bShouldExportVolumeDistanceField(false)
,	NumPhotonsEmittedDirect(0)
,	DirectPhotonMap(FVector4f(0,0,0), HALF_WORLD_MAX)
,	NumPhotonsEmittedFirstBounce(0)
,	FirstBouncePhotonMap(FVector4f(0,0,0), HALF_WORLD_MAX)
,	FirstBounceEscapedPhotonMap(FVector4f(0,0,0), HALF_WORLD_MAX)
,	FirstBouncePhotonSegmentMap(FVector4f(0,0,0), HALF_WORLD_MAX)
,	NumPhotonsEmittedSecondBounce(0)
,	SecondBouncePhotonMap(FVector4f(0,0,0), HALF_WORLD_MAX)
,	IrradiancePhotonMap(FVector4f(0,0,0), HALF_WORLD_MAX)
,	AggregateMesh(NULL)
,	VoxelizationSurfaceAggregateMesh(NULL)
,	VoxelizationVolumeAggregateMesh(NULL)
,	LandscapeCullingVoxelizationAggregateMesh(NULL)
,	Scene(InScene)
,	NumTexelsCompleted(0)
,	NumOutstandingVolumeDataLayers(0)
,	OutstandingVolumeDataLayerIndex(-1)
,	NumStaticLightingThreads(InScene.GeneralSettings.bAllowMultiThreadedStaticLighting ? FMath::Max(InNumThreads, 1) : 1)
,	DebugIrradiancePhotonCalculationArrayIndex(INDEX_NONE)
,	DebugIrradiancePhotonCalculationPhotonIndex(INDEX_NONE)
,	Exporter(InExporter)
{
	const double SceneSetupStart = FPlatformTime::Seconds();
	UE_LOGF(LogLightmass, Log, "FStaticLightingSystem started using GKDOPMaxTrisPerLeaf: %d", DEFAULT_MAX_TRIS_PER_LEAF);

	ValidateSettings(InScene);

	bool bDumpAllMappings = false;

	GroupVisibilityGridSizeXY = 0;
	GroupVisibilityGridSizeZ = 0;

	// Pre-allocate containers.
	int32 NumMeshes = 0;
	int32 NumVertices = 0;
	int32 NumTriangles = 0;
	int32 NumMappings = InScene.TextureLightingMappings.Num() +
		InScene.FluidMappings.Num() + InScene.LandscapeMappings.Num() + InScene.BspMappings.Num() + InScene.LandscapeVolumeMappings.Num();
	int32 NumMeshInstances = InScene.BspMappings.Num() + InScene.StaticMeshInstances.Num() + InScene.VolumeMappings.Num();
	AllMappings.Reserve( NumMappings );
	Meshes.Reserve( NumMeshInstances );

	// Initialize Meshes, Mappings, AllMappings and AggregateMesh from the scene
	UE_LOGF(LogLightmass, Log,  "Number of texture mappings: %d", InScene.TextureLightingMappings.Num() );
	for (int32 MappingIndex = 0; MappingIndex < InScene.TextureLightingMappings.Num(); MappingIndex++)
	{
		FStaticMeshStaticLightingTextureMapping* Mapping = &InScene.TextureLightingMappings[MappingIndex];
		Mappings.Add(Mapping->Guid, Mapping);
		AllMappings.Add(Mapping);
		if (bDumpAllMappings)
		{
			UE_LOGF(LogLightmass, Log, "\t%ls", *(Mapping->Guid.ToString()));
		}
	}

	UE_LOGF(LogLightmass, Log,  "Number of fluid mappings:   %d", InScene.FluidMappings.Num());
	for (int32 MappingIndex = 0; MappingIndex < InScene.FluidMappings.Num(); MappingIndex++)
	{
		FFluidSurfaceStaticLightingTextureMapping* Mapping = &InScene.FluidMappings[MappingIndex];
		NumMeshes++;
		NumVertices += Mapping->Mesh->NumVertices;
		NumTriangles += Mapping->Mesh->NumTriangles;
		Mappings.Add(Mapping->Guid, Mapping);
		AllMappings.Add(Mapping);
		if (bDumpAllMappings)
		{
			UE_LOGF(LogLightmass, Log, "\t%ls", *(Mapping->Guid.ToString()));
		}
	}

	for (int32 MeshIndex = 0; MeshIndex < InScene.FluidMeshInstances.Num(); MeshIndex++)
	{
		Meshes.Add(&InScene.FluidMeshInstances[MeshIndex]);
	}

	UE_LOGF(LogLightmass, Log,  "Number of landscape mappings:   %d", InScene.LandscapeMappings.Num());
	for (int32 MappingIndex = 0; MappingIndex < InScene.LandscapeMappings.Num(); MappingIndex++)
	{
		FLandscapeStaticLightingTextureMapping* Mapping = &InScene.LandscapeMappings[MappingIndex];
		NumMeshes++;
		NumVertices += Mapping->Mesh->NumVertices;
		NumTriangles += Mapping->Mesh->NumTriangles;
		Mappings.Add(Mapping->Guid, Mapping);
		LandscapeMappings.Add(Mapping);
		AllMappings.Add(Mapping);
		if (bDumpAllMappings)
		{
			UE_LOGF(LogLightmass, Log, "\t%ls", *(Mapping->Guid.ToString()));
		}
	}

	for (int32 MeshIndex = 0; MeshIndex < InScene.LandscapeMeshInstances.Num(); MeshIndex++)
	{
		Meshes.Add(&InScene.LandscapeMeshInstances[MeshIndex]);
	}

	UE_LOGF(LogLightmass, Log,  "Number of BSP mappings:     %d", InScene.BspMappings.Num() );
	for( int32 MeshIdx=0; MeshIdx < InScene.BspMappings.Num(); ++MeshIdx )
	{
		FBSPSurfaceStaticLighting* BSPMapping = &InScene.BspMappings[MeshIdx];
		Meshes.Add(BSPMapping);
		NumMeshes++;
		NumVertices += BSPMapping->NumVertices;
		NumTriangles += BSPMapping->NumTriangles;

		// add the BSP mappings light mapping object
		AllMappings.Add(&BSPMapping->Mapping);
		Mappings.Add(BSPMapping->Mapping.Guid, &BSPMapping->Mapping);
		if (bDumpAllMappings)
		{
			UE_LOGF(LogLightmass, Log, "\t%ls", *(BSPMapping->Mapping.Guid.ToString()));
		}
	}

	UE_LOGF(LogLightmass, Log,  "Number of volume mappings:   %d", InScene.VolumeMappings.Num());
	for (int32 MappingIndex = 0; MappingIndex < InScene.VolumeMappings.Num(); MappingIndex++)
	{
		FStaticLightingGlobalVolumeMapping* Mapping = &InScene.VolumeMappings[MappingIndex];
		Mappings.Add(Mapping->Guid, Mapping);
		AllMappings.Add(Mapping);
		if (bDumpAllMappings)
		{
			UE_LOGF(LogLightmass, Log, "\t%ls", *(Mapping->Guid.ToString()));
		}
	}

	UE_LOGF(LogLightmass, Log,  "Number of landscape volume mappings:   %d", InScene.LandscapeVolumeMappings.Num());
	for (int32 MappingIndex = 0; MappingIndex < InScene.LandscapeVolumeMappings.Num(); MappingIndex++)
	{
		FLandscapeStaticLightingGlobalVolumeMapping* Mapping = &InScene.LandscapeVolumeMappings[MappingIndex];
		Mappings.Add(Mapping->Guid, Mapping);
		AllMappings.Add(Mapping);
		LandscapeMappings.Add(Mapping);
		if (bDumpAllMappings)
		{
			UE_LOGF(LogLightmass, Log, "\t%ls", *(Mapping->Guid.ToString()));
		}
	}

	UE_LOGF(LogLightmass, Log,  "Number of static mesh instance mappings: %d", InScene.StaticMeshInstances.Num() );
	for (int32 MeshIndex = 0; MeshIndex < InScene.StaticMeshInstances.Num(); MeshIndex++)
	{
		FStaticMeshStaticLightingMesh* MeshInstance = &InScene.StaticMeshInstances[MeshIndex];
		FStaticLightingMapping** MappingPtr = Mappings.Find(MeshInstance->Guid);
		if (MappingPtr != NULL)
		{
			MeshInstance->Mapping = *MappingPtr;
		}
		else
		{
			MeshInstance->Mapping = NULL;
		}
		Meshes.Add(MeshInstance);
		NumMeshes++;
		NumVertices += MeshInstance->NumVertices;
		NumTriangles += MeshInstance->NumTriangles;
	}

	check(Meshes.Num() == AllMappings.Num());

	int32 MaxVisibilityId = -1;
	for (int32 MeshIndex = 0; MeshIndex < Meshes.Num(); MeshIndex++)
	{
		const FStaticLightingMesh* Mesh = Meshes[MeshIndex];
		for (int32 VisIndex = 0; VisIndex < Mesh->VisibilityIds.Num(); VisIndex++)
		{
			MaxVisibilityId = FMath::Max(MaxVisibilityId, Mesh->VisibilityIds[VisIndex]);
		}
	}

	VisibilityMeshes.Empty(MaxVisibilityId + 1);
	VisibilityMeshes.AddZeroed(MaxVisibilityId + 1);
	for (int32 MeshIndex = 0; MeshIndex < Meshes.Num(); MeshIndex++)
	{
		FStaticLightingMesh* Mesh = Meshes[MeshIndex];
		for (int32 VisIndex = 0; VisIndex < Mesh->VisibilityIds.Num(); VisIndex++)
		{
			const int32 VisibilityId = Mesh->VisibilityIds[VisIndex];
			if (VisibilityId >= 0)
			{
				VisibilityMeshes[VisibilityId].Meshes.AddUnique(Mesh);
			}
		}
	}

	for (int32 VisibilityMeshIndex = 0; VisibilityMeshIndex < VisibilityMeshes.Num(); VisibilityMeshIndex++)
	{
		checkSlow(VisibilityMeshes[VisibilityMeshIndex].Meshes.Num() > 0);
	}

	{
		FScopedRDTSCTimer MeshSetupTimer(Stats.MeshAreaLightSetupTime);
		for (int32 MeshIndex = 0; MeshIndex < Meshes.Num(); MeshIndex++)
		{
			const int32 BckNumMeshAreaLights = MeshAreaLights.Num();
			// Create mesh area lights from each mesh
			Meshes[MeshIndex]->CreateMeshAreaLights(*this, Scene, MeshAreaLights);
			if (MeshAreaLights.Num() > BckNumMeshAreaLights)
			{
				Stats.NumMeshAreaLightMeshes++;
			}
			Meshes[MeshIndex]->SetDebugMaterial(MaterialSettings.bUseDebugMaterial, MaterialSettings.DebugDiffuse);
		}
	}

	for (int32 MeshIndex = 0; MeshIndex < Meshes.Num(); MeshIndex++)
	{
		for (int32 LightIndex = 0; LightIndex < MeshAreaLights.Num(); LightIndex++)
		{
			// Register the newly created mesh area lights with every relevant mesh so they are used for lighting
			if (MeshAreaLights[LightIndex].AffectsBounds(FBoxSphereBounds3f(Meshes[MeshIndex]->BoundingBox)))
			{
				Meshes[MeshIndex]->RelevantLights.Add(&MeshAreaLights[LightIndex]);
			}
		}
	}

#if USE_EMBREE	
	if (Scene.EmbreeDevice)
	{
		if (Scene.bVerifyEmbree)
		{
			AggregateMesh = new FEmbreeVerifyAggregateMesh(Scene);
		}
		else
		{
			AggregateMesh = new FEmbreeAggregateMesh(Scene);
		}
	}
	else
#endif
	{
		AggregateMesh = new FDefaultAggregateMesh(Scene);
	}
	// Add all meshes to the kDOP.
	AggregateMesh->ReserveMemory(NumMeshes, NumVertices, NumTriangles);

	if (Scene.GeneralSettings.bUseFastVoxelization)
	{
		VoxelizationSurfaceAggregateMesh = new FDefaultAggregateMesh(Scene);
		VoxelizationVolumeAggregateMesh = new FDefaultAggregateMesh(Scene);
		LandscapeCullingVoxelizationAggregateMesh = new FDefaultAggregateMesh(Scene);
	}

	for (int32 MappingIndex = 0; MappingIndex < InScene.FluidMappings.Num(); MappingIndex++)
	{
		FFluidSurfaceStaticLightingTextureMapping* Mapping = &InScene.FluidMappings[MappingIndex];
		AggregateMesh->AddMesh(Mapping->Mesh, Mapping);

		if (Scene.GeneralSettings.bUseFastVoxelization)
		{
			if (Mapping->GetVolumeMapping() == nullptr)
			{
				VoxelizationSurfaceAggregateMesh->AddMeshForVoxelization(Mapping->Mesh, Mapping);
			}
			else
			{
				VoxelizationVolumeAggregateMesh->AddMeshForVoxelization(Mapping->Mesh, Mapping);
			}
		}
	}
	for (int32 MappingIndex = 0; MappingIndex < LandscapeMappings.Num(); MappingIndex++)
	{
		FStaticLightingMapping* Mapping = LandscapeMappings[MappingIndex];
		AggregateMesh->AddMesh(Mapping->Mesh, Mapping);

		if (Scene.GeneralSettings.bUseFastVoxelization)
		{
			if (Mapping->GetLandscapeVolumeMapping() == nullptr)
			{
				VoxelizationSurfaceAggregateMesh->AddMeshForVoxelization(Mapping->Mesh, Mapping);
			}
			else
			{
				VoxelizationVolumeAggregateMesh->AddMeshForVoxelization(Mapping->Mesh, Mapping);
			}

			LandscapeCullingVoxelizationAggregateMesh->AddMeshForVoxelization(Mapping->Mesh, Mapping);
		}
	}
	for( int32 MeshIdx=0; MeshIdx < InScene.BspMappings.Num(); ++MeshIdx )
	{
		FBSPSurfaceStaticLighting* BSPMapping = &InScene.BspMappings[MeshIdx];
		AggregateMesh->AddMesh(BSPMapping, &BSPMapping->Mapping);

		if (Scene.GeneralSettings.bUseFastVoxelization)
		{
			if (BSPMapping->Mapping.GetVolumeMapping() == nullptr)
			{
				VoxelizationSurfaceAggregateMesh->AddMeshForVoxelization(BSPMapping, &BSPMapping->Mapping);
			}
			else
			{
				VoxelizationVolumeAggregateMesh->AddMeshForVoxelization(BSPMapping, &BSPMapping->Mapping);
			}
		}
	}
	for (int32 MeshIndex = 0; MeshIndex < InScene.StaticMeshInstances.Num(); MeshIndex++)
	{
		FStaticMeshStaticLightingMesh* MeshInstance = &InScene.StaticMeshInstances[MeshIndex];
		AggregateMesh->AddMesh(MeshInstance, MeshInstance->Mapping);

		if (Scene.GeneralSettings.bUseFastVoxelization)
		{
			if (MeshInstance->GetInstanceableStaticMesh() != nullptr)
			{
				if (MeshInstance->StaticMesh->VoxelizationMesh == nullptr)
				{
					if (MeshInstance->LightingFlags & GI_INSTANCE_CASTSHADOW && MeshInstance->DoesMeshBelongToLOD0())
					{
						MeshInstance->StaticMesh->VoxelizationMesh = new FDefaultAggregateMesh(Scene);
						MeshInstance->StaticMesh->VoxelizationMesh->AddMeshForVoxelization(MeshInstance, MeshInstance->Mapping, true);
						MeshInstance->StaticMesh->VoxelizationMesh->PrepareForRaytracing();
					}
				}
			}
			else
			{
				// For non-instanceable static meshes (splines), add them to the aggregate scene voxelization mesh
				if (MeshInstance->Mapping->GetVolumeMapping() == nullptr)
				{
					VoxelizationSurfaceAggregateMesh->AddMeshForVoxelization(MeshInstance, MeshInstance->Mapping);
				}
				else
				{
					VoxelizationVolumeAggregateMesh->AddMeshForVoxelization(MeshInstance, MeshInstance->Mapping);
				}
			}
		}
	}

	// Comparing mappings based on cost, descending.
	struct FCompareProcessingCost
	{
		FORCEINLINE bool operator()( const FStaticLightingMapping& A, const FStaticLightingMapping& B ) const
		{
			return B.GetProcessingCost() < A.GetProcessingCost();
		}
	};
	// Sort mappings by processing cost, descending.
	Mappings.ValueSort(FCompareProcessingCost());
	AllMappings.Sort(FCompareProcessingCost());

	GStatistics.NumTotalMappings = Mappings.Num();

	const FBoxSphereBounds3f SceneBounds = FBoxSphereBounds3f(AggregateMesh->GetBounds());
	const FBoxSphereBounds3f ImportanceBounds = GetImportanceBounds();
	// Never trace further than the importance or scene diameter
	MaxRayDistance = ImportanceBounds.SphereRadius > 0.0f ? ImportanceBounds.SphereRadius * 2.0f : SceneBounds.SphereRadius * 2.0f;

	Stats.NumLights = InScene.DirectionalLights.Num() + InScene.PointLights.Num() + InScene.SpotLights.Num() + InScene.RectLights.Num() + MeshAreaLights.Num();
	Stats.NumMeshAreaLights = MeshAreaLights.Num();
	for (int32 i = 0; i < MeshAreaLights.Num(); i++)
	{
		Stats.NumMeshAreaLightPrimitives += MeshAreaLights[i].GetNumPrimitives();
		Stats.NumSimplifiedMeshAreaLightPrimitives += MeshAreaLights[i].GetNumSimplifiedPrimitives();
	}

	// Add all light types except sky lights to the system's Lights array
	Lights.Reserve(Stats.NumLights);
	for (int32 LightIndex = 0; LightIndex < InScene.DirectionalLights.Num(); LightIndex++)
	{
		InScene.DirectionalLights[LightIndex].Initialize(
			SceneBounds, 
			PhotonMappingSettings.bEmitPhotonsOutsideImportanceVolume,
			ImportanceBounds,
			Scene.PhotonMappingSettings.IndirectPhotonEmitDiskRadius,
			Scene.SceneConstants.LightGridSize,
			Scene.PhotonMappingSettings.DirectPhotonDensity,
			Scene.PhotonMappingSettings.DirectPhotonDensity * Scene.PhotonMappingSettings.OutsideImportanceVolumeDensityScale);

		Lights.Add(&InScene.DirectionalLights[LightIndex]);
	}
	
	// Initialize lights and add them to the solver's Lights array
	for (int32 LightIndex = 0; LightIndex < InScene.PointLights.Num(); LightIndex++)
	{
		InScene.PointLights[LightIndex].Initialize(Scene.PhotonMappingSettings.IndirectPhotonEmitConeAngle);
		Lights.Add(&InScene.PointLights[LightIndex]);
	}

	for (int32 LightIndex = 0; LightIndex < InScene.SpotLights.Num(); LightIndex++)
	{
		InScene.SpotLights[LightIndex].Initialize(Scene.PhotonMappingSettings.IndirectPhotonEmitConeAngle);
		Lights.Add(&InScene.SpotLights[LightIndex]);
	}

	for (int32 LightIndex = 0; LightIndex < InScene.RectLights.Num(); LightIndex++)
	{
		InScene.RectLights[LightIndex].Initialize(Scene.PhotonMappingSettings.IndirectPhotonEmitConeAngle);
		Lights.Add(&InScene.RectLights[LightIndex]);
	}

	const FBoxSphereBounds3f EffectiveImportanceBounds = ImportanceBounds.SphereRadius > 0.0f ? ImportanceBounds : SceneBounds;
	for (int32 LightIndex = 0; LightIndex < MeshAreaLights.Num(); LightIndex++)
	{
		MeshAreaLights[LightIndex].Initialize(Scene.PhotonMappingSettings.IndirectPhotonEmitConeAngle, EffectiveImportanceBounds);
		Lights.Add(&MeshAreaLights[LightIndex]);
	}

	for (int32 LightIndex = 0; LightIndex < InScene.SkyLights.Num(); LightIndex++)
	{
		SkyLights.Add(&InScene.SkyLights[LightIndex]);
	}

	//@todo - only count mappings being built
	Stats.NumMappings = AllMappings.Num();
	for (int32 MappingIndex = 0; MappingIndex < AllMappings.Num(); MappingIndex++)
	{
		FStaticLightingTextureMapping* TextureMapping = AllMappings[MappingIndex]->GetTextureMapping();
		if (TextureMapping && !(AllMappings[MappingIndex]->GetVolumeMapping() || AllMappings[MappingIndex]->GetLandscapeVolumeMapping()))
		{
			Stats.NumTexelsProcessed += TextureMapping->CachedSizeX * TextureMapping->CachedSizeY;
		}
		AllMappings[MappingIndex]->SceneMappingIndex = MappingIndex;
		AllMappings[MappingIndex]->Initialize(*this);
	}

	InitializePhotonSettings();

	// Prepare the aggregate mesh for raytracing.
	AggregateMesh->PrepareForRaytracing();
	AggregateMesh->DumpStats();

	if (Scene.GeneralSettings.bUseFastVoxelization)
	{
		VoxelizationSurfaceAggregateMesh->PrepareForRaytracing();
		VoxelizationVolumeAggregateMesh->PrepareForRaytracing();

		LandscapeCullingVoxelizationAggregateMesh->PrepareForRaytracing();
	}

	NumCompletedRadiosityIterationMappings.Empty(GeneralSettings.NumSkyLightingBounces);
	NumCompletedRadiosityIterationMappings.AddDefaulted(GeneralSettings.NumSkyLightingBounces);

	Stats.SceneSetupTime = FPlatformTime::Seconds() - SceneSetupStart;
	GStatistics.SceneSetupTime += Stats.SceneSetupTime;

	// spread out the work over multiple parallel threads
	MultithreadProcess();
}

FStaticLightingSystem::~FStaticLightingSystem()
{
	delete AggregateMesh;
	AggregateMesh = NULL;

	if (Scene.GeneralSettings.bUseFastVoxelization)
	{
		delete VoxelizationSurfaceAggregateMesh;
		VoxelizationSurfaceAggregateMesh = NULL;

		delete VoxelizationVolumeAggregateMesh;
		VoxelizationVolumeAggregateMesh = NULL;

		for (int32 MeshIndex = 0; MeshIndex < Scene.StaticMeshInstances.Num(); MeshIndex++)
		{
			const FStaticMeshStaticLightingMesh* MeshInstance = &Scene.StaticMeshInstances[MeshIndex];
			if (MeshInstance->StaticMesh->VoxelizationMesh != nullptr)
			{
				delete MeshInstance->StaticMesh->VoxelizationMesh;
				MeshInstance->StaticMesh->VoxelizationMesh = nullptr;
			}
		}

		delete LandscapeCullingVoxelizationAggregateMesh;
		LandscapeCullingVoxelizationAggregateMesh = NULL;
	}
}

/**
 * Creates multiple worker threads and starts the process locally.
 */
void FStaticLightingSystem::MultithreadProcess()
{
	const double StartTime = FPlatformTime::Seconds();

	UE_LOGF(LogLightmass, Log,  "Processing..." );

	GStatistics.PhotonsStart = FPlatformTime::Seconds();
	CacheSamples();

	if (PhotonMappingSettings.bUsePhotonMapping)
	{		
		// Build photon maps
		EmitPhotons();
	}

	if (ImportanceTracingSettings.bUseRadiositySolverForSkylightMultibounce)
	{
		SetupRadiosity();
		RunRadiosityIterations();
	}

	FinalizeSurfaceCache();

	if (DynamicObjectSettings.bVisualizeVolumeLightInterpolation)
	{
		// Calculate volume samples now if they will be needed by the lighting threads for shading,
		// Otherwise the volume samples will be calculated when the task is received from swarm.
		BeginCalculateVolumeSamples();
	}

	SetupPrecomputedVisibility();

	// Before we spawn the static lighting threads, prefetch tasks they'll be working on
	GSwarm->PrefetchTasks();

	GStatistics.PhotonsEnd = GStatistics.WorkTimeStart = FPlatformTime::Seconds();

	const double SequentialThreadedProcessingStart = FPlatformTime::Seconds();
	// Spawn the static lighting threads.
	for(int32 ThreadIndex = 0;ThreadIndex < NumStaticLightingThreads;ThreadIndex++)
	{
		FMappingProcessingThreadRunnable* ThreadRunnable = new FMappingProcessingThreadRunnable(this, ThreadIndex, StaticLightingTask_ProcessMappings);
		Threads.Add(ThreadRunnable);
		const FString ThreadName = FString::Printf(TEXT("MappingProcessingThread%u"), ThreadIndex);
		ThreadRunnable->Thread = FRunnableThread::Create(ThreadRunnable, *ThreadName);
	}
	GStatistics.NumThreads = NumStaticLightingThreads + 1;	// Includes the main-thread who is only exporting.

	// Stop the static lighting threads.
	double MaxThreadTime = GStatistics.ThreadStatistics.TotalTime;
	float MaxThreadBusyTime = 0;

	int32 NumStaticLightingThreadsDone = 0;
	while ( NumStaticLightingThreadsDone < NumStaticLightingThreads )
	{
		for (int32 ThreadIndex = 0; ThreadIndex < Threads.Num(); ThreadIndex++ )
		{
			if ( Threads[ThreadIndex].Thread != NULL )
			{
				// Check to see if the thread has exited with an error
				if ( Threads[ThreadIndex].CheckHealth( true ) )
				{
					// Wait for the thread to exit
					if (Threads[ThreadIndex].IsComplete())
					{
						Threads[ThreadIndex].Thread->WaitForCompletion();
						// Accumulate all thread statistics
						GStatistics.ThreadStatistics += Threads[ThreadIndex].ThreadStatistics;
						MaxThreadTime = FMath::Max<double>(MaxThreadTime, Threads[ThreadIndex].ThreadStatistics.TotalTime);
						if ( GReportDetailedStats )
						{
							UE_LOGF(LogLightmass, Log,  "Thread %d finished: %ls", ThreadIndex, *FPlatformTime::PrettyTime(Threads[ThreadIndex].ThreadStatistics.TotalTime) );
						}

						MaxThreadBusyTime = FMath::Max(MaxThreadBusyTime, Threads[ThreadIndex].ExecutionTime - Threads[ThreadIndex].IdleTime);
						Stats.TotalLightingThreadTime += Threads[ThreadIndex].ExecutionTime - Threads[ThreadIndex].IdleTime;

						// We're done with the thread object, destroy it
						delete Threads[ThreadIndex].Thread;
						Threads[ThreadIndex].Thread = NULL;
						NumStaticLightingThreadsDone++;
					}
					else
					{
						FPlatformProcess::Sleep(0.01f);
					}
				}
			}
		}

		// Try to do some mappings while we're waiting for threads to finish
		if ( NumStaticLightingThreadsDone < NumStaticLightingThreads )
		{
			CompleteTextureMappingList.ApplyAndClear( *this );
			ExportNonMappingTasks();
		}

#if USE_LOCAL_SWARM_INTERFACE
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
#endif

		GLog->FlushThreadedLogs();
	}
	Threads.Empty();
	GStatistics.WorkTimeEnd = FPlatformTime::Seconds();

	// Threads will idle when they have no more tasks but before the user accepts the async build changes, so we have to make sure we only count busy time
	Stats.MainThreadLightingTime = (SequentialThreadedProcessingStart - StartTime) + MaxThreadBusyTime;

	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_ExportingResults, -1 ) );

	// Apply any outstanding completed mappings.
	CompleteTextureMappingList.ApplyAndClear( *this );
	ExportNonMappingTasks();

	// Adjust worktime to represent the slowest thread, since that's when all threads were finished.
	// This makes it easier to see how well the actual thread processing is parallelized.
	double Adjustment = (GStatistics.WorkTimeEnd - GStatistics.WorkTimeStart) - MaxThreadTime;
	if ( Adjustment > 0.0 )
	{
		GStatistics.WorkTimeEnd -= Adjustment;
	}

	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Finished, -1 ) );

	// Let's say the main thread used up the whole parallel time.
	GStatistics.ThreadStatistics.TotalTime += MaxThreadTime;
	const float FinishAndExportTime = FPlatformTime::Seconds() - GStatistics.WorkTimeEnd;
	DumpStats(Stats.SceneSetupTime + Stats.MainThreadLightingTime + FinishAndExportTime);
	AggregateMesh->DumpCheckStats();
}

/** Exports tasks that are not mappings, if they are ready. */
void FStaticLightingSystem::ExportNonMappingTasks()
{
	// Export volume lighting samples to Swarm if they are complete
	if (bShouldExportVolumeSampleData)
	{
		bShouldExportVolumeSampleData = false;

		Exporter.ExportVolumeLightingSamples(
			DynamicObjectSettings.bVisualizeVolumeLightSamples,
			VolumeLightingDebugOutput,
			VolumeBounds.Origin, 
			VolumeBounds.BoxExtent, 
			VolumeLightingSamples);

		// Release volume lighting samples unless they are being used by the lighting threads for shading
		if (!DynamicObjectSettings.bVisualizeVolumeLightInterpolation)
		{
			VolumeLightingSamples.Empty();
		}

		// Tell Swarm the task is complete (if we're not in debugging mode).
		if ( !IsDebugMode() )
		{
			FLightmassSwarm* Swarm = GetExporter().GetSwarm();
			Swarm->TaskCompleted( PrecomputedVolumeLightingGuid );
		}
	}

	CompleteVisibilityTaskList.ApplyAndClear(*this);
	CompleteVolumetricLightmapTaskList.ApplyAndClear(*this);

	{
		TMap<const FLight*, FStaticShadowDepthMap*> CompletedStaticShadowDepthMapsCopy;
		{
			// Enter a critical section before modifying DominantSpotLightShadowInfos since the worker threads may also modify it at any time
			FScopeLock Lock(&CompletedStaticShadowDepthMapsSync);
			CompletedStaticShadowDepthMapsCopy = CompletedStaticShadowDepthMaps;
			CompletedStaticShadowDepthMaps.Empty();
		}

		for (TMap<const FLight*,FStaticShadowDepthMap*>::TIterator It(CompletedStaticShadowDepthMapsCopy); It; ++It)
		{
			const FLight* Light = It.Key();
			Exporter.ExportStaticShadowDepthMap(Light->Guid, *It.Value());

			// Tell Swarm the task is complete (if we're not in debugging mode).
			if (!IsDebugMode())
			{
				FLightmassSwarm* Swarm = GetExporter().GetSwarm();
				Swarm->TaskCompleted(Light->Guid);
			}

			delete It.Value();
		}
	}

	if (bShouldExportMeshAreaLightData)
	{
		Exporter.ExportMeshAreaLightData(MeshAreaLights, MeshAreaLightSettings.MeshAreaLightGeneratedDynamicLightSurfaceOffset);

		// Tell Swarm the task is complete (if we're not in debugging mode).
		if ( !IsDebugMode() )
		{
			FLightmassSwarm* Swarm = GetExporter().GetSwarm();
			Swarm->TaskCompleted( MeshAreaLightDataGuid );
		}
		bShouldExportMeshAreaLightData = 0;
	}

	if (bShouldExportVolumeDistanceField)
	{
		Exporter.ExportVolumeDistanceField(VolumeSizeX, VolumeSizeY, VolumeSizeZ, VolumeDistanceFieldSettings.VolumeMaxDistance, DistanceFieldVolumeBounds, VolumeDistanceField);

		// Tell Swarm the task is complete (if we're not in debugging mode).
		if ( !IsDebugMode() )
		{
			FLightmassSwarm* Swarm = GetExporter().GetSwarm();
			Swarm->TaskCompleted( VolumeDistanceFieldGuid );
		}
		bShouldExportVolumeDistanceField = 0;
	}
}

int32 FStaticLightingSystem::GetNumShadowRays(int32 BounceNumber, bool bPenumbra) const
{
	int32 NumShadowRaysResult = 0;
	if (BounceNumber == 0 && bPenumbra)
	{
		NumShadowRaysResult = ShadowSettings.NumPenumbraShadowRays;
	}
	else if (BounceNumber == 0 && !bPenumbra)
	{
		NumShadowRaysResult = ShadowSettings.NumShadowRays;
	}
	else if (BounceNumber > 0)
	{
		// Use less rays for each progressive bounce, since the variance will matter less.
		NumShadowRaysResult = FMath::Max(ShadowSettings.NumBounceShadowRays / BounceNumber, 1);
	}
	return NumShadowRaysResult;
}

int32 FStaticLightingSystem::GetNumUniformHemisphereSamples(int32 BounceNumber) const
{
	int32 NumSamplesResult = CachedHemisphereSamples.Num();
	checkSlow(BounceNumber > 0);
	return NumSamplesResult;
}

int32 FStaticLightingSystem::GetNumPhotonImportanceHemisphereSamples() const
{
	return PhotonMappingSettings.bUsePhotonMapping ? 
		FMath::TruncToInt(ImportanceTracingSettings.NumHemisphereSamples * PhotonMappingSettings.FinalGatherImportanceSampleFraction) : 0;
}

FBoxSphereBounds3f FStaticLightingSystem::GetImportanceBounds(bool bClampToScene) const
{
	FBoxSphereBounds3f ImportanceBounds = Scene.GetImportanceBounds();
	
	if (bClampToScene)
	{
		const FBoxSphereBounds3f SceneBounds = FBoxSphereBounds3f(AggregateMesh->GetBounds());
		const float SceneToImportanceOriginSquared = (ImportanceBounds.Origin - SceneBounds.Origin).SizeSquared();
		if (SceneToImportanceOriginSquared > FMath::Square(SceneBounds.SphereRadius))
		{
			// Disable the importance bounds if the center of the importance volume is outside of the scene.
			ImportanceBounds.SphereRadius = 0.0f;
		}
		else if (SceneToImportanceOriginSquared > FMath::Square(SceneBounds.SphereRadius - ImportanceBounds.SphereRadius))
		{
			// Clamp the importance volume's radius so that all parts of it are inside the scene.
			ImportanceBounds.SphereRadius = SceneBounds.SphereRadius - FMath::Sqrt(SceneToImportanceOriginSquared);
		}
		else if (SceneBounds.SphereRadius <= ImportanceBounds.SphereRadius)
		{
			// Disable the importance volume if it is larger than the scene.
			ImportanceBounds.SphereRadius = 0.0f;
		}
	}
	
	return ImportanceBounds;
}

/** Returns true if the specified position is inside any of the importance volumes. */
bool FStaticLightingSystem::IsPointInImportanceVolume(const FVector4f& Position, float Tolerance) const
{
	if (Scene.ImportanceVolumes.Num() > 0)
	{
		return Scene.IsPointInImportanceVolume(Position, Tolerance);
	}
	else
	{
		return true;
	}
}

/** Changes the scene's settings if necessary so that only valid combinations are used */
void FStaticLightingSystem::ValidateSettings(FScene& InScene)
{
	//@todo - verify valid ranges of all settings

	InScene.GeneralSettings.NumIndirectLightingBounces = FMath::Max(InScene.GeneralSettings.NumIndirectLightingBounces, 0);
	InScene.GeneralSettings.IndirectLightingSmoothness = FMath::Clamp(InScene.GeneralSettings.IndirectLightingSmoothness, .25f, 10.0f);
	InScene.GeneralSettings.IndirectLightingQuality = FMath::Clamp(InScene.GeneralSettings.IndirectLightingQuality, .1f, 100.0f);
	InScene.GeneralSettings.ViewSingleBounceNumber = FMath::Min(InScene.GeneralSettings.ViewSingleBounceNumber, InScene.GeneralSettings.NumIndirectLightingBounces);

	if (FMath::IsNearlyEqual(InScene.PhotonMappingSettings.IndirectPhotonDensity, 0.0f))
	{
		// Allocate all samples toward uniform sampling if there are no indirect photons
		InScene.PhotonMappingSettings.FinalGatherImportanceSampleFraction = 0;
	}
#if LIGHTMASS_DO_PROCESSING
	if (!InScene.PhotonMappingSettings.bUseIrradiancePhotons)
#endif
	{
		InScene.PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces = false;
	}
	InScene.PhotonMappingSettings.FinalGatherImportanceSampleFraction = FMath::Clamp(InScene.PhotonMappingSettings.FinalGatherImportanceSampleFraction, 0.0f, 1.0f);
	if (InScene.ImportanceTracingSettings.NumHemisphereSamples * (1.0f - InScene.PhotonMappingSettings.FinalGatherImportanceSampleFraction) < 1)
	{
		// Irradiance caching needs some uniform samples
		InScene.IrradianceCachingSettings.bAllowIrradianceCaching = false;
	}

	if (InScene.PhotonMappingSettings.bUsePhotonMapping && !InScene.PhotonMappingSettings.bUseFinalGathering)
	{
		// Irradiance caching currently only supported with final gathering
		InScene.IrradianceCachingSettings.bAllowIrradianceCaching = false;
	}

	InScene.PhotonMappingSettings.ConeFilterConstant = FMath::Max(InScene.PhotonMappingSettings.ConeFilterConstant, 1.0f);
	if (!InScene.IrradianceCachingSettings.bAllowIrradianceCaching)
	{
		InScene.IrradianceCachingSettings.bUseIrradianceGradients = false;
	}

	if (InScene.IrradianceCachingSettings.bUseIrradianceGradients)
	{
		// Irradiance gradients require stratified sampling because the information from each sampled cell is used to calculate the gradient
		InScene.ImportanceTracingSettings.bUseStratifiedSampling = true;
	}
	else
	{
		InScene.IrradianceCachingSettings.bShowGradientsOnly = false;
	}

	if (InScene.DynamicObjectSettings.bVisualizeVolumeLightInterpolation)
	{
		// Disable irradiance caching if we are visualizing volume light interpolation, otherwise we will be getting a twice interpolated result.
		InScene.IrradianceCachingSettings.bAllowIrradianceCaching = false;
	}

	// Round up to nearest odd number
	ShadowSettings.MinDistanceFieldUpsampleFactor = FMath::Clamp(ShadowSettings.MinDistanceFieldUpsampleFactor - ShadowSettings.MinDistanceFieldUpsampleFactor % 2 + 1, 1, 17);
	ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceX = FMath::Max(ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceX, DELTA);
	ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceY = FMath::Max(ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceY, DELTA);

	InScene.IrradianceCachingSettings.InterpolationMaxAngle = FMath::Clamp(InScene.IrradianceCachingSettings.InterpolationMaxAngle, 0.0f, 90.0f);
	InScene.IrradianceCachingSettings.PointBehindRecordMaxAngle = FMath::Clamp(InScene.IrradianceCachingSettings.PointBehindRecordMaxAngle, 0.0f, 90.0f);
	InScene.IrradianceCachingSettings.DistanceSmoothFactor = FMath::Max(InScene.IrradianceCachingSettings.DistanceSmoothFactor, 1.0f);
	InScene.IrradianceCachingSettings.AngleSmoothFactor = FMath::Max(InScene.IrradianceCachingSettings.AngleSmoothFactor, 1.0f);
	InScene.IrradianceCachingSettings.SkyOcclusionSmoothnessReduction = FMath::Clamp(InScene.IrradianceCachingSettings.SkyOcclusionSmoothnessReduction, 0.1f, 1.0f);

	if (InScene.GeneralSettings.IndirectLightingQuality > 50)
	{
		InScene.ImportanceTracingSettings.NumAdaptiveRefinementLevels += 2;
	}
	else if (InScene.GeneralSettings.IndirectLightingQuality > 10)
	{
		InScene.ImportanceTracingSettings.NumAdaptiveRefinementLevels += 1;
	}

	InScene.ShadowSettings.NumShadowRays = FMath::TruncToInt(InScene.ShadowSettings.NumShadowRays * FMath::Sqrt(InScene.GeneralSettings.IndirectLightingQuality));
	InScene.ShadowSettings.NumPenumbraShadowRays = FMath::TruncToInt(InScene.ShadowSettings.NumPenumbraShadowRays * FMath::Sqrt(InScene.GeneralSettings.IndirectLightingQuality));

	InScene.ImportanceTracingSettings.NumAdaptiveRefinementLevels = FMath::Min(InScene.ImportanceTracingSettings.NumAdaptiveRefinementLevels, MaxNumRefiningDepths);
}

/** Logs solver stats */
void FStaticLightingSystem::DumpStats(float TotalStaticLightingTime) const
{
	FString SolverStats = TEXT("\n\n");
	SolverStats += FString::Printf(TEXT("Total Static Lighting time: %7.2f seconds, %i threads\n"), TotalStaticLightingTime, NumStaticLightingThreads );
	SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Scene setup\n"), 100.0f * Stats.SceneSetupTime / TotalStaticLightingTime, Stats.SceneSetupTime);
	if (Stats.NumMeshAreaLights > 0)
	{
		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Mesh Area Light setup\n"), 100.0f * Stats.MeshAreaLightSetupTime / TotalStaticLightingTime, Stats.MeshAreaLightSetupTime);
	}

	if (PhotonMappingSettings.bUsePhotonMapping)
	{
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Emit Direct Photons\n"), 100.0f * Stats.EmitDirectPhotonsTime / TotalStaticLightingTime, Stats.EmitDirectPhotonsTime);
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Cache Indirect Photon Paths\n"), 100.0f * Stats.CachingIndirectPhotonPathsTime / TotalStaticLightingTime, Stats.CachingIndirectPhotonPathsTime);
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Emit Indirect Photons\n"), 100.0f * Stats.EmitIndirectPhotonsTime / TotalStaticLightingTime, Stats.EmitIndirectPhotonsTime);
		if (PhotonMappingSettings.bUseIrradiancePhotons)
		{
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Mark %.3f million Irradiance Photons\n"), 100.0f * Stats.IrradiancePhotonMarkingTime / TotalStaticLightingTime, Stats.IrradiancePhotonMarkingTime, Stats.NumIrradiancePhotons / 1000000.0f);
			if (PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces)
			{
				SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Cache %.3f million Irradiance Photon Samples on surfaces\n"), 100.0f * Stats.CacheIrradiancePhotonsTime / TotalStaticLightingTime, Stats.CacheIrradiancePhotonsTime, Stats.NumCachedIrradianceSamples / 1000000.0f);
			}
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Calculate %.3f million Irradiance Photons\n"), 100.0f * Stats.IrradiancePhotonCalculatingTime / TotalStaticLightingTime, Stats.IrradiancePhotonCalculatingTime, Stats.NumFoundIrradiancePhotons / 1000000.0f);
		}
	}

	if (Stats.PrecomputedVisibilitySetupTime / TotalStaticLightingTime > .02f)
	{
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    sPVS setup\n"), 100.0f * Stats.PrecomputedVisibilitySetupTime / TotalStaticLightingTime, Stats.PrecomputedVisibilitySetupTime);
	}

	SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Lighting\n"), 100.0f * Stats.MainThreadLightingTime / TotalStaticLightingTime, Stats.MainThreadLightingTime);
	const float UnaccountedMainThreadTime = FMath::Max(TotalStaticLightingTime - (Stats.SceneSetupTime + Stats.EmitDirectPhotonsTime + Stats.CachingIndirectPhotonPathsTime + Stats.EmitIndirectPhotonsTime + Stats.IrradiancePhotonMarkingTime + Stats.CacheIrradiancePhotonsTime + Stats.IrradiancePhotonCalculatingTime + Stats.MainThreadLightingTime), 0.0f);
	SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Unaccounted\n"), 100.0f * UnaccountedMainThreadTime / TotalStaticLightingTime, UnaccountedMainThreadTime);

	// Send the message in multiple parts since it cuts off in the middle otherwise
	LogSolverMessage(SolverStats);
	SolverStats = TEXT("");
	if (PhotonMappingSettings.bUsePhotonMapping)
	{
		if (Stats.EmitDirectPhotonsTime / TotalStaticLightingTime > .02) 
		{
			SolverStats += FString::Printf( TEXT("Total Direct Photon Emitting thread seconds: %.1f\n"), Stats.EmitDirectPhotonsThreadTime);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Sampling Lights\n"), 100.0f * Stats.DirectPhotonsLightSamplingThreadTime / Stats.EmitDirectPhotonsThreadTime, Stats.DirectPhotonsLightSamplingThreadTime);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Custom attenuation\n"), 100.0f * Stats.DirectCustomAttenuationThreadTime / Stats.EmitDirectPhotonsThreadTime, Stats.DirectCustomAttenuationThreadTime);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Tracing\n"), 100.0f * Stats.DirectPhotonsTracingThreadTime / Stats.EmitDirectPhotonsThreadTime, Stats.DirectPhotonsTracingThreadTime);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Processing results\n"), 100.0f * Stats.ProcessDirectPhotonsThreadTime / Stats.EmitDirectPhotonsThreadTime, Stats.ProcessDirectPhotonsThreadTime);
			const float UnaccountedDirectPhotonThreadTime = FMath::Max(Stats.EmitDirectPhotonsThreadTime - (Stats.ProcessDirectPhotonsThreadTime + Stats.DirectPhotonsLightSamplingThreadTime + Stats.DirectPhotonsTracingThreadTime + Stats.DirectCustomAttenuationThreadTime), 0.0f);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Unaccounted\n"), 100.0f * UnaccountedDirectPhotonThreadTime / Stats.EmitDirectPhotonsThreadTime, UnaccountedDirectPhotonThreadTime);
		}

		if (Stats.EmitIndirectPhotonsTime / TotalStaticLightingTime > .02) 
		{
			SolverStats += FString::Printf( TEXT("\n") );
			SolverStats += FString::Printf( TEXT("Total Indirect Photon Emitting thread seconds: %.1f\n"), Stats.EmitIndirectPhotonsThreadTime);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Sampling Lights\n"), 100.0f * Stats.LightSamplingThreadTime / Stats.EmitIndirectPhotonsThreadTime, Stats.LightSamplingThreadTime);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Intersect Light rays\n"), 100.0f * Stats.IntersectLightRayThreadTime / Stats.EmitIndirectPhotonsThreadTime, Stats.IntersectLightRayThreadTime);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    PhotonBounceTracing\n"), 100.0f * Stats.PhotonBounceTracingThreadTime / Stats.EmitIndirectPhotonsThreadTime, Stats.PhotonBounceTracingThreadTime);
			SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Custom attenuation\n"), 100.0f * Stats.IndirectCustomAttenuationThreadTime / Stats.EmitIndirectPhotonsThreadTime, Stats.IndirectCustomAttenuationThreadTime);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Processing results\n"), 100.0f * Stats.ProcessIndirectPhotonsThreadTime / Stats.EmitIndirectPhotonsThreadTime, Stats.ProcessIndirectPhotonsThreadTime);
			const float UnaccountedIndirectPhotonThreadTime = FMath::Max(Stats.EmitIndirectPhotonsThreadTime - (Stats.ProcessIndirectPhotonsThreadTime + Stats.LightSamplingThreadTime + Stats.IntersectLightRayThreadTime + Stats.PhotonBounceTracingThreadTime), 0.0f);
			SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Unaccounted\n"), 100.0f * UnaccountedIndirectPhotonThreadTime / Stats.EmitIndirectPhotonsThreadTime, UnaccountedIndirectPhotonThreadTime);
		}

		if (PhotonMappingSettings.bUseIrradiancePhotons)
		{
			if (PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces
				// Only log Irradiance photon caching stats if it was more than 2 percent of the total time
				&& Stats.CacheIrradiancePhotonsTime / TotalStaticLightingTime > .02)
			{
				SolverStats += FString::Printf( TEXT("\n") );
				SolverStats += FString::Printf( TEXT("Total Irradiance Photon Caching thread seconds: %.1f\n"), Stats.IrradiancePhotonCachingThreadTime);
				SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Octree traversal\n"), 100.0f * Stats.IrradiancePhotonOctreeTraversalTime / Stats.IrradiancePhotonCachingThreadTime, Stats.IrradiancePhotonOctreeTraversalTime);
				SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    %.3f million Visibility rays\n"), 100.0f * Stats.IrradiancePhotonSearchRayTime / Stats.IrradiancePhotonCachingThreadTime, Stats.IrradiancePhotonSearchRayTime, Stats.NumIrradiancePhotonSearchRays / 1000000.0f);
				const float UnaccountedIrradiancePhotonCachingThreadTime = FMath::Max(Stats.IrradiancePhotonCachingThreadTime - (Stats.IrradiancePhotonOctreeTraversalTime + Stats.IrradiancePhotonSearchRayTime), 0.0f);
				SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Unaccounted\n"), 100.0f * UnaccountedIrradiancePhotonCachingThreadTime / Stats.IrradiancePhotonCachingThreadTime, UnaccountedIrradiancePhotonCachingThreadTime);
			}

			// Only log Irradiance photon calculating stats if it was more than 2 percent of the total time
			if (Stats.IrradiancePhotonCalculatingTime / TotalStaticLightingTime > .02)
			{
				SolverStats += FString::Printf( TEXT("\n") );
				SolverStats += FString::Printf( TEXT("Total Calculating Irradiance Photons thread seconds: %.1f\n"), Stats.IrradiancePhotonCalculatingThreadTime);
				SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Pushing Octree Children\n"), 100.0f * Stats.CalculateIrradiancePhotonStats.PushingOctreeChildrenThreadTime / Stats.IrradiancePhotonCalculatingThreadTime, Stats.CalculateIrradiancePhotonStats.PushingOctreeChildrenThreadTime);
				SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Processing Octree Elements\n"), 100.0f * Stats.CalculateIrradiancePhotonStats.ProcessingOctreeElementsThreadTime / Stats.IrradiancePhotonCalculatingThreadTime, Stats.CalculateIrradiancePhotonStats.ProcessingOctreeElementsThreadTime);
				SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Finding furthest photon\n"), 100.0f * Stats.CalculateIrradiancePhotonStats.FindingFurthestPhotonThreadTime / Stats.IrradiancePhotonCalculatingThreadTime, Stats.CalculateIrradiancePhotonStats.FindingFurthestPhotonThreadTime);
				SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Calculating Irradiance\n"), 100.0f * Stats.CalculateIrradiancePhotonStats.CalculateIrradianceThreadTime / Stats.IrradiancePhotonCalculatingThreadTime, Stats.CalculateIrradiancePhotonStats.CalculateIrradianceThreadTime);
				const float UnaccountedCalculateIrradiancePhotonsTime = FMath::Max(Stats.IrradiancePhotonCalculatingThreadTime - 
					(Stats.CalculateIrradiancePhotonStats.PushingOctreeChildrenThreadTime + Stats.CalculateIrradiancePhotonStats.ProcessingOctreeElementsThreadTime + Stats.CalculateIrradiancePhotonStats.CalculateIrradianceThreadTime), 0.0f);
				SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Unaccounted\n"), 100.0f * UnaccountedCalculateIrradiancePhotonsTime / Stats.IrradiancePhotonCalculatingThreadTime, UnaccountedCalculateIrradiancePhotonsTime);
			}
		}

		SolverStats += FString::Printf( TEXT("\n") );
		SolverStats += FString::Printf( TEXT("Radiosity Setup thread seconds: %.1f, Radiosity Iteration thread seconds: %.1f\n"), Stats.RadiositySetupThreadTime, Stats.RadiosityIterationThreadTime);
	}

	// Send the message in multiple parts since it cuts off in the middle otherwise
	LogSolverMessage(SolverStats);
	SolverStats = TEXT("");

	const float TotalLightingBusyThreadTime = Stats.TotalLightingThreadTime;

	SolverStats += FString::Printf( TEXT("\n") );
	SolverStats += FString::Printf( TEXT("Total busy Lighting thread seconds: %.2f\n"), TotalLightingBusyThreadTime);
	const float SampleSetupTime = Stats.VertexSampleCreationTime + Stats.TexelRasterizationTime;
	SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Texel and vertex setup\n"), 100.0f * SampleSetupTime / TotalLightingBusyThreadTime, SampleSetupTime);
	SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Direct lighting\n"), 100.0f * Stats.DirectLightingTime / TotalLightingBusyThreadTime, Stats.DirectLightingTime);
	SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Area shadows with %.3f million rays\n"), 100.0f * Stats.AreaShadowsThreadTime / TotalLightingBusyThreadTime, Stats.AreaShadowsThreadTime, Stats.NumDirectLightingShadowRays / 1000000.0f);
	if (Stats.AreaLightingThreadTime / TotalLightingBusyThreadTime > .04f)
	{
		SolverStats += FString::Printf( TEXT("%12.1f%%%8.1fs    Area lighting\n"), 100.0f * Stats.AreaLightingThreadTime / TotalLightingBusyThreadTime, Stats.AreaLightingThreadTime);
	}

	if (Stats.NumSignedDistanceFieldCalculations > 0)
	{
		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Signed distance field source sparse sampling\n"), 100.0f * Stats.SignedDistanceFieldSourceFirstPassThreadTime / TotalLightingBusyThreadTime, Stats.SignedDistanceFieldSourceFirstPassThreadTime);
		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Signed distance field source refining sampling\n"), 100.0f * Stats.SignedDistanceFieldSourceSecondPassThreadTime / TotalLightingBusyThreadTime, Stats.SignedDistanceFieldSourceSecondPassThreadTime);
		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Signed distance field transition searching\n"), 100.0f * Stats.SignedDistanceFieldSearchThreadTime / TotalLightingBusyThreadTime, Stats.SignedDistanceFieldSearchThreadTime);
	}
	const float UnaccountedDirectLightingTime = FMath::Max(Stats.DirectLightingTime - (Stats.AreaShadowsThreadTime + Stats.SignedDistanceFieldSourceFirstPassThreadTime + Stats.SignedDistanceFieldSourceSecondPassThreadTime + Stats.SignedDistanceFieldSearchThreadTime), 0.0f);
	SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Unaccounted\n"), 100.0f * UnaccountedDirectLightingTime / TotalLightingBusyThreadTime, UnaccountedDirectLightingTime);

	SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Block on indirect lighting cache tasks\n"), 100.0f * Stats.BlockOnIndirectLightingCacheTasksTime / TotalLightingBusyThreadTime, Stats.BlockOnIndirectLightingCacheTasksTime);

	if (IrradianceCachingSettings.bAllowIrradianceCaching)
	{
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Block on IrradianceCache Interpolation tasks\n"), 100.0f * Stats.BlockOnIndirectLightingInterpolateTasksTime / TotalLightingBusyThreadTime, Stats.BlockOnIndirectLightingInterpolateTasksTime);
	}
	if (Stats.StaticShadowDepthMapThreadTime > 0.1f)
	{
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Static shadow depth maps (max %.1fs)\n"), 100.0f * Stats.StaticShadowDepthMapThreadTime / TotalLightingBusyThreadTime, Stats.StaticShadowDepthMapThreadTime, Stats.MaxStaticShadowDepthMapThreadTime);
	}
	if (Stats.VolumeDistanceFieldThreadTime > 0.1f)
	{
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Volume distance field\n"), 100.0f * Stats.VolumeDistanceFieldThreadTime / TotalLightingBusyThreadTime, Stats.VolumeDistanceFieldThreadTime);
	}
	const float PrecomputedVisibilityThreadTime = Stats.PrecomputedVisibilityThreadTime;
	if (PrecomputedVisibilityThreadTime > 0.1f)
	{
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Precomputed Visibility\n"), 100.0f * PrecomputedVisibilityThreadTime / TotalLightingBusyThreadTime, PrecomputedVisibilityThreadTime);
		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Sample generation\n"), 100.0f * Stats.PrecomputedVisibilitySampleSetupThreadTime / TotalLightingBusyThreadTime, Stats.PrecomputedVisibilitySampleSetupThreadTime);
		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Uniform tracing\n"), 100.0f * Stats.PrecomputedVisibilityRayTraceThreadTime / TotalLightingBusyThreadTime, Stats.PrecomputedVisibilityRayTraceThreadTime);
		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    Importance sampling\n"), 100.0f * Stats.PrecomputedVisibilityImportanceSampleThreadTime / TotalLightingBusyThreadTime, Stats.PrecomputedVisibilityImportanceSampleThreadTime);
	}
	if (Stats.NumDynamicObjectSurfaceSamples + Stats.NumDynamicObjectVolumeSamples > 0)
	{
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Volume Sample placement\n"), 100.0f * Stats.VolumeSamplePlacementThreadTime / TotalLightingBusyThreadTime, Stats.VolumeSamplePlacementThreadTime);
	}
	if (Stats.NumVolumetricLightmapSamples > 0)
	{
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Volumetric Lightmap - %.3f million samples\n"), 100.0f * Stats.TotalVolumetricLightmapLightingThreadTime / TotalLightingBusyThreadTime, Stats.TotalVolumetricLightmapLightingThreadTime, Stats.NumVolumetricLightmapSamples / 1000000.0f);
		
		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    VoxelizationTime\n"), 100.0f * Stats.VolumetricLightmapVoxelizationTime / TotalLightingBusyThreadTime, Stats.VolumetricLightmapVoxelizationTime);


		if (Stats.VolumetricLightmapGatherImportancePhotonsTime > 0)
		{
			SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    GatherImportancePhotons\n"), 100.0f * Stats.VolumetricLightmapGatherImportancePhotonsTime / TotalLightingBusyThreadTime, Stats.VolumetricLightmapGatherImportancePhotonsTime);
		}

		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    DirectLightingTime\n"), 100.0f * Stats.VolumetricLightmapDirectLightingTime / TotalLightingBusyThreadTime, Stats.VolumetricLightmapDirectLightingTime);
		SolverStats += FString::Printf( TEXT("%8.1f%%%8.1fs    FinalGatherTime\n"), 100.0f * Stats.VolumetricLightmapFinalGatherTime / TotalLightingBusyThreadTime, Stats.VolumetricLightmapFinalGatherTime);
	}
	const float UnaccountedLightingThreadTime = FMath::Max(TotalLightingBusyThreadTime - (SampleSetupTime + Stats.DirectLightingTime + Stats.BlockOnIndirectLightingCacheTasksTime + Stats.BlockOnIndirectLightingInterpolateTasksTime + Stats.IndirectLightingCacheTaskThreadTimeSeparateTask + Stats.SecondPassIrradianceCacheInterpolationTime + Stats.SecondPassIrradianceCacheInterpolationTimeSeparateTask + Stats.VolumeSamplePlacementThreadTime + Stats.StaticShadowDepthMapThreadTime + Stats.VolumeDistanceFieldThreadTime + PrecomputedVisibilityThreadTime + Stats.TotalVolumetricLightmapLightingThreadTime), 0.0f);
	SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    Unaccounted\n"), 100.0f * UnaccountedLightingThreadTime / TotalLightingBusyThreadTime, UnaccountedLightingThreadTime);
	// Send the message in multiple parts since it cuts off in the middle otherwise
	LogSolverMessage(SolverStats);
	SolverStats = TEXT("\n");

	float IndirectLightingCacheTaskThreadTime = Stats.IndirectLightingCacheTaskThreadTime + Stats.IndirectLightingCacheTaskThreadTimeSeparateTask;

	SolverStats += FString::Printf( TEXT("Indirect lighting cache task thread seconds: %.2f\n"), IndirectLightingCacheTaskThreadTime);
	// These inner loop timings rely on rdtsc to avoid the massive overhead of Query Performance Counter.
	// rdtsc is not dependable with multi-threading (see FRDTSCCycleTimer comments and Intel documentation) but we use it anyway because it's the only option.
	//@todo - rdtsc is also not dependable if the OS changes which processor the thread gets executed on.  
	// Use SetThreadAffinityMask to prevent this case.
	if (PhotonMappingSettings.bUsePhotonMapping)
	{
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    ImportancePhotonGatherTime\n"), 100.0f * Stats.ImportancePhotonGatherTime / IndirectLightingCacheTaskThreadTime, Stats.ImportancePhotonGatherTime);
		SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    CalculateImportanceSampleTime\n"), 100.0f * Stats.CalculateImportanceSampleTime / IndirectLightingCacheTaskThreadTime, Stats.CalculateImportanceSampleTime);
	}
	SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    FirstBounceRayTraceTime for %.3f million rays\n"), 100.0f * Stats.FirstBounceRayTraceTime / IndirectLightingCacheTaskThreadTime, Stats.FirstBounceRayTraceTime, Stats.NumFirstBounceRaysTraced / 1000000.0f);
	SolverStats += FString::Printf( TEXT("%4.1f%%%8.1fs    CalculateExitantRadiance\n"), 100.0f * Stats.CalculateExitantRadianceTime / IndirectLightingCacheTaskThreadTime, Stats.CalculateExitantRadianceTime);

	SolverStats += FString::Printf( TEXT("\n") );
	SolverStats += FString::Printf( TEXT("Traced %.3f million first hit visibility rays for a total of %.1f thread seconds (%.3f million per thread second)\n"), Stats.NumFirstHitRaysTraced / 1000000.0f, Stats.FirstHitRayTraceThreadTime, Stats.NumFirstHitRaysTraced / 1000000.0f / Stats.FirstHitRayTraceThreadTime);
	SolverStats += FString::Printf( TEXT("Traced %.3f million boolean visibility rays for a total of %.1f thread seconds (%.3f million per thread second)\n"), Stats.NumBooleanRaysTraced / 1000000.0f, Stats.BooleanRayTraceThreadTime, Stats.NumBooleanRaysTraced / 1000000.0f / Stats.BooleanRayTraceThreadTime);
	const FBoxSphereBounds3f SceneBounds = FBoxSphereBounds3f(AggregateMesh->GetBounds());
	const FBoxSphereBounds3f ImportanceBounds = GetImportanceBounds();
	SolverStats += FString::Printf( TEXT("Scene radius %.1f, Importance bounds radius %.1f\n"), SceneBounds.SphereRadius, ImportanceBounds.SphereRadius);
	SolverStats += FString::Printf( TEXT("%u Mappings, %.3f million Texels, %.3f million mapped texels\n"), Stats.NumMappings, Stats.NumTexelsProcessed / 1000000.0f, Stats.NumMappedTexels / 1000000.0f);
	
	// Send the message in multiple parts since it cuts off in the middle otherwise
	LogSolverMessage(SolverStats);
	SolverStats = TEXT("");

	float TextureMappingThreadTime = Stats.TotalTextureMappingLightingThreadTime + Stats.SecondPassIrradianceCacheInterpolationTimeSeparateTask + Stats.IndirectLightingCacheTaskThreadTimeSeparateTask;
	const float UnaccountedMappingThreadTimePct = 100.0f * FMath::Max(TotalLightingBusyThreadTime - (TextureMappingThreadTime + Stats.TotalVolumeSampleLightingThreadTime + Stats.TotalVolumetricLightmapLightingThreadTime + PrecomputedVisibilityThreadTime), 0.0f) / TotalLightingBusyThreadTime;
	SolverStats += FString::Printf( TEXT("%.1f%% of Total Lighting thread seconds on Texture Mappings, %1.f%% on Volume Samples, %1.f%% on Volumetric Lightmap, %1.f%% on Visibility, %.1f%% Unaccounted\n"), 100.0f * TextureMappingThreadTime / TotalLightingBusyThreadTime, 100.0f * Stats.TotalVolumeSampleLightingThreadTime / TotalLightingBusyThreadTime, 100.0f * Stats.TotalVolumetricLightmapLightingThreadTime / TotalLightingBusyThreadTime, 100.0f * PrecomputedVisibilityThreadTime / TotalLightingBusyThreadTime, UnaccountedMappingThreadTimePct);
	SolverStats += FString::Printf( TEXT("%u Lights total, %.1f Shadow rays per light sample on average\n"), Stats.NumLights, Stats.NumDirectLightingShadowRays / (float)(Stats.NumMappedTexels + Stats.NumVertexSamples));
	if (Stats.NumMeshAreaLights > 0)
	{
		SolverStats += FString::Printf( TEXT("%u Emissive meshes, %u Mesh area lights, %lld simplified mesh area light primitives, %lld original primitives\n"), Stats.NumMeshAreaLightMeshes, Stats.NumMeshAreaLights, Stats.NumSimplifiedMeshAreaLightPrimitives, Stats.NumMeshAreaLightPrimitives);
	}
	if (Stats.NumSignedDistanceFieldCalculations > 0)
	{
		SolverStats += FString::Printf( TEXT("Signed distance field shadows: %.1f average upsample factor, %.3f million sparse source rays, %.3f million refining source rays, %.3f million transition search scatters\n"), Stats.AccumulatedSignedDistanceFieldUpsampleFactors / Stats.NumSignedDistanceFieldCalculations, Stats.NumSignedDistanceFieldAdaptiveSourceRaysFirstPass / 1000000.0f, Stats.NumSignedDistanceFieldAdaptiveSourceRaysSecondPass / 1000000.0f, Stats.NumSignedDistanceFieldScatters / 1000000.0f);
	}
	const int32 TotalVolumeLightingSamples = Stats.NumDynamicObjectSurfaceSamples + Stats.NumDynamicObjectVolumeSamples;
	if (TotalVolumeLightingSamples > 0)
	{
		SolverStats += FString::Printf( TEXT("%u Volume lighting samples, %.1f%% placed on surfaces, %.1f%% placed in the volume, %.1f thread seconds\n"), TotalVolumeLightingSamples, 100.0f * Stats.NumDynamicObjectSurfaceSamples / (float)TotalVolumeLightingSamples, 100.0f * Stats.NumDynamicObjectVolumeSamples / (float)TotalVolumeLightingSamples, Stats.TotalVolumeSampleLightingThreadTime);
	}

	if (Stats.NumPrecomputedVisibilityQueries > 0)
	{
		SolverStats += FString::Printf( TEXT("Precomputed Visibility %u Cells (%.1f%% from camera tracks, %u processed on this agent), %u Meshes, %.3f million rays, %.1fKb data\n"), Stats.NumPrecomputedVisibilityCellsTotal, 100.0f * Stats.NumPrecomputedVisibilityCellsCamaraTracks / Stats.NumPrecomputedVisibilityCellsTotal, Stats.NumPrecomputedVisibilityCellsProcessed, Stats.NumPrecomputedVisibilityMeshes, Stats.NumPrecomputedVisibilityRayTraces / 1000000.0f, Stats.PrecomputedVisibilityDataBytes / 1024.0f);
		const uint64 NumQueriesVisible = Stats.NumQueriesVisibleByDistanceRatio + Stats.NumQueriesVisibleExplicitSampling + Stats.NumQueriesVisibleImportanceSampling;
		const uint64 TotalNumQueries = Stats.NumPrecomputedVisibilityQueries + Stats.NumPrecomputedVisibilityGroupQueries;
		SolverStats += FString::Printf( TEXT("   %.3f million mesh queries, %.3f million group queries, %.1f%% visible, (%.1f%% trivially visible, %.1f%% explicit sampling, %.1f%% importance sampling)\n"), Stats.NumPrecomputedVisibilityQueries / 1000000.0f, Stats.NumPrecomputedVisibilityGroupQueries / 1000000.0f, 100.0f * NumQueriesVisible / TotalNumQueries, 100.0f * Stats.NumQueriesVisibleByDistanceRatio / NumQueriesVisible, 100.0f * Stats.NumQueriesVisibleExplicitSampling / NumQueriesVisible, 100.0f * Stats.NumQueriesVisibleImportanceSampling / NumQueriesVisible);
		SolverStats += FString::Printf( TEXT("   %ux%ux%u group cells with %u occupied, %u meshes individually queried, %.3f million mesh queries skipped\n"), GroupVisibilityGridSizeXY, GroupVisibilityGridSizeXY, GroupVisibilityGridSizeZ, VisibilityGroups.Num(), Stats.NumPrecomputedVisibilityMeshesExcludedFromGroups, Stats.NumPrecomputedVisibilityMeshQueriesSkipped / 1000000.0f);
	}

	// Send the message in multiple parts since it cuts off in the middle otherwise
	LogSolverMessage(SolverStats);
	SolverStats = TEXT("");
	if (PhotonMappingSettings.bUsePhotonMapping)
	{
		const float FirstPassEmittedPhotonEfficiency = 100.0f * FMath::Max(Stats.NumDirectPhotonsGathered, NumIndirectPhotonPaths) / Stats.NumFirstPassPhotonsEmitted;
		SolverStats += FString::Printf( TEXT("%.3f million first pass Photons Emitted (out of %.3f million requested) to deposit %.3f million Direct Photons and %u Indirect Photon Paths, efficiency of %.2f%%\n"), Stats.NumFirstPassPhotonsEmitted / 1000000.0f, Stats.NumFirstPassPhotonsRequested / 1000000.0f, Stats.NumDirectPhotonsGathered / 1000000.0f, NumIndirectPhotonPaths, FirstPassEmittedPhotonEfficiency);
		const float SecondPassEmittedPhotonEfficiency = 100.0f * Stats.NumIndirectPhotonsGathered / Stats.NumSecondPassPhotonsEmitted;
		SolverStats += FString::Printf( TEXT("%.3f million second pass Photons Emitted (out of %.3f million requested) to deposit %.3f million Indirect Photons, efficiency of %.2f%%\n"), Stats.NumSecondPassPhotonsEmitted / 1000000.0f, Stats.NumSecondPassPhotonsRequested / 1000000.0f, Stats.NumIndirectPhotonsGathered / 1000000.0f, SecondPassEmittedPhotonEfficiency);
		SolverStats += FString::Printf( TEXT("%.3f million Photon Gathers, %.3f million Irradiance Photon Gathers\n"), Stats.NumPhotonGathers / 1000000.0f, Stats.NumIrradiancePhotonMapSearches / 1000000.0f);
		SolverStats += FString::Printf( TEXT("%.3f million Importance Photons found, %.3f million Importance Photon PDF calculations\n"), Stats.TotalFoundImportancePhotons / 1000000.0f, Stats.NumImportancePDFCalculations / 1000000.0f);
		if (PhotonMappingSettings.bUseIrradiancePhotons && Stats.IrradiancePhotonCalculatingTime / TotalStaticLightingTime > .02)
		{
			SolverStats += FString::Printf( TEXT("%.3f million Irradiance Photons, %.1f%% Direct, %.1f%% Indirect, %.3f million actually found\n"), Stats.NumIrradiancePhotons / 1000000.0f, 100.0f * Stats.NumDirectIrradiancePhotons / Stats.NumIrradiancePhotons, 100.0f * (Stats.NumIrradiancePhotons - Stats.NumDirectIrradiancePhotons) / Stats.NumIrradiancePhotons, Stats.NumFoundIrradiancePhotons / 1000000.0f);
			const float IterationsPerSearch = Stats.CalculateIrradiancePhotonStats.NumSearchIterations / (float)Stats.CalculateIrradiancePhotonStats.NumIterativePhotonMapSearches;
			if (Stats.CalculateIrradiancePhotonStats.NumIterativePhotonMapSearches > 0)
			{
				SolverStats += FString::Printf( TEXT("%.1f Irradiance calculating search iterations per search (%.3f million searches, %.3f million iterations)\n"), IterationsPerSearch, Stats.CalculateIrradiancePhotonStats.NumIterativePhotonMapSearches / 1000000.0f, Stats.CalculateIrradiancePhotonStats.NumSearchIterations / 1000000.0f);
			}
			SolverStats += FString::Printf( TEXT("%.3f million octree nodes tested during irradiance photon calculating, %.3f million nodes visited, %.3f million elements tested, %.3f million elements accepted\n"), Stats.CalculateIrradiancePhotonStats.NumOctreeNodesTested / 1000000.0f, Stats.CalculateIrradiancePhotonStats.NumOctreeNodesVisited / 1000000.0f, Stats.CalculateIrradiancePhotonStats.NumElementsTested / 1000000.0f, Stats.CalculateIrradiancePhotonStats.NumElementsAccepted / 1000000.0f);
		}
	}
	if (IrradianceCachingSettings.bAllowIrradianceCaching)
	{
		const int32 NumIrradianceCacheBounces = PhotonMappingSettings.bUsePhotonMapping ? 1 : GeneralSettings.NumIndirectLightingBounces;
		for (int32 BounceIndex = 0; BounceIndex < NumIrradianceCacheBounces; BounceIndex++)
		{
			const FIrradianceCacheStats& CurrentStats = Stats.Cache[BounceIndex];
			if (CurrentStats.NumCacheLookups > 0)
			{
				const float MissRate = 100.0f * CurrentStats.NumRecords / CurrentStats.NumCacheLookups;
				SolverStats += FString::Printf( TEXT("%.1f%%	Bounce %i Irradiance cache miss rate (%.3f million lookups, %.3f million misses, %.3f million inside geometry)\n"), MissRate, BounceIndex + 1, CurrentStats.NumCacheLookups / 1000000.0f, CurrentStats.NumRecords / 1000000.0f, CurrentStats.NumInsideGeometry / 1000000.0f);
			}
		}
	}

	if (PhotonMappingSettings.bUseFinalGathering)
	{
		uint64 TotalNumRefiningSamples = 0;

		for (int i = 0; i < ImportanceTracingSettings.NumAdaptiveRefinementLevels; i++)
		{
			TotalNumRefiningSamples += Stats.NumRefiningFinalGatherSamples[i];
		}

		SolverStats += FString::Printf( TEXT("Final Gather: %.1fs on %.3f million base samples, %.1fs on %.3f million refining samples for %u refinement levels. \n"), Stats.BaseFinalGatherSampleTime, Stats.NumBaseFinalGatherSamples / 1000000.0f, Stats.RefiningFinalGatherSampleTime, TotalNumRefiningSamples / 1000000.0f, ImportanceTracingSettings.NumAdaptiveRefinementLevels);
		
		if (TotalNumRefiningSamples > 0)
		{
			SolverStats += FString::Printf( TEXT("   %.1f%% due to brightness differences, %.1f%% due to importance photons, %.1f%% other reasons, Samples at depth: "), 100.0f * Stats.NumRefiningSamplesDueToBrightness / TotalNumRefiningSamples, 100.0f * Stats.NumRefiningSamplesDueToImportancePhotons / TotalNumRefiningSamples, 100.0f * Stats.NumRefiningSamplesOther / TotalNumRefiningSamples);

			for (int i = 0; i < ImportanceTracingSettings.NumAdaptiveRefinementLevels; i++)
			{
				SolverStats += FString::Printf(TEXT("%.1f%%, "), 100.0f * Stats.NumRefiningFinalGatherSamples[i] / TotalNumRefiningSamples);
			}

			SolverStats += FString::Printf(TEXT("\n"));
		}
	}

#if PLATFORM_WINDOWS
	PROCESS_MEMORY_COUNTERS_EX ProcessMemoryInfo;
	if (GetProcessMemoryInfo(GetCurrentProcess(), (PROCESS_MEMORY_COUNTERS*)&ProcessMemoryInfo, sizeof(ProcessMemoryInfo)))
	{
		SolverStats += FString::Printf( TEXT("%.1f Mb Peak Working Set\n"), ProcessMemoryInfo.PeakWorkingSetSize / (1024.0f * 1024.0f));
	}
	else
	{
		SolverStats += TEXT("GetProcessMemoryInfo Failed!");
	}
	SolverStats += FString::Printf( TEXT("\n") );
#elif PLATFORM_MAC
	{
		struct rusage MemUsage;
		if (getrusage( RUSAGE_SELF, &MemUsage ) == 0)
		{
			SolverStats += FString::Printf( TEXT("%.1f Mb Peak Working Set\n"), MemUsage.ru_maxrss / (1024.0f * 1024.0f));
		}
		else
		{
			SolverStats += TEXT("getrusage() failed!");
		}
	}
	SolverStats += FString::Printf( TEXT("\n") );
#endif

	LogSolverMessage(SolverStats);

	const bool bDumpMemoryStats = false;
	if (bDumpMemoryStats)
	{
#if PLATFORM_WINDOWS
		PROCESS_MEMORY_COUNTERS ProcessMemory;
		verify(GetProcessMemoryInfo(GetCurrentProcess(), &ProcessMemory, sizeof(ProcessMemory)));
		UE_LOGF(LogLightmass, Log, "Virtual memory used %.1fMb, Peak %.1fMb", 
			ProcessMemory.PagefileUsage / 1048576.0f, 
			ProcessMemory.PeakPagefileUsage / 1048576.0f);
#elif PLATFORM_MAC
		{
			task_basic_info_64_data_t TaskInfo;
			mach_msg_type_number_t TaskInfoCount = TASK_BASIC_INFO_COUNT;
			task_info( mach_task_self(), TASK_BASIC_INFO, (task_info_t)&TaskInfo, &TaskInfoCount );
			UE_LOGF(LogLightmass, Log, "Virtual memory used %.1fMb",
				TaskInfo.virtual_size / 1048576.0f);	// can't get peak virtual memory on Mac
		}
#endif
		AggregateMesh->DumpStats();
		UE_LOGF(LogLightmass, Log, "DirectPhotonMap");
		DirectPhotonMap.DumpStats(false);
		UE_LOGF(LogLightmass, Log, "FirstBouncePhotonMap");
		FirstBouncePhotonMap.DumpStats(false);
		UE_LOGF(LogLightmass, Log, "FirstBounceEscapedPhotonMap");
		FirstBounceEscapedPhotonMap.DumpStats(false);
		UE_LOGF(LogLightmass, Log, "FirstBouncePhotonSegmentMap");
		FirstBouncePhotonSegmentMap.DumpStats(false);
		UE_LOGF(LogLightmass, Log, "SecondBouncePhotonMap");
		SecondBouncePhotonMap.DumpStats(false);
		UE_LOGF(LogLightmass, Log, "IrradiancePhotonMap");
		IrradiancePhotonMap.DumpStats(false);
		uint64 IrradiancePhotonCacheBytes = 0;
		for (int32 i = 0; i < AllMappings.Num(); i++)
		{
			IrradiancePhotonCacheBytes += AllMappings[i]->GetIrradiancePhotonCacheBytes();
		}
		UE_LOGF(LogLightmass, Log, "%.3fMb for Irradiance Photon surface caches", IrradiancePhotonCacheBytes / 1048576.0f);
	}
}

/** Logs a solver message */
void FStaticLightingSystem::LogSolverMessage(const FString& Message) const
{
	if (Scene.DebugInput.bRelaySolverStats)
	{
		// Relay the message back to Unreal if allowed
		GSwarm->SendMessage(NSwarm::FInfoMessage(*Message));
	}
	GLog->Log(*Message);
}

/** Logs a progress update message when appropriate */
void FStaticLightingSystem::UpdateInternalStatus(int32 OldNumTexelsCompleted) const
{
	const int32 NumProgressSteps = 10;

	const float InvTotal = 1.0f / Stats.NumTexelsProcessed;
	const float PreviousCompletedFraction = OldNumTexelsCompleted * InvTotal;
	const float CurrentCompletedFraction = NumTexelsCompleted * InvTotal;
	// Only log NumProgressSteps times
	if (FMath::TruncToInt(PreviousCompletedFraction * NumProgressSteps) < FMath::TruncToInt(CurrentCompletedFraction * NumProgressSteps))
	{
		LogSolverMessage(FString::Printf(TEXT("Lighting %.1f%%"), CurrentCompletedFraction * 100.0f));
	}
}

/** Caches samples for any sampling distributions that are known ahead of time, which greatly reduces noise in those estimates in exchange for structured artifacts. */
void FStaticLightingSystem::CacheSamples()
{
	FLMRandomStream RandomStream(102341);

	int32 NumUniformHemisphereSamples = 0;
		
	if (PhotonMappingSettings.bUsePhotonMapping)
	{
		const float NumSamplesFloat = 
			ImportanceTracingSettings.NumHemisphereSamples 
			* GeneralSettings.IndirectLightingQuality;

		NumUniformHemisphereSamples = FMath::TruncToInt(NumSamplesFloat);
	}
	else
	{
		NumUniformHemisphereSamples = ImportanceTracingSettings.NumHemisphereSamples;
	}

	CachedHemisphereSamples.Empty(NumUniformHemisphereSamples);
	CachedHemisphereSampleUniforms.Empty(NumUniformHemisphereSamples);

	if (ImportanceTracingSettings.bUseStratifiedSampling)
	{
		// Split the sampling domain up into cells with equal area
		// Using PI times more Phi steps as Theta steps, but the relationship between them could be anything
		const float NumThetaStepsFloat = FMath::Sqrt(NumUniformHemisphereSamples / (float)PI);
		const int32 NumThetaSteps = FMath::TruncToInt(NumThetaStepsFloat);
		const int32 NumPhiSteps = FMath::TruncToInt(NumThetaStepsFloat * (float)PI);

		GenerateStratifiedUniformHemisphereSamples(NumThetaSteps, NumPhiSteps, RandomStream, CachedHemisphereSamples, CachedHemisphereSampleUniforms);
	}
	else
	{
		for (int32 SampleIndex = 0; SampleIndex < NumUniformHemisphereSamples; SampleIndex++)
		{
			const FVector4f& CurrentSample = GetUniformHemisphereVector(RandomStream, ImportanceTracingSettings.MaxHemisphereRayAngle);
			CachedHemisphereSamples.Add(CurrentSample);
		}
	}

	{
		FVector4f CombinedVector(0);

		for (int32 SampleIndex = 0; SampleIndex < CachedHemisphereSamples.Num(); SampleIndex++)
		{
			CombinedVector += CachedHemisphereSamples[SampleIndex];
		}

		CachedSamplesMaxUnoccludedLength = (CombinedVector / CachedHemisphereSamples.Num()).Size3();
	}
	
	for (int32 SampleSet = 0; SampleSet < UE_ARRAY_COUNT(CachedHemisphereSamplesForRadiosity); SampleSet++)
	{
		float SampleSetScale = FMath::Lerp(.5f, .125f, SampleSet / ((float)UE_ARRAY_COUNT(CachedHemisphereSamplesForRadiosity) - 1));
		int32 TargetNumApproximateSkyLightingSamples = FMath::Max(FMath::TruncToInt(ImportanceTracingSettings.NumHemisphereSamples * SampleSetScale * GeneralSettings.IndirectLightingQuality), 12);
		CachedHemisphereSamplesForRadiosity[SampleSet].Empty(TargetNumApproximateSkyLightingSamples);
		CachedHemisphereSamplesForRadiosityUniforms[SampleSet].Empty(TargetNumApproximateSkyLightingSamples);

		const float NumThetaStepsFloat = FMath::Sqrt(TargetNumApproximateSkyLightingSamples / (float)PI);
		const int32 NumThetaSteps = FMath::TruncToInt(NumThetaStepsFloat);
		const int32 NumPhiSteps = FMath::TruncToInt(NumThetaStepsFloat * (float)PI);

		GenerateStratifiedUniformHemisphereSamples(NumThetaSteps, NumPhiSteps, RandomStream, CachedHemisphereSamplesForRadiosity[SampleSet], CachedHemisphereSamplesForRadiosityUniforms[SampleSet]);
	}

	// Cache samples on the surface of each light for area shadows
	for (int32 LightIndex = 0; LightIndex < Lights.Num(); LightIndex++)
	{
		FLight* Light = Lights[LightIndex];
		for (int32 BounceIndex = 0; BounceIndex < GeneralSettings.NumIndirectLightingBounces + 1; BounceIndex++)
		{
			const int32 NumPenumbraTypes = BounceIndex == 0 ? 2 : 1;
			Light->CacheSurfaceSamples(BounceIndex, GetNumShadowRays(BounceIndex, false), GetNumShadowRays(BounceIndex, true), RandomStream);
		}
	}

	{
		const int32 NumUpperVolumeSamples = ImportanceTracingSettings.NumHemisphereSamples * DynamicObjectSettings.NumHemisphereSamplesScale;
		const float NumThetaStepsFloat = FMath::Sqrt(NumUpperVolumeSamples / (float)PI);
		const int32 NumThetaSteps = FMath::TruncToInt(NumThetaStepsFloat);
		const int32 NumPhiSteps = FMath::TruncToInt(NumThetaStepsFloat * (float)PI);

		GenerateStratifiedUniformHemisphereSamples(NumThetaSteps, NumPhiSteps, RandomStream, CachedVolumetricLightmapUniformHemisphereSamples, CachedVolumetricLightmapUniformHemisphereSampleUniforms);

		FVector4f CombinedVector(0);

		for (int32 SampleIndex = 0; SampleIndex < CachedVolumetricLightmapUniformHemisphereSamples.Num(); SampleIndex++)
		{
			CombinedVector += CachedVolumetricLightmapUniformHemisphereSamples[SampleIndex];
		}

		CachedVolumetricLightmapMaxUnoccludedLength = (CombinedVector / CachedVolumetricLightmapUniformHemisphereSamples.Num()).Size3();
	}

	CachedVolumetricLightmapVertexOffsets.Add(FVector3f(0, 0, 0));
}

bool FStaticLightingThreadRunnable::CheckHealth(bool bReportError) const
{
	if( bTerminatedByError && bReportError )
	{
		UE_LOGF(LogLightmass, Fatal, "Static lighting thread exception:\r\n%ls", *ErrorMessage );
	}
	return !bTerminatedByError;
}

void FStaticLightingThreadRunnable::FixThreadGroupAffinity() const
{
#if PLATFORM_WINDOWS
	int NumProcessorGroups = GetActiveProcessorGroupCount();
	int PrefixSum = 0;
	for (int GroupIndex = 0; GroupIndex < NumProcessorGroups; GroupIndex++)
	{
		int NumProcessorsThisGroup = GetActiveProcessorCount(GroupIndex);
		if (ThreadIndex < PrefixSum + NumProcessorsThisGroup)
		{
			GROUP_AFFINITY GroupAffinity = {};
			GroupAffinity.Group = GroupIndex;
			GroupAffinity.Mask = (1ull << NumProcessorsThisGroup) - 1;
			check(SetThreadGroupAffinity(GetCurrentThread(), &GroupAffinity, NULL));

			break;
		}
		PrefixSum += NumProcessorsThisGroup;
	}
#endif
}

uint32 FMappingProcessingThreadRunnable::Run()
{
	const double StartThreadTime = FPlatformTime::Seconds();

	FixThreadGroupAffinity();
#if PLATFORM_WINDOWS
	if(!FPlatformMisc::IsDebuggerPresent())
	{
		__try
		{
			if (TaskType == StaticLightingTask_ProcessMappings)
			{
				System->ThreadLoop(false, ThreadIndex, ThreadStatistics, IdleTime);
			}
			else if (TaskType == StaticLightingTask_CacheIrradiancePhotons)
			{
				System->CacheIrradiancePhotonsThreadLoop(ThreadIndex, false);
			}
			else if (TaskType == StaticLightingTask_RadiositySetup)
			{
				System->RadiositySetupThreadLoop(ThreadIndex, false);
			}
			else if (TaskType == StaticLightingTask_RadiosityIterations)
			{
				System->RadiosityIterationThreadLoop(ThreadIndex, false);
			}
			else if (TaskType == StaticLightingTask_FinalizeSurfaceCache)
			{
				System->FinalizeSurfaceCacheThreadLoop(ThreadIndex, false);
			}
			else
			{
				UE_LOGF(LogLightmass, Fatal, "Unsupported task type");
			}
		}
		__except( ReportCrash( GetExceptionInformation() ) )
		{
			ErrorMessage = GErrorHist;

			bTerminatedByError = true;
		}
	}
	else
#endif
	{
		if (TaskType == StaticLightingTask_ProcessMappings)
		{
			System->ThreadLoop(false, ThreadIndex, ThreadStatistics, IdleTime);
		}
		else if (TaskType == StaticLightingTask_CacheIrradiancePhotons)
		{
			System->CacheIrradiancePhotonsThreadLoop(ThreadIndex, false);
		}
		else if (TaskType == StaticLightingTask_RadiositySetup)
		{
			System->RadiositySetupThreadLoop(ThreadIndex, false);
		}
		else if (TaskType == StaticLightingTask_RadiosityIterations)
		{
			System->RadiosityIterationThreadLoop(ThreadIndex, false);
		}
		else if (TaskType == StaticLightingTask_FinalizeSurfaceCache)
		{
			System->FinalizeSurfaceCacheThreadLoop(ThreadIndex, false);
		}
		else
		{
			UE_LOGF(LogLightmass, Fatal, "Unsupported task type");
		}
	}
	ExecutionTime = FPlatformTime::Seconds() - StartThreadTime;
	FinishedCounter.Increment();
	return 0;
}

/**
 * Retrieves the next task from Swarm. Blocking, thread-safe function call. Returns NULL when there are no more tasks.
 * @return	The next mapping task to process.
 */
FStaticLightingMapping*	FStaticLightingSystem::ThreadGetNextMapping( 
	FThreadStatistics& ThreadStatistics, 
	FGuid& TaskGuid,
	uint32 WaitTime, 
	bool& bWaitTimedOut, 
	bool& bDynamicObjectTask, 
	int32& PrecomputedVisibilityTaskIndex,
	int32& VolumetricLightmapTaskIndex,
	bool& bStaticShadowDepthMapTask,
	bool& bMeshAreaLightDataTask,
	bool& bVolumeDataTask)
{
	FStaticLightingMapping* Mapping = NULL;

	// Initialize output parameters
	bWaitTimedOut = true;
	bDynamicObjectTask = false;
	PrecomputedVisibilityTaskIndex = INDEX_NONE;
	VolumetricLightmapTaskIndex = INDEX_NONE;
	bStaticShadowDepthMapTask = false;
	bMeshAreaLightDataTask = false;
	bVolumeDataTask = false;

	if ( GDebugMode )
	{
		FScopeLock Lock( &CriticalSection );
		bWaitTimedOut = false;

		// If we're in debugging mode, just grab the next mapping from the scene.
		TMap<FGuid, FStaticLightingMapping*>::TIterator It(Mappings);
		if ( It )
		{
			Mapping = It.Value();
			It.RemoveCurrent();
		}
	}
	else
	{
		// Request a new task from Swarm.
		FLightmassSwarm* Swarm = Exporter.GetSwarm();
		double SwarmRequestStart = FPlatformTime::Seconds();
		bool bGotTask = Swarm->RequestTask( TaskGuid, WaitTime );
		double SwarmRequestEnd = FPlatformTime::Seconds();
		if ( bGotTask )
		{
			if (TaskGuid == PrecomputedVolumeLightingGuid)
			{
				bDynamicObjectTask = true;
				Swarm->AcceptTask( TaskGuid );
				bWaitTimedOut = false;
			}
			else if (TaskGuid == MeshAreaLightDataGuid)
			{
				bMeshAreaLightDataTask = true;
				Swarm->AcceptTask( TaskGuid );
				bWaitTimedOut = false;
			}
			else if (TaskGuid == VolumeDistanceFieldGuid)
			{
				bVolumeDataTask = true;
				Swarm->AcceptTask( TaskGuid );
				bWaitTimedOut = false;
			}
			else if (Scene.FindLightByGuid(TaskGuid))
			{
				bStaticShadowDepthMapTask = true;
				Swarm->AcceptTask( TaskGuid );
				bWaitTimedOut = false;
			}
			else
			{
				int32 FoundVisibilityIndex = INDEX_NONE;
				Scene.VisibilityBucketGuids.Find(TaskGuid, FoundVisibilityIndex);

				if (FoundVisibilityIndex >= 0)
				{
					PrecomputedVisibilityTaskIndex = FoundVisibilityIndex;
					Swarm->AcceptTask(TaskGuid);
					bWaitTimedOut = false;
				}
				else
				{
					int32 FoundVolumetricLightmapIndex = INDEX_NONE;
					Scene.VolumetricLightmapTaskGuids.Find(TaskGuid, FoundVolumetricLightmapIndex);

					if (FoundVolumetricLightmapIndex >= 0)
					{
						VolumetricLightmapTaskIndex = FoundVolumetricLightmapIndex;
						Swarm->AcceptTask(TaskGuid);
						bWaitTimedOut = false;
					}
					else
					{
						FStaticLightingMapping** MappingPtr = Mappings.Find(TaskGuid);
						if (MappingPtr && FPlatformAtomics::InterlockedExchange(&(*MappingPtr)->bProcessed, true) == false)
						{
							// We received a new mapping to process. Tell Swarm we accept the task.
							Swarm->AcceptTask(TaskGuid);
							bWaitTimedOut = false;
							Mapping = *MappingPtr;
						}
						else
						{
							// Couldn't find the mapping. Tell Swarm we reject the task and try again later.
							UE_LOGF(LogLightmass, Log, "Lightmass - Rejecting task (%08X%08X%08X%08X)!", TaskGuid.A, TaskGuid.B, TaskGuid.C, TaskGuid.D);
							Swarm->RejectTask(TaskGuid);
						}
					}
				}
			}
		}
		else if ( Swarm->ReceivedQuitRequest() || Swarm->IsDone() )
		{
			bWaitTimedOut = false;
		}
		ThreadStatistics.SwarmRequestTime += SwarmRequestEnd - SwarmRequestStart;
	}
	return Mapping;
}

void FStaticLightingSystem::ThreadLoop(bool bIsMainThread, int32 ThreadIndex, FThreadStatistics& ThreadStatistics, float& IdleTime)
{
	const double ThreadTimeStart = FPlatformTime::Seconds();
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Processing0, ThreadIndex ) );

	bool bSignaledMappingsComplete = false;
	bool bIsDone = false;
	while (!bIsDone)
	{
		const double StartLoopTime = FPlatformTime::Seconds();

		bool bAnyTaskProcessedByThisThread = false;

		// Process any existing local tasks before fetching new tasks from Swarm
		{
			while (FCacheIndirectTaskDescription * NextCacheTask = CacheIndirectLightingTasks.Pop())
			{
				//UE_LOGF(LogLightmass, Warning, "Thread %u picked up Cache Indirect task for %u", ThreadIndex, NextCacheTask->TextureMapping->Guid.D);
				ProcessCacheIndirectLightingTask(NextCacheTask, false);
				NextCacheTask->TextureMapping->CompletedCacheIndirectLightingTasks.Push(NextCacheTask);
				FPlatformAtomics::InterlockedDecrement(&NextCacheTask->TextureMapping->NumOutstandingCacheTasks);

				bAnyTaskProcessedByThisThread = true;
			}

			while (FInterpolateIndirectTaskDescription * NextInterpolateTask = InterpolateIndirectLightingTasks.Pop())
			{
				//UE_LOGF(LogLightmass, Warning, "Thread %u picked up Interpolate indirect task for %u", ThreadIndex, NextInterpolateTask->TextureMapping->Guid.D);
				ProcessInterpolateTask(NextInterpolateTask, false);
				NextInterpolateTask->TextureMapping->CompletedInterpolationTasks.Push(NextInterpolateTask);
				FPlatformAtomics::InterlockedDecrement(&NextInterpolateTask->TextureMapping->NumOutstandingInterpolationTasks);

				bAnyTaskProcessedByThisThread = true;
			}

			bAnyTaskProcessedByThisThread |= ProcessVolumetricLightmapTaskIfAvailable();

			while (NumOutstandingVolumeDataLayers > 0)
			{
				const int32 ThreadZ = FPlatformAtomics::InterlockedIncrement(&OutstandingVolumeDataLayerIndex);
				if (ThreadZ < VolumeSizeZ)
				{
					CalculateVolumeDistanceFieldWorkRange(ThreadZ);
					const int32 NumTasksRemaining = FPlatformAtomics::InterlockedDecrement(&NumOutstandingVolumeDataLayers);
					if (NumTasksRemaining == 0)
					{
						FPlatformAtomics::InterlockedExchange(&bShouldExportVolumeDistanceField, true);
					}

					bAnyTaskProcessedByThisThread = true;
				}
			}

			while (NumVolumeSampleTasksOutstanding > 0)
			{
				const int32 TaskIndex = FPlatformAtomics::InterlockedIncrement(&NextVolumeSampleTaskIndex);

				if (TaskIndex < VolumeSampleTasks.Num())
				{
					ProcessVolumeSamplesTask(VolumeSampleTasks[TaskIndex]);
					const int32 NumTasksRemaining = FPlatformAtomics::InterlockedDecrement(&NumVolumeSampleTasksOutstanding);

					if (NumTasksRemaining == 0)
					{
						FPlatformAtomics::InterlockedExchange(&bShouldExportVolumeSampleData, true);
					}

					bAnyTaskProcessedByThisThread = true;
				}
			}
		}

		uint32 DefaultRequestForTaskTimeout = 0;
		FGuid TaskGuid;
		bool bRequestForTaskTimedOut;
		bool bDynamicObjectTask;
		int32 PrecomputedVisibilityTaskIndex;
		int32 VolumetricLightmapTaskIndex;
		bool bStaticShadowDepthMapTask;
		bool bMeshAreaLightDataTask;
		bool bVolumeDataTask;

		const double RequestTimeStart = FPlatformTime::Seconds();
		FStaticLightingMapping* Mapping = ThreadGetNextMapping( 
			ThreadStatistics, 
			TaskGuid, 
			DefaultRequestForTaskTimeout, 
			bRequestForTaskTimedOut, 
			bDynamicObjectTask, 
			PrecomputedVisibilityTaskIndex,
			VolumetricLightmapTaskIndex,
			bStaticShadowDepthMapTask,
			bMeshAreaLightDataTask,
			bVolumeDataTask);

		const double RequestTimeEnd = FPlatformTime::Seconds();
		ThreadStatistics.RequestTime += RequestTimeEnd - RequestTimeStart;
		if (Mapping)
		{
			const double MappingTimeStart = FPlatformTime::Seconds();
			// Build the mapping's static lighting.
			
			if(Mapping->GetTextureMapping())
			{
				check(!Mapping->GetVolumeMapping());
				check(!Mapping->GetLandscapeVolumeMapping());
				ProcessTextureMapping(Mapping->GetTextureMapping());
				double MappingTimeEnd = FPlatformTime::Seconds();
				ThreadStatistics.TextureMappingTime += MappingTimeEnd - MappingTimeStart;
				ThreadStatistics.NumTextureMappings++;
			}
		}
		else if (bDynamicObjectTask)
		{
			BeginCalculateVolumeSamples();

			// If we didn't generate any samples then we can end the task
			if (!IsDebugMode() && NumVolumeSampleTasksOutstanding <= 0)
			{
				FLightmassSwarm* Swarm = GetExporter().GetSwarm();
				Swarm->TaskCompleted(PrecomputedVolumeLightingGuid);
			}
		}
		else if (PrecomputedVisibilityTaskIndex >= 0)
		{
			CalculatePrecomputedVisibility(PrecomputedVisibilityTaskIndex);
		}
		else if (VolumetricLightmapTaskIndex >= 0)
		{
			CalculateAdaptiveVolumetricLightmap(VolumetricLightmapTaskIndex);
		}
		else if (bMeshAreaLightDataTask)
		{
			FPlatformAtomics::InterlockedExchange(&bShouldExportMeshAreaLightData, true);
		}
		else if (bVolumeDataTask)
		{
			BeginCalculateVolumeDistanceField();
		}
		else if (bStaticShadowDepthMapTask)
		{
			CalculateStaticShadowDepthMap(TaskGuid);
		}
		else
		{
			if (!bAnyTaskProcessedByThisThread && TasksInProgressThatWillNeedHelp <= 0 && NumOutstandingVolumeDataLayers <= 0 && NumVolumeSampleTasksOutstanding <= 0)
			{
				if (!bSignaledMappingsComplete)
				{
					bSignaledMappingsComplete = true;
					GSwarm->SendMessage(NSwarm::FTimingMessage(NSwarm::PROGSTATE_Processing0, ThreadIndex));
				}

				if (!bRequestForTaskTimedOut)
				{
					// All mappings have been processed, so end this thread.
					bIsDone = true;
				}
				else
				{
					FPlatformProcess::Sleep(.001f);
					IdleTime += FPlatformTime::Seconds() - StartLoopTime;
				}
			}
		}
		
		// NOTE: Main thread shouldn't be running this anymore.
		check( !bIsMainThread );
	}
	ThreadStatistics.TotalTime += FPlatformTime::Seconds() - ThreadTimeStart;
	FPlatformAtomics::InterlockedIncrement(&GStatistics.NumThreadsFinished);
}

/**
 * Applies the static lighting to the mappings in the list, and clears the list.
 * Also reports back to Unreal after each mapping has been exported.
 * @param LightingSystem - Reference to the static lighting system
 */
template<typename StaticLightingDataType>
void TCompleteStaticLightingList<StaticLightingDataType>::ApplyAndClear(FStaticLightingSystem& LightingSystem)
{
	while(FirstElement)
	{
		// Atomically read the complete list and clear the shared head pointer.
		TList<StaticLightingDataType>* LocalFirstElement;
		TList<StaticLightingDataType>* CurrentElement;
		uint32 ElementCount = 0;

		do { LocalFirstElement = FirstElement; }
		while(FPlatformAtomics::InterlockedCompareExchangePointer((void**)&FirstElement,NULL,LocalFirstElement) != LocalFirstElement);

		// Should never be null, but this satisfies static-analysis
		if (!LocalFirstElement)
			continue;

		// Traverse the local list, count the number of entries, and find the minimum guid
		TList<StaticLightingDataType>* PreviousElement = NULL;
		TList<StaticLightingDataType>* MinimumElementLink = NULL;
		TList<StaticLightingDataType>* MinimumElement = NULL;

		CurrentElement = LocalFirstElement;
		MinimumElement = CurrentElement;
		FGuid MinimumGuid = MinimumElement->Element.Mapping->Guid;

		while(CurrentElement)
		{
			ElementCount++;
			if (CurrentElement->Element.Mapping->Guid < MinimumGuid)
			{
				MinimumGuid = CurrentElement->Element.Mapping->Guid;
				MinimumElementLink = PreviousElement;
				MinimumElement = CurrentElement;
			}
			PreviousElement = CurrentElement;
			CurrentElement = CurrentElement->Next;
		}
		// Slice and dice the list to put the minimum at the head before we continue
		if (MinimumElementLink != NULL)
		{
			MinimumElementLink->Next = MinimumElement->Next;
			MinimumElement->Next = LocalFirstElement;
			LocalFirstElement = MinimumElement;
		}

		// Traverse the local list and export
		CurrentElement = LocalFirstElement;

		// Start exporting, planning to put everything into one file
		bool bUseUniqueChannel = true;
		if (LightingSystem.GetExporter().BeginExportResults(CurrentElement->Element, ElementCount) >= 0)
		{
			// We opened a group channel, export all mappings out together
			bUseUniqueChannel = false;
		}

		const double ExportTimeStart = FPlatformTime::Seconds();
		while(CurrentElement)
		{
			// write back to Unreal
			LightingSystem.GetExporter().ExportResults(CurrentElement->Element, bUseUniqueChannel);

			// Update the corresponding statistics depending on whether we're exporting in parallel to the worker threads or not.
			bool bIsRunningInParallel = GStatistics.NumThreadsFinished < (GStatistics.NumThreads-1);
			if ( bIsRunningInParallel )
			{
				GStatistics.ThreadStatistics.ExportTime += FPlatformTime::Seconds() - ExportTimeStart;
			}
			else
			{
				static bool bFirst = true;
				if ( bFirst )
				{
					bFirst = false;
					GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_ExportingResults, -1 ) );
				}
				GStatistics.ExtraExportTime += FPlatformTime::Seconds() - ExportTimeStart;
			}
			GStatistics.NumExportedMappings++;

			// Move to the next element
			CurrentElement = CurrentElement->Next;
		}

		// If we didn't use unique channels, close up the group channel now
		if (!bUseUniqueChannel)
		{
			LightingSystem.GetExporter().EndExportResults();
		}

		// Traverse again, cleaning up and notifying swarm
		FLightmassSwarm* Swarm = LightingSystem.GetExporter().GetSwarm();
		CurrentElement = LocalFirstElement;
		while(CurrentElement)
		{
			// Tell Swarm the task is complete (if we're not in debugging mode).
			if ( !LightingSystem.IsDebugMode() )
			{
				Swarm->TaskCompleted( CurrentElement->Element.Mapping->Guid );
			}

			// Delete this link and advance to the next.
			TList<StaticLightingDataType>* NextElement = CurrentElement->Next;
			delete CurrentElement;
			CurrentElement = NextElement;
		}
	}
}

template<typename DataType>
void TCompleteTaskList<DataType>::ApplyAndClear(FStaticLightingSystem& LightingSystem)
{
	while(this->FirstElement)
	{
		// Atomically read the complete list and clear the shared head pointer.
		TList<DataType>* LocalFirstElement;
		TList<DataType>* CurrentElement;
		uint32 ElementCount = 0;

		do { LocalFirstElement = this->FirstElement; }
		while(FPlatformAtomics::InterlockedCompareExchangePointer((void**)&this->FirstElement,NULL,LocalFirstElement) != LocalFirstElement);

		// Traverse the local list, count the number of entries, and find the minimum guid
		TList<DataType>* PreviousElement = NULL;
		TList<DataType>* MinimumElementLink = NULL;
		TList<DataType>* MinimumElement = NULL;

		CurrentElement = LocalFirstElement;
		MinimumElement = CurrentElement;
		FGuid MinimumGuid = MinimumElement->Element.Guid;

		while(CurrentElement)
		{
			ElementCount++;
			if (CurrentElement->Element.Guid < MinimumGuid)
			{
				MinimumGuid = CurrentElement->Element.Guid;
				MinimumElementLink = PreviousElement;
				MinimumElement = CurrentElement;
			}
			PreviousElement = CurrentElement;
			CurrentElement = CurrentElement->Next;
		}
		// Slice and dice the list to put the minimum at the head before we continue
		if (MinimumElementLink != NULL)
		{
			MinimumElementLink->Next = MinimumElement->Next;
			MinimumElement->Next = LocalFirstElement;
			LocalFirstElement = MinimumElement;
		}

		// Traverse the local list and export
		CurrentElement = LocalFirstElement;

		const double ExportTimeStart = FPlatformTime::Seconds();
		while(CurrentElement)
		{
			// write back to Unreal
			LightingSystem.GetExporter().ExportResults(CurrentElement->Element);

			// Move to the next element
			CurrentElement = CurrentElement->Next;
		}

		// Traverse again, cleaning up and notifying swarm
		FLightmassSwarm* Swarm = LightingSystem.GetExporter().GetSwarm();
		CurrentElement = LocalFirstElement;
		while(CurrentElement)
		{
			// Tell Swarm the task is complete (if we're not in debugging mode).
			if ( !LightingSystem.IsDebugMode() )
			{
				Swarm->TaskCompleted( CurrentElement->Element.Guid );
			}

			// Delete this link and advance to the next.
			TList<DataType>* NextElement = CurrentElement->Next;
			delete CurrentElement;
			CurrentElement = NextElement;
		}
	}
}

class FStoredLightingSample
{
public:
	FLinearColor IncomingRadiance;
	FVector4f WorldSpaceDirection;
};

class FSampleCollector
{
public:

	inline void SetOcclusion(float InOcclusion)
	{}

	inline void AddIncomingRadiance(const FLinearColor& IncomingRadiance, float Weight, const FVector4f& TangentSpaceDirection, const FVector4f& WorldSpaceDirection)
	{
		if (FLinearColorUtils::LinearRGBToXYZ(IncomingRadiance * Weight).G > DELTA)
		{
			FStoredLightingSample NewSample;
			NewSample.IncomingRadiance = IncomingRadiance * Weight;
			NewSample.WorldSpaceDirection = WorldSpaceDirection;
			Samples.Add(NewSample);
		}
	}

	bool AreFloatsValid() const
	{
		return true;
	}

	FSampleCollector operator+( const FSampleCollector& Other ) const
	{
		FSampleCollector NewCollector;
		NewCollector.Samples = Samples;
		NewCollector.Samples.Append(Other.Samples);
		NewCollector.EnvironmentSamples = EnvironmentSamples;
		NewCollector.EnvironmentSamples.Append(Other.EnvironmentSamples);
		return NewCollector;
	}

	TArray<FStoredLightingSample> Samples;
	TArray<FStoredLightingSample> EnvironmentSamples;
};

void FStaticLightingSystem::CalculateStaticShadowDepthMap(FGuid LightGuid)
{
	const FLight* Light = Scene.FindLightByGuid(LightGuid);
	check(Light);
	const FDirectionalLight* DirectionalLight = Light->GetDirectionalLight();
	const FSpotLight* SpotLight = Light->GetSpotLight();
	const FPointLight* PointLight = Light->GetPointLight();
	check(DirectionalLight || SpotLight || PointLight);
	const float ClampedResolutionScale = FMath::Clamp(Light->ShadowResolutionScale, .125f, 8.0f);

	const double StartTime = FPlatformTime::Seconds();

	FStaticLightingMappingContext Context(NULL, *this);
	FStaticShadowDepthMap* ShadowDepthMap = new FStaticShadowDepthMap();

	if (DirectionalLight)
	{
		FVector4f XAxis, YAxis;
		DirectionalLight->Direction.FindBestAxisVectors3(XAxis, YAxis);
		// Create a coordinate system for the dominant directional light, with the z axis corresponding to the light's direction
		ShadowDepthMap->WorldToLight = FBasisVectorMatrix44f(XAxis, YAxis, DirectionalLight->Direction, FVector4f(0,0,0));

		FBoxSphereBounds3f ImportanceVolume = GetImportanceBounds().SphereRadius > 0.0f ? GetImportanceBounds() : FBoxSphereBounds3f(AggregateMesh->GetBounds());
		const FBox3f LightSpaceImportanceBounds = ImportanceVolume.GetBox().TransformBy(ShadowDepthMap->WorldToLight);

		ShadowDepthMap->ShadowMapSizeX = FMath::TruncToInt(FMath::Max(LightSpaceImportanceBounds.GetExtent().X * 2.0f * ClampedResolutionScale / ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceX, 4.0f));
		ShadowDepthMap->ShadowMapSizeX = ShadowDepthMap->ShadowMapSizeX == appTruncErrorCode ? INT_MAX : ShadowDepthMap->ShadowMapSizeX;
		ShadowDepthMap->ShadowMapSizeY = FMath::TruncToInt(FMath::Max(LightSpaceImportanceBounds.GetExtent().Y * 2.0f * ClampedResolutionScale / ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceY, 4.0f));
		ShadowDepthMap->ShadowMapSizeY = ShadowDepthMap->ShadowMapSizeY == appTruncErrorCode ? INT_MAX : ShadowDepthMap->ShadowMapSizeY;

		// Clamp the number of dominant shadow samples generated if necessary while maintaining aspect ratio
		if ((uint64)ShadowDepthMap->ShadowMapSizeX * (uint64)ShadowDepthMap->ShadowMapSizeY > (uint64)ShadowSettings.StaticShadowDepthMapMaxSamples)
		{
			const float AspectRatio = ShadowDepthMap->ShadowMapSizeX / (float)ShadowDepthMap->ShadowMapSizeY;
			ShadowDepthMap->ShadowMapSizeY = FMath::TruncToInt(FMath::Sqrt(ShadowSettings.StaticShadowDepthMapMaxSamples / AspectRatio));
			ShadowDepthMap->ShadowMapSizeX = FMath::TruncToInt((float)ShadowSettings.StaticShadowDepthMapMaxSamples / (float)ShadowDepthMap->ShadowMapSizeY);
		}

		// Allocate the shadow map
		ShadowDepthMap->ShadowMap.Empty(ShadowDepthMap->ShadowMapSizeX * ShadowDepthMap->ShadowMapSizeY);
		ShadowDepthMap->ShadowMap.AddZeroed(ShadowDepthMap->ShadowMapSizeX * ShadowDepthMap->ShadowMapSizeY);

		{
			const float InvDistanceRange = 1.0f / (LightSpaceImportanceBounds.Max.Z - LightSpaceImportanceBounds.Min.Z);
			const FMatrix44f LightToWorld = ShadowDepthMap->WorldToLight.InverseFast();

			for (int32 Y = 0; Y < ShadowDepthMap->ShadowMapSizeY; Y++)
			{
				for (int32 X = 0; X < ShadowDepthMap->ShadowMapSizeX; X++)
				{
					float MaxSampleDistance = 0.0f;
					// Super sample each cell
					for (int32 SubSampleY = 0; SubSampleY < ShadowSettings.StaticShadowDepthMapSuperSampleFactor; SubSampleY++)
					{
						const float YFraction = (Y + SubSampleY / (float)ShadowSettings.StaticShadowDepthMapSuperSampleFactor) / (float)(ShadowDepthMap->ShadowMapSizeY - 1);
						for (int32 SubSampleX = 0; SubSampleX < ShadowSettings.StaticShadowDepthMapSuperSampleFactor; SubSampleX++)
						{
							const float XFraction = (X + SubSampleX / (float)ShadowSettings.StaticShadowDepthMapSuperSampleFactor) / (float)(ShadowDepthMap->ShadowMapSizeX - 1);
							// Construct a ray in light space along the direction of the light, starting at the minimum light space Z going to the maximum.
							const FVector4f LightSpaceStartPosition(
								LightSpaceImportanceBounds.Min.X + XFraction * (LightSpaceImportanceBounds.Max.X - LightSpaceImportanceBounds.Min.X),
								LightSpaceImportanceBounds.Min.Y + YFraction * (LightSpaceImportanceBounds.Max.Y - LightSpaceImportanceBounds.Min.Y),
								LightSpaceImportanceBounds.Min.Z);
							const FVector4f LightSpaceEndPosition(LightSpaceStartPosition.X, LightSpaceStartPosition.Y, LightSpaceImportanceBounds.Max.Z);
							// Transform the ray into world space in order to trace against the world space aggregate mesh
							const FVector4f WorldSpaceStartPosition = LightToWorld.TransformPosition(LightSpaceStartPosition);
							const FVector4f WorldSpaceEndPosition = LightToWorld.TransformPosition(LightSpaceEndPosition);
							const FLightRay LightRay(
								WorldSpaceStartPosition,
								WorldSpaceEndPosition,
								NULL,
								NULL,
								// We are tracing from the light instead of to the light,
								// So flip sidedness so that backface culling matches up with tracing to the light
								LIGHTRAY_FLIP_SIDEDNESS
								);

							FLightRayIntersection Intersection;
							AggregateMesh->IntersectLightRay(LightRay, true, false, true, Context.RayCache, Intersection);

							if (Intersection.bIntersects)
							{
								// Use the maximum distance of all super samples for each cell, to get a conservative shadow map
								MaxSampleDistance = FMath::Max(MaxSampleDistance, (Intersection.IntersectionVertex.WorldPosition - WorldSpaceStartPosition).Size3());
							}
						}
					}

					if (MaxSampleDistance == 0.0f)
					{
						MaxSampleDistance = LightSpaceImportanceBounds.Max.Z - LightSpaceImportanceBounds.Min.Z;
					} 

					ShadowDepthMap->ShadowMap[Y * ShadowDepthMap->ShadowMapSizeX + X] = FStaticShadowDepthMapSample(FFloat16(MaxSampleDistance * InvDistanceRange));
				}
			}
		}

		ShadowDepthMap->WorldToLight *= FTranslationMatrix44f(-LightSpaceImportanceBounds.Min)
			* FScaleMatrix44f(FVector3f(1.0f) / (LightSpaceImportanceBounds.Max - LightSpaceImportanceBounds.Min));

		FScopeLock Lock(&CompletedStaticShadowDepthMapsSync);
		CompletedStaticShadowDepthMaps.Add(DirectionalLight, ShadowDepthMap);
	}
	else if (SpotLight)
	{
		FVector4f XAxis, YAxis;
		SpotLight->Direction.FindBestAxisVectors3(XAxis, YAxis);
		// Create a coordinate system for the spot light, with the z axis corresponding to the light's direction, and translated to the light's origin
		ShadowDepthMap->WorldToLight = FTranslationMatrix44f(-SpotLight->Position) 
			* FBasisVectorMatrix44f(XAxis, YAxis, SpotLight->Direction, FVector4f(0,0,0));

		// Distance from the light's direction axis to the edge of the cone at the radius of the light
		const float HalfCrossSectionLength = SpotLight->Radius * FMath::Tan(SpotLight->OuterConeAngle * (float)PI / 180.0f);

		const FVector4f LightSpaceImportanceBoundMin = FVector4f(-HalfCrossSectionLength, -HalfCrossSectionLength, 0);
		const FVector4f LightSpaceImportanceBoundMax = FVector4f(HalfCrossSectionLength, HalfCrossSectionLength, SpotLight->Radius);

		ShadowDepthMap->ShadowMapSizeX = FMath::TruncToInt(FMath::Max(HalfCrossSectionLength * ClampedResolutionScale / ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceX, 4.0f));
		ShadowDepthMap->ShadowMapSizeX = ShadowDepthMap->ShadowMapSizeX == appTruncErrorCode ? INT_MAX : ShadowDepthMap->ShadowMapSizeX;
		ShadowDepthMap->ShadowMapSizeY = ShadowDepthMap->ShadowMapSizeX;

		// Clamp the number of dominant shadow samples generated if necessary while maintaining aspect ratio
		if ((uint64)ShadowDepthMap->ShadowMapSizeX * (uint64)ShadowDepthMap->ShadowMapSizeY > (uint64)ShadowSettings.StaticShadowDepthMapMaxSamples)
		{
			const float AspectRatio = ShadowDepthMap->ShadowMapSizeX / (float)ShadowDepthMap->ShadowMapSizeY;
			ShadowDepthMap->ShadowMapSizeY = FMath::TruncToInt(FMath::Sqrt(ShadowSettings.StaticShadowDepthMapMaxSamples / AspectRatio));
			ShadowDepthMap->ShadowMapSizeX = FMath::TruncToInt((float)ShadowSettings.StaticShadowDepthMapMaxSamples / (float)ShadowDepthMap->ShadowMapSizeY);
		}

		ShadowDepthMap->ShadowMap.Empty(ShadowDepthMap->ShadowMapSizeX * ShadowDepthMap->ShadowMapSizeY);
		ShadowDepthMap->ShadowMap.AddZeroed(ShadowDepthMap->ShadowMapSizeX * ShadowDepthMap->ShadowMapSizeY);

		// Calculate the maximum possible distance for quantization
		const float MaxPossibleDistance = LightSpaceImportanceBoundMax.Z - LightSpaceImportanceBoundMin.Z;
		const FMatrix44f LightToWorld = ShadowDepthMap->WorldToLight.InverseFast();
		const FBoxSphereBounds3f ImportanceVolume = GetImportanceBounds().SphereRadius > 0.0f ? GetImportanceBounds() : FBoxSphereBounds3f(AggregateMesh->GetBounds());

		for (int32 Y = 0; Y < ShadowDepthMap->ShadowMapSizeY; Y++)
		{
			for (int32 X = 0; X < ShadowDepthMap->ShadowMapSizeX; X++)
			{
				float MaxSampleDistance = 0.0f;
				// Super sample each cell
				for (int32 SubSampleY = 0; SubSampleY < ShadowSettings.StaticShadowDepthMapSuperSampleFactor; SubSampleY++)
				{
					const float YFraction = (Y + SubSampleY / (float)ShadowSettings.StaticShadowDepthMapSuperSampleFactor) / (float)(ShadowDepthMap->ShadowMapSizeY - 1);
					for (int32 SubSampleX = 0; SubSampleX < ShadowSettings.StaticShadowDepthMapSuperSampleFactor; SubSampleX++)
					{
						const float XFraction = (X + SubSampleX / (float)ShadowSettings.StaticShadowDepthMapSuperSampleFactor) / (float)(ShadowDepthMap->ShadowMapSizeX - 1);
						// Construct a ray in light space along the direction of the light, starting at the light and going to the maximum light space Z.
						const FVector4f LightSpaceStartPosition(0,0,0);
						const FVector4f LightSpaceEndPosition(
							LightSpaceImportanceBoundMin.X + XFraction * (LightSpaceImportanceBoundMax.X - LightSpaceImportanceBoundMin.X),
							LightSpaceImportanceBoundMin.Y + YFraction * (LightSpaceImportanceBoundMax.Y - LightSpaceImportanceBoundMin.Y),
							LightSpaceImportanceBoundMax.Z);
						// Transform the ray into world space in order to trace against the world space aggregate mesh
						const FVector4f WorldSpaceStartPosition = LightToWorld.TransformPosition(LightSpaceStartPosition);
						const FVector4f WorldSpaceEndPosition = LightToWorld.TransformPosition(LightSpaceEndPosition);
						const FLightRay LightRay(
							WorldSpaceStartPosition,
							WorldSpaceEndPosition,
							NULL,
							NULL,
							// We are tracing from the light instead of to the light,
							// So flip sidedness so that backface culling matches up with tracing to the light
							LIGHTRAY_FLIP_SIDEDNESS
							);

						FLightRayIntersection Intersection;
						AggregateMesh->IntersectLightRay(LightRay, true, false, true, Context.RayCache, Intersection);

						if (Intersection.bIntersects)
						{
							const FVector4f LightSpaceIntersectPosition = ShadowDepthMap->WorldToLight.TransformPosition(Intersection.IntersectionVertex.WorldPosition);
							// Use the maximum distance of all super samples for each cell, to get a conservative shadow map
							MaxSampleDistance = FMath::Max(MaxSampleDistance, LightSpaceIntersectPosition.Z);
						}
					}
				}

				if (MaxSampleDistance == 0.0f)
				{
					MaxSampleDistance = MaxPossibleDistance;
				}

				ShadowDepthMap->ShadowMap[Y * ShadowDepthMap->ShadowMapSizeX + X] = FStaticShadowDepthMapSample(FFloat16(MaxSampleDistance / MaxPossibleDistance));
			}
		}

		ShadowDepthMap->WorldToLight *= 
			// Perspective projection sized to the spotlight cone
			FPerspectiveMatrix44f(SpotLight->OuterConeAngle * (float)PI / 180.0f, 1, 1, 0, SpotLight->Radius)
			// Convert from NDC to texture space, normalize Z
			* FMatrix44f(
				FPlane4f(.5f,								0,							0,									0),
				FPlane4f(0,								.5f,						0,									0),
				FPlane4f(0,								0,							1.0f / LightSpaceImportanceBoundMax.Z,		0),
				FPlane4f(.5f,								.5f,						0,									1));

		FScopeLock Lock(&CompletedStaticShadowDepthMapsSync);
		CompletedStaticShadowDepthMaps.Add(SpotLight, ShadowDepthMap);
	}
	else if (PointLight)
	{
		ShadowDepthMap->ShadowMapSizeX = FMath::TruncToInt(FMath::Max(PointLight->Radius * 4 * ClampedResolutionScale / ShadowSettings.StaticShadowDepthMapTransitionSampleDistanceX, 4.0f));
		ShadowDepthMap->ShadowMapSizeX = ShadowDepthMap->ShadowMapSizeX == appTruncErrorCode ? INT_MAX : ShadowDepthMap->ShadowMapSizeX;
		ShadowDepthMap->ShadowMapSizeY = ShadowDepthMap->ShadowMapSizeX;

		// Clamp the number of dominant shadow samples generated if necessary while maintaining aspect ratio
		if ((uint64)ShadowDepthMap->ShadowMapSizeX * (uint64)ShadowDepthMap->ShadowMapSizeY > (uint64)ShadowSettings.StaticShadowDepthMapMaxSamples)
		{
			const float AspectRatio = ShadowDepthMap->ShadowMapSizeX / (float)ShadowDepthMap->ShadowMapSizeY;
			ShadowDepthMap->ShadowMapSizeY = FMath::TruncToInt(FMath::Sqrt(ShadowSettings.StaticShadowDepthMapMaxSamples / AspectRatio));
			ShadowDepthMap->ShadowMapSizeX = FMath::TruncToInt((float)ShadowSettings.StaticShadowDepthMapMaxSamples / (float)ShadowDepthMap->ShadowMapSizeY);
		}

		// Allocate the shadow map
		ShadowDepthMap->ShadowMap.Empty(ShadowDepthMap->ShadowMapSizeX * ShadowDepthMap->ShadowMapSizeY);
		ShadowDepthMap->ShadowMap.AddZeroed(ShadowDepthMap->ShadowMapSizeX * ShadowDepthMap->ShadowMapSizeY);

		ShadowDepthMap->WorldToLight = FMatrix44f::Identity;

		for (int32 Y = 0; Y < ShadowDepthMap->ShadowMapSizeY; Y++)
		{
			for (int32 X = 0; X < ShadowDepthMap->ShadowMapSizeX; X++)
			{
				float MaxSampleDistance = 0.0f;
				// Super sample each cell
				for (int32 SubSampleY = 0; SubSampleY < ShadowSettings.StaticShadowDepthMapSuperSampleFactor; SubSampleY++)
				{
					const float YFraction = (Y + SubSampleY / (float)ShadowSettings.StaticShadowDepthMapSuperSampleFactor) / (float)(ShadowDepthMap->ShadowMapSizeY - 1);
					const float Phi = YFraction * PI;
					const float SinPhi = FMath::Sin(Phi);

					for (int32 SubSampleX = 0; SubSampleX < ShadowSettings.StaticShadowDepthMapSuperSampleFactor; SubSampleX++)
					{
						const float XFraction = (X + SubSampleX / (float)ShadowSettings.StaticShadowDepthMapSuperSampleFactor) / (float)(ShadowDepthMap->ShadowMapSizeX - 1);
						const float Theta = XFraction * 2 * PI;
						const FVector3f Direction(FMath::Cos(Theta) * SinPhi, FMath::Sin(Theta) * SinPhi, FMath::Cos(Phi));

						const FVector4f WorldSpaceStartPosition = PointLight->Position;
						const FVector4f WorldSpaceEndPosition = PointLight->Position + Direction * PointLight->Radius;
						const FLightRay LightRay(
							WorldSpaceStartPosition,
							WorldSpaceEndPosition,
							NULL,
							NULL,
							// We are tracing from the light instead of to the light,
							// So flip sidedness so that backface culling matches up with tracing to the light
							LIGHTRAY_FLIP_SIDEDNESS
							);

						FLightRayIntersection Intersection;
						AggregateMesh->IntersectLightRay(LightRay, true, false, true, Context.RayCache, Intersection);

						if (Intersection.bIntersects)
						{
							// Use the maximum distance of all super samples for each cell, to get a conservative shadow map
							MaxSampleDistance = FMath::Max(MaxSampleDistance, (Intersection.IntersectionVertex.WorldPosition - PointLight->Position).Size3());
						}
					}
				}

				if (MaxSampleDistance == 0.0f)
				{
					MaxSampleDistance = PointLight->Radius;
				}

				ShadowDepthMap->ShadowMap[Y * ShadowDepthMap->ShadowMapSizeX + X] = FStaticShadowDepthMapSample(FFloat16(MaxSampleDistance / PointLight->Radius));
			}
		}

		FScopeLock Lock(&CompletedStaticShadowDepthMapsSync);
		CompletedStaticShadowDepthMaps.Add(PointLight, ShadowDepthMap);
	}

	const float NewTime = FPlatformTime::Seconds() - StartTime;
	Context.Stats.StaticShadowDepthMapThreadTime = NewTime;
	Context.Stats.MaxStaticShadowDepthMapThreadTime = NewTime;
}

/**
 * Calculates shadowing for a given mapping surface point and light.
 * @param Mapping - The mapping the point comes from.
 * @param WorldSurfacePoint - The point to check shadowing at.
 * @param Light - The light to check shadowing from.
 * @param CoherentRayCache - The calling thread's collision cache.
 * @return true if the surface point is shadowed from the light.
 */
bool FStaticLightingSystem::CalculatePointShadowing(
	const FStaticLightingMapping* Mapping,
	const FVector4f& WorldSurfacePoint,
	const FLight* Light,
	FStaticLightingMappingContext& MappingContext,
	bool bDebugThisSample
	) const
{
	if(Light->GetSkyLight())
	{
		return true;
	}
	else
	{
		// Treat points which the light doesn't affect as shadowed to avoid the costly ray check.
		if(!Light->AffectsBounds(FBoxSphereBounds3f(WorldSurfacePoint,FVector4f(0,0,0,0),0)))
		{
			return true;
		}

		// Check for visibility between the point and the light.
		bool bIsShadowed = false;
		if ((Light->LightFlags & GI_LIGHT_CASTSHADOWS) && (Light->LightFlags & GI_LIGHT_CASTSTATICSHADOWS))
		{
			// TODO find best point on light to shadow from
			// Construct a line segment between the light and the surface point.
			const FVector4f LightPosition = FVector4f(Light->Position.X, Light->Position.Y, Light->Position.Z, 0);
			const FVector4f LightVector = LightPosition - WorldSurfacePoint * Light->Position.W;
			const FLightRay LightRay(
				WorldSurfacePoint + LightVector.GetSafeNormal() * SceneConstants.VisibilityRayOffsetDistance,
				WorldSurfacePoint + LightVector,
				Mapping,
				Light
				);

			// Check the line segment for intersection with the static lighting meshes.
			FLightRayIntersection Intersection;
			AggregateMesh->IntersectLightRay(LightRay, false, false, true, MappingContext.RayCache, Intersection);
			bIsShadowed = Intersection.bIntersects;

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisSample)
			{
				FDebugStaticLightingRay DebugRay(LightRay.Start, LightRay.End, bIsShadowed != 0);
				if (bIsShadowed)
				{
					DebugRay.End = Intersection.IntersectionVertex.WorldPosition;
				}
				MappingContext.DebugOutput->ShadowRays.Add(DebugRay);
			}
#endif
		}

		return bIsShadowed;
	}
}

/** Calculates area shadowing from a light for the given vertex. */
FVector2f FStaticLightingSystem::CalculatePointAreaShadowing(
	const FStaticLightingMapping* Mapping,
	const FStaticLightingVertex& Vertex,
	int32 ElementIndex,
	float SampleRadius,
	const FLight* Light,
	FStaticLightingMappingContext& MappingContext,
	FLMRandomStream& RandomStream,
	FLinearColor& UnnormalizedTransmission,
	const TArray<FLightSurfaceSample>& LightPositionSamples,
	bool bDebugThisSample
	) const
{
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisSample)
	{
		int32 TempBreak = 0;
	}
#endif

	UnnormalizedTransmission = FLinearColor::Black;
	// Treat points which the light doesn't affect as shadowed to avoid the costly ray check.
	if( !Light->AffectsBounds(FBoxSphereBounds3f(Vertex.WorldPosition,FVector4f(0,0,0),0)))
	{
		return FVector2f::ZeroVector;
	}

	// Check for visibility between the point and the light
	if ((Light->LightFlags & GI_LIGHT_CASTSHADOWS) && (Light->LightFlags & GI_LIGHT_CASTSTATICSHADOWS))
	{
		MappingContext.Stats.NumDirectLightingShadowRays += LightPositionSamples.Num();
		const bool bIsTwoSided = Mapping->Mesh->IsTwoSided(ElementIndex);
		FVector2f ShadowedValue = FVector2f::ZeroVector;

		// Integrate over the surface of the light using monte carlo integration
		// Note that we are making the approximation that the BRDF and the Light's emission are equal in all of these directions and therefore are not in the integrand
		for(int32 RayIndex = 0; RayIndex < LightPositionSamples.Num(); RayIndex++)
		{
			FLightSurfaceSample CurrentSample = LightPositionSamples[RayIndex];
			// Allow the light to modify the surface position for this receiving position
			Light->ValidateSurfaceSample(Vertex.WorldPosition, CurrentSample);

			// Construct a line segment between the light and the surface point.
			const FVector4f LightVector = CurrentSample.Position - Vertex.WorldPosition;
			FVector4f SampleOffset(0,0,0);
			if (GeneralSettings.bAccountForTexelSize)
			{
				/*
				//@todo - the rays cross over on the way to the light and mess up penumbra shapes.  
				//@todo - need to use more than texel size, otherwise BSP generates lots of texels that become half shadowed at corners
				SampleOffset = Vertex.WorldTangentX * LightPositionSamples(RayIndex).DiskPosition.X * SampleRadius * SceneConstants.VisibilityTangentOffsetSampleRadiusScale
					+ Vertex.WorldTangentY * LightPositionSamples(RayIndex).DiskPosition.Y * SampleRadius * SceneConstants.VisibilityTangentOffsetSampleRadiusScale;
					*/
			}

			float DistSqr = LightVector.SizeSquared3() + KINDA_SMALL_NUMBER;
			float SamplePDF = CurrentSample.PDF * -Dot3( CurrentSample.Normal, LightVector ) / ( DistSqr * FMath::Sqrt( DistSqr ) );
			if( SamplePDF <= 0.0f )
			{
				continue;
			}

			FVector4f NormalForOffset = Vertex.WorldTangentZ;
			// Flip the normal used for offsetting the start of the ray for two sided materials if a flipped normal would be closer to the light.
			// This prevents incorrect shadowing where using the frontface normal would cause the ray to start inside a nearby object.
			if (bIsTwoSided && Dot3(-NormalForOffset, LightVector) > Dot3(NormalForOffset, LightVector))
			{
				NormalForOffset = -NormalForOffset;
			}
			
			const FLightRay LightRay(
				// Offset the start of the ray by some fraction along the direction of the ray and some fraction along the vertex normal.
				Vertex.WorldPosition 
					+ LightVector.GetSafeNormal() * SceneConstants.VisibilityRayOffsetDistance 
					+ NormalForOffset * SampleRadius * SceneConstants.VisibilityNormalOffsetSampleRadiusScale 
					+ SampleOffset,
				Vertex.WorldPosition + LightVector,
				Mapping,
				Light
				);

			// Check the line segment for intersection with the static lighting meshes.
			FLightRayIntersection Intersection;
			//@todo - change this back to request boolean visibility once transmission is supported with boolean visibility ray intersections
			AggregateMesh->IntersectLightRay(LightRay, true, true, true, MappingContext.RayCache, Intersection);

			if (!Intersection.bIntersects)
			{
				UnnormalizedTransmission += Intersection.Transmission * SamplePDF;
				ShadowedValue.X += SamplePDF;
			}
			ShadowedValue.Y += SamplePDF;

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisSample)
			{
				FDebugStaticLightingRay DebugRay(LightRay.Start, LightRay.End, Intersection.bIntersects);
				if (Intersection.bIntersects)
				{
					DebugRay.End = Intersection.IntersectionVertex.WorldPosition;
				}
				MappingContext.DebugOutput->ShadowRays.Add(DebugRay);
			}
#endif
		}

		return ShadowedValue;
	}
	UnnormalizedTransmission = FLinearColor::White;
	return FVector2f( 1.0f, 1.0f );
}

/** Calculates the lighting contribution of a light to a mapping vertex. */
FGatheredLightSample FStaticLightingSystem::CalculatePointLighting(
	const FStaticLightingMapping* Mapping, 
	const FStaticLightingVertex& Vertex, 
	int32 ElementIndex,
	const FLight* Light,
	const FLinearColor& InLightIntensity,
	const FLinearColor& InTransmission
	) const
{
	// don't do sky lights here
	if (Light->GetSkyLight() == NULL)
	{
	    // Calculate the direction from the vertex to the light.
		const FVector4f WorldLightVector = Light->GetDirectLightingDirection(Vertex.WorldPosition, Vertex.WorldTangentZ);
    
	    // Transform the light vector to tangent space.
	    const FVector4f TangentLightVector = 
		    FVector4f(
			    Dot3(WorldLightVector, Vertex.WorldTangentX),
			    Dot3(WorldLightVector, Vertex.WorldTangentY),
			    Dot3(WorldLightVector, Vertex.WorldTangentZ),
				0
				).GetSafeNormal();

		// Compute the incident lighting of the light on the vertex.
		const FLinearColor LightIntensity = InLightIntensity * InTransmission;

		// Compute the light-map sample for the front-face of the vertex.
		FGatheredLightSample FrontFaceSample = FGatheredLightSampleUtil::PointLightWorldSpace<2>(LightIntensity, TangentLightVector, WorldLightVector.GetSafeNormal());

		if (Mapping->Mesh->UsesTwoSidedLighting(ElementIndex))
		{
			const FVector4f BackFaceTangentLightVector = 
				FVector4f(
					Dot3(WorldLightVector, -Vertex.WorldTangentX),
					Dot3(WorldLightVector, -Vertex.WorldTangentY),
					Dot3(WorldLightVector, -Vertex.WorldTangentZ),
					0
					).GetSafeNormal();
			const FGatheredLightSample BackFaceSample = FGatheredLightSampleUtil::PointLightWorldSpace<2>(LightIntensity, BackFaceTangentLightVector, -WorldLightVector.GetSafeNormal());
			// Average front and back face lighting
			return (FrontFaceSample + BackFaceSample) * .5f;
		}
		else
		{
			return FrontFaceSample;
		}
	}

	return FGatheredLightSample();
}

/** Returns a light sample that represents the material attribute specified by MaterialSettings.ViewMaterialAttribute at the intersection. */
FGatheredLightSample FStaticLightingSystem::GetVisualizedMaterialAttribute(const FStaticLightingMapping* Mapping, const FLightRayIntersection& Intersection) const
{
	FGatheredLightSample MaterialSample;
	if (Intersection.bIntersects && Intersection.Mapping == Mapping)
	{
		// The ray intersected an opaque surface, we can visualize anything that opaque materials store
		//@todo - Currently can't visualize emissive on translucent materials
		if (MaterialSettings.ViewMaterialAttribute == VMA_Emissive)
		{
			FLinearColor Emissive = FLinearColor::Black;
			if (Intersection.Mesh->IsEmissive(Intersection.ElementIndex))
			{
				Emissive = Intersection.Mesh->EvaluateEmissive(Intersection.IntersectionVertex.TextureCoordinates[0], Intersection.ElementIndex);
			}
			MaterialSample = FGatheredLightSampleUtil::AmbientLight<2>(Emissive);
		}
		else if (MaterialSettings.ViewMaterialAttribute == VMA_Diffuse)
		{
			const FLinearColor Diffuse = Intersection.Mesh->EvaluateDiffuse(Intersection.IntersectionVertex.TextureCoordinates[0], Intersection.ElementIndex);
			MaterialSample = FGatheredLightSampleUtil::AmbientLight<2>(Diffuse);
		}
		else if (MaterialSettings.ViewMaterialAttribute == VMA_Normal)
		{
			const FVector4f Normal = Intersection.Mesh->EvaluateNormal(Intersection.IntersectionVertex.TextureCoordinates[0], Intersection.ElementIndex);

			FLinearColor NormalColor;
			NormalColor.R = Normal.X * 0.5f + 0.5f;
			NormalColor.G = Normal.Y * 0.5f + 0.5f;
			NormalColor.B = Normal.Z * 0.5f + 0.5f;
			NormalColor.A = 1.0f;

			MaterialSample = FGatheredLightSampleUtil::AmbientLight<2>(NormalColor);
		}
	}
	else if (MaterialSettings.ViewMaterialAttribute != VMA_Transmission)
	{
		// The ray didn't intersect an opaque surface and we're not visualizing transmission
		MaterialSample = FGatheredLightSampleUtil::AmbientLight<2>(FLinearColor::Black);
	}

	if (MaterialSettings.ViewMaterialAttribute == VMA_Transmission)
	{
		// Visualizing transmission, replace the light sample with the transmission picked up along the ray
		MaterialSample = FGatheredLightSampleUtil::AmbientLight<2>(Intersection.Transmission);
	}
	return MaterialSample;
}

/**
 * Checks if a light is behind a triangle.
 * @param TrianglePoint - Any point on the triangle.
 * @param TriangleNormal - The (not necessarily normalized) triangle surface normal.
 * @param Light - The light to classify.
 * @return true if the light is behind the triangle.
 */
bool IsLightBehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal, const FLight* Light)
{
	const bool bIsSkyLight = Light->GetSkyLight() != NULL;
	if (!bIsSkyLight)
	{
		// Calculate the direction from the triangle to the light.
		const FVector4f LightPosition = FVector4f(Light->Position.X, Light->Position.Y, Light->Position.Z, 0);
		const FVector4f WorldLightVector = LightPosition - TrianglePoint * Light->Position.W;

		// Check if the light is in front of the triangle.
		const float Dot = Dot3(WorldLightVector, TriangleNormal);
		return Dot < 0.0f;
	}
	else
	{
		// Sky lights are always in front of a surface.
		return false;
	}
}

/**
 * Culls lights that are behind a triangle.
 * @param bTwoSidedMaterial - true if the triangle has a two-sided material.  If so, lights behind the surface are not culled.
 * @param TrianglePoint - Any point on the triangle.
 * @param TriangleNormal - The (not necessarily normalized) triangle surface normal.
 * @param Lights - The lights to cull.
 * @return A map from Lights index to a boolean which is true if the light is in front of the triangle.
 */
TBitArray<> CullBackfacingLights(bool bTwoSidedMaterial,const FVector4f& TrianglePoint,const FVector4f& TriangleNormal,const TArray<FLight*>& Lights)
{
	if(!bTwoSidedMaterial)
	{
		TBitArray<> Result(false,Lights.Num());
		for(int32 LightIndex = 0;LightIndex < Lights.Num();LightIndex++)
		{
			Result[LightIndex] = !IsLightBehindSurface(TrianglePoint,TriangleNormal,Lights[LightIndex]);
		}
		return Result;
	}
	else
	{
		return TBitArray<>(true,Lights.Num());
	}
}


} //namespace Lightmass

