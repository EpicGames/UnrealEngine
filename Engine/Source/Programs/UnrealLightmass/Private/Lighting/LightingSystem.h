// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CPUSolver.h"
#include "ImportExport.h"
#include "LightingCache.h"
#include "Containers/ChunkedArray.h"
#include "Containers/List.h"
#include "HAL/Runnable.h"
#include "LightmapData.h"
#include "Math/PackedVector.h"
#include "GatheredLightingSample.h"
#include "LightmassScene.h"

namespace Lightmass
{

/** Whether to allow static lighting stats that may affect the system's performance. */
#define ALLOW_STATIC_LIGHTING_STATS 1

/** 
 * When enabled, makes Lightmass do pretty much no processing at all (NOP). 
 * Useful when iterating on import / export
 */
#define LIGHTMASS_DO_PROCESSING 1

#if ALLOW_STATIC_LIGHTING_STATS
	#define LIGHTINGSTAT(x) x
#else
	#define LIGHTINGSTAT(x)
#endif

/**
 * The raw data which is used to construct a 2D light-map.
 */
class FGatheredLightMapData2D
{
public:

	/** The width of the light-map. */
	uint32 SizeX;
	/** The height of the light-map. */
	uint32 SizeY;

	/** The lights which this light-map stores. */
	TArray<const FLight*> Lights;

	bool bHasSkyShadowing;

	FGatheredLightMapData2D(uint32 InSizeX, uint32 InSizeY) :
		SizeX(InSizeX),
		SizeY(InSizeY),
		bHasSkyShadowing(false)
	{
		Data.Empty(SizeX * SizeY);
		Data.AddZeroed(SizeX * SizeY);
	}

	// Accessors.
	const FGatheredLightMapSample& operator()(uint32 X,uint32 Y) const { return Data[SizeX * Y + X]; }
	FGatheredLightMapSample& operator()(uint32 X,uint32 Y) { return Data[SizeX * Y + X]; }
	uint32 GetSizeX() const { return SizeX; }
	uint32 GetSizeY() const { return SizeY; }

	void Empty()
	{
		Data.Empty();
		SizeX = SizeY = 0;
		Lights.Empty();
	}

	void AddLight(const FLight* NewLight)
	{
		Lights.AddUnique(NewLight);
	}

	FLightMapData2D* ConvertToLightmap2D(bool bDebugThisMapping, int32 PaddedDebugX, int32 PaddedDebugY) const;

private:
	TArray<FGatheredLightMapSample> Data;
};

struct FFinalGatherInfo
{
	FFinalGatherInfo() :
		NumBackfaceHits(0),
		NumSamplesOccluded(0)
	{}

	int32 NumBackfaceHits;
	float NumSamplesOccluded;
};

struct FTexelCorner
{
	FVector4f WorldPosition;
};

/** Information about a texel's corners */
struct FTexelToCorners
{ 
	/** The position of each corner */
	FTexelCorner Corners[NumTexelCorners];
	/** The tangent basis of the last valid corner to be rasterized */
	FVector4f WorldTangentX;
	FVector4f WorldTangentY;
	FVector4f WorldTangentZ;
	/** Whether each corner lies on the mesh */
	bool bValid[NumTexelCorners];
};

/** Map from texel to the corners of that texel. */
class FTexelToCornersMap
{
public:

	/** Initialization constructor. */
	FTexelToCornersMap(int32 InSizeX,int32 InSizeY):		
		SizeX(InSizeX),
		SizeY(InSizeY)
	{
		// Clear the map to zero.
		Data.AddZeroed(SizeX * SizeY);
	}

	// Accessors.
	FTexelToCorners& operator()(int32 X,int32 Y)
	{
		const uint32 TexelIndex = Y * SizeX + X;
		return Data[TexelIndex];
	}
	const FTexelToCorners& operator()(int32 X,int32 Y) const
	{
		const int32 TexelIndex = Y * SizeX + X;
		return Data[TexelIndex];
	}

	int32 GetSizeX() const { return SizeX; }
	int32 GetSizeY() const { return SizeY; }
	void Empty() { Data.Empty(); }

private:

	/** The mapping data. */
	TArray<FTexelToCorners> Data;

	/** The width of the mapping data. */
	int32 SizeX;

	/** The height of the mapping data. */
	int32 SizeY;
};

/** A particle representing the distribution of a light's radiant power. */
class FPhoton
{
private:

	/** Position that the photon was deposited at in XYZ, and Id in W for debugging. */
	FVector4f PositionAndId;

	/** Direction the photon came from in XYZ, and distance that the photon traveled along its last path before being deposited in W. */
	FVector4f IncidentDirectionAndDistance;

	/** Normal of the surface the photon was deposited on in XYZ, and fraction of the originating light's power that this photon represents in W. */
	FVector4f SurfaceNormalAndPower;

public:

	FPhoton(int32 InId, const FVector4f& InPosition, float InDistance, const FVector4f& InIncidentDirection, const FVector4f& InSurfaceNormal, const FLinearColor& InPower)
	{
		PositionAndId = FVector4f(InPosition, *(float*)&InId);
		IncidentDirectionAndDistance = FVector4f(InIncidentDirection, InDistance);
		checkSlow(FLinearColorUtils::AreFloatsValid(InPower));
		const FColor PowerRGBE = InPower.ToRGBE();
		SurfaceNormalAndPower = FVector4f(InSurfaceNormal, *(float*)&PowerRGBE);
	}

	FORCEINLINE int32 GetId() const
	{
		return *(int32*)&PositionAndId.W;
	}

	FORCEINLINE FVector4f GetPosition() const
	{
		return FVector4f(PositionAndId, 0.0f);
	}

	FORCEINLINE FVector4f GetIncidentDirection() const
	{
		return FVector4f(IncidentDirectionAndDistance, 0.0f);
	}

	FORCEINLINE float GetDistance() const
	{
		return IncidentDirectionAndDistance.W;
	}

	FORCEINLINE FVector4f GetSurfaceNormal() const
	{
		return FVector4f(SurfaceNormalAndPower, 0.0f);
	}

	FORCEINLINE FLinearColor GetPower() const
	{
		const FColor PowerRGBE = *(FColor*)&SurfaceNormalAndPower.W;
		const FLinearColor OutPower = PowerRGBE.FromRGBE();
		checkSlow(FLinearColorUtils::AreFloatsValid(OutPower));
		return OutPower;
	}
};

/** An octree element that contains a photon */
struct FPhotonElement
{
	/** Stores a photon by value so we can discard the original array and avoid a level of indirection */
	const FPhoton Photon;

	/** Initialization constructor. */
	FPhotonElement(const FPhoton& InPhoton) :
		Photon(InPhoton)
	{}
};

typedef TOctree<FPhotonElement,struct FPhotonMapOctreeSemantics> FPhotonOctree;

/** Octree semantic definitions. */
struct FPhotonMapOctreeSemantics
{
	//@todo - evaluate different performance/memory tradeoffs with these
	enum { MaxElementsPerLeaf = 16 };
	enum { MaxNodeDepth = 12 };
	enum { LoosenessDenominator = 16 };

	// Using the default heap allocator instead of an inline allocator to reduce memory usage
	typedef FDefaultAllocator ElementAllocator;

	static FBoxCenterAndExtent GetBoundingBox(const FPhotonElement& PhotonElement)
	{
		return FBoxCenterAndExtent(PhotonElement.Photon.GetPosition(), FVector4f(0,0,0));
	}
};

struct FPhotonSegmentElement
{
	const FPhoton* Photon;

	FVector3f SegmentCenter;
	FVector3f SegmentExtent;

	/** Initialization constructor. */
	FPhotonSegmentElement(const FPhoton* InPhoton, float InStartOffset, float InSegmentLength) :
		Photon(InPhoton)
	{
		const FVector3f PhotonDirection = Photon->GetIncidentDirection() * Photon->GetDistance();
		const FVector3f SegmentStart = Photon->GetPosition() + PhotonDirection * InStartOffset;
		const FVector3f SegmentEnd = SegmentStart + PhotonDirection * InSegmentLength;

		FBox3f SegmentBounds(ForceInit);
		SegmentBounds += SegmentStart;
		SegmentBounds += SegmentEnd;

		SegmentCenter = SegmentBounds.GetCenter();
		// Inflate the segment extent to cover the photon path better
		SegmentExtent = SegmentBounds.GetExtent();
	}

	inline float ComputeSquaredDistanceToPoint(FVector3f InPoint) const
	{
		float Projection = FVector3f::DotProduct(InPoint - Photon->GetPosition(), Photon->GetIncidentDirection());
		Projection = FMath::Clamp(Projection, 0.0f, Photon->GetDistance());
		FVector3f ProjectedPosition = Photon->GetPosition() + Photon->GetIncidentDirection() * Projection;
		return (InPoint - ProjectedPosition).SizeSquared();
	}
};

/** Octree semantic definitions. */
struct FPhotonSegmentMapOctreeSemantics
{
	//@todo - evaluate different performance/memory tradeoffs with these
	enum { MaxElementsPerLeaf = 16 };
	enum { MaxNodeDepth = 12 };
	enum { LoosenessDenominator = 16 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	static FBoxCenterAndExtent GetBoundingBox(const FPhotonSegmentElement& PhotonSegmentElement)
	{
		return FBoxCenterAndExtent(PhotonSegmentElement.SegmentCenter, PhotonSegmentElement.SegmentExtent);
	}
};

typedef TOctree<FPhotonSegmentElement,struct FPhotonSegmentMapOctreeSemantics> FPhotonSegmentOctree;

/** A photon which stores a precalculated irradiance estimate. */
class FIrradiancePhoton : public FIrradiancePhotonData
{
public:

	FIrradiancePhoton(const FVector4f& InPosition, const FVector4f& InSurfaceNormal, bool bInHasContributionFromDirectPhotons)
	{
		PositionAndDirectContribution = FVector4f(InPosition, bInHasContributionFromDirectPhotons);
		SurfaceNormalAndIrradiance = FVector4f(InSurfaceNormal, 0.0f);
	}

	FORCEINLINE bool HasDirectContribution() const
	{
		return PositionAndDirectContribution.W > 0.0f;
	}

	FORCEINLINE void SetHasDirectContribution() 
	{
		PositionAndDirectContribution.W = true;
	}

	FORCEINLINE void SetUsed()
	{
		SurfaceNormalAndIrradiance.W = 1.0f;
	}

	FORCEINLINE bool IsUsed() const
	{
		return SurfaceNormalAndIrradiance.W > 0.0f;
	}

	FORCEINLINE void SetIrradiance(FLinearColor InIrradiance)
	{
		checkSlow(FLinearColorUtils::AreFloatsValid(InIrradiance));
		const FColor IrradianceRGBE = InIrradiance.ToRGBE();
		SurfaceNormalAndIrradiance.W = *(float*)&IrradianceRGBE;
	}

	FORCEINLINE FLinearColor GetIrradiance() const
	{
		const FColor IrradianceRGBE = *(FColor*)&SurfaceNormalAndIrradiance.W;
		const FLinearColor OutIrradiance = IrradianceRGBE.FromRGBE();
		checkSlow(FLinearColorUtils::AreFloatsValid(OutIrradiance));
		return OutIrradiance;
	}

	FORCEINLINE FVector4f GetPosition() const
	{
		return FVector4f(PositionAndDirectContribution, 0.0f);
	}

	FORCEINLINE FVector4f GetSurfaceNormal() const
	{
		return FVector4f(SurfaceNormalAndIrradiance, 0.0f);
	}
};

/** An octree element that contains an irradiance photon */
struct FIrradiancePhotonElement
{
public:
	/** Initialization constructor. */
	FIrradiancePhotonElement(const int32 InPhotonIndex, TArray<FIrradiancePhoton>& InPhotonArray) :
		PhotonIndex(InPhotonIndex),
		PhotonArray(InPhotonArray)
	{}

	FIrradiancePhoton& GetPhoton() { return PhotonArray[PhotonIndex]; }
	const FIrradiancePhoton& GetPhoton() const  { return PhotonArray[PhotonIndex]; }

private:
	int32 PhotonIndex;
	TArray<FIrradiancePhoton>& PhotonArray;
};

typedef TOctree<FIrradiancePhotonElement,struct FIrradiancePhotonMapOctreeSemantics> FIrradiancePhotonOctree;

/** Octree semantic definitions. */
struct FIrradiancePhotonMapOctreeSemantics
{
	//@todo - evaluate different performance/memory tradeoffs with these
	enum { MaxElementsPerLeaf = 32 };
	enum { MaxNodeDepth = 12 };
	enum { LoosenessDenominator = 16 };

	typedef FDefaultAllocator ElementAllocator;

	static FBoxCenterAndExtent GetBoundingBox(const FIrradiancePhotonElement& PhotonElement)
	{
		return FBoxCenterAndExtent(PhotonElement.GetPhoton().GetPosition(), FVector4f(0,0,0));
	}
};

/** A lighting sample in world space storing incident radiance from a whole sphere of directions. */
class FVolumeLightingSample : public FVolumeLightingSampleData
{
public:
	FVolumeLightingSample(const FVector4f& InPositionAndRadius)
	{
		for (int32 CoefficientIndex = 0; CoefficientIndex < LM_NUM_SH_COEFFICIENTS; CoefficientIndex++)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ChannelIndex++)
			{
				HighQualityCoefficients[CoefficientIndex][ChannelIndex] = 0;
				LowQualityCoefficients[CoefficientIndex][ChannelIndex] = 0;
			}
		}

		PositionAndRadius = InPositionAndRadius;
	}
	inline FVector4f GetPosition() const
	{
		return FVector4f(PositionAndRadius, 0.0f);
	}
	inline float GetRadius() const
	{
		return PositionAndRadius.W;
	}

	void SetFromSHVector(const FSHVectorRGB3& SHVector);

	/** Constructs an SH environment from this lighting sample. */
	void ToSHVector(FSHVectorRGB3& SHVector) const;
};

struct FVolumeSampleInterpolationElement
{
	const int32 SampleIndex;
	const TArray<FVolumeLightingSample>& VolumeSamples;

	/** Initialization constructor. */
	FVolumeSampleInterpolationElement(const int32 InSampleIndex, const TArray<FVolumeLightingSample>& InVolumeSamples) :
		SampleIndex(InSampleIndex),
		VolumeSamples(InVolumeSamples)
	{}
};

typedef TOctree<FVolumeSampleInterpolationElement,struct FVolumeLightingInterpolationOctreeSemantics> FVolumeLightingInterpolationOctree;

/** Octree semantic definitions. */
struct FVolumeLightingInterpolationOctreeSemantics
{
	//@todo - evaluate different performance/memory tradeoffs with these
	enum { MaxElementsPerLeaf = 4 };
	enum { MaxNodeDepth = 12 };
	enum { LoosenessDenominator = 16 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	static FBoxCenterAndExtent GetBoundingBox(const FVolumeSampleInterpolationElement& Element)
	{
		const FVolumeLightingSample& Sample = Element.VolumeSamples[Element.SampleIndex];
		return FBoxCenterAndExtent(FVector4f(Sample.PositionAndRadius, 0.0f), FVector4f(Sample.PositionAndRadius.W, Sample.PositionAndRadius.W, Sample.PositionAndRadius.W));
	}
};

class FPrecomputedVisibilityCell
{
public:
	FBox3f Bounds;
	TArray<uint8> VisibilityData;
};

/** Stores depth for a single cell of a shadow map for a stationary light. */
class FStaticShadowDepthMapSample : public FStaticShadowDepthMapSampleData
{
public:
	FStaticShadowDepthMapSample(FFloat16 InDistance)
	{
		Distance = InDistance;
	}
};

/** Stores information about how ShadowMap was generated. */
class FStaticShadowDepthMap : public FStaticShadowDepthMapData
{
public:
	TArray<FStaticShadowDepthMapSample> ShadowMap;
};

/** Number of light bounces that we are keeping track of stats for */
static const int32 NumTrackedBounces = 1;

static const int32 MaxNumRefiningDepths = 6;

/** Stats for a single mapping.  All times are thread seconds if the stat was calculated during a multi threaded mapping process. */
class FStaticLightingMappingStats
{
public:

	/** Part of TotalLightingThreadTime that was spent on texture mappings. */
	float TotalTextureMappingLightingThreadTime;

	/** Part of TotalLightingThreadTime that was spent on volume samples. */
	float TotalVolumeSampleLightingThreadTime;

	/** Part of TotalLightingThreadTime that was spent on volumetric lightmap work. */
	float TotalVolumetricLightmapLightingThreadTime;

	/** Time taken to generate the FTexelToVertexMap */
	float TexelRasterizationTime;

	/** Time taken to create vertex samples. */
	float VertexSampleCreationTime;

	/** Number of texels mapped to geometry in the scene. */
	int32 NumMappedTexels;

	/** Number of vertex light samples calculated. */
	int32 NumVertexSamples;

	/** Time taken to calculate direct lighting */
	float DirectLightingTime;

	/** Thread seconds spent calculating area shadows. */
	float AreaShadowsThreadTime;

	/** Thread seconds spent calculating light attenuation and influence. */
	float AreaLightingThreadTime;

	/** Accumulated signed distance field upsample factors for all mappings that used signed distance field shadows. */
	float AccumulatedSignedDistanceFieldUpsampleFactors;

	/** Number of mappings that used signed distance field shadows. */
	int32 NumSignedDistanceFieldCalculations;

	/** Number of rays traced during the sparse source data generation pass. */
	uint64 NumSignedDistanceFieldAdaptiveSourceRaysFirstPass;

	/** Number of rays traced during the refining source data generation pass. */
	uint64 NumSignedDistanceFieldAdaptiveSourceRaysSecondPass;

	/** Thread seconds spend processing the sparse source data generation pass. */
	float SignedDistanceFieldSourceFirstPassThreadTime;

	/** Thread seconds spend processing the refining source data generation pass. */
	float SignedDistanceFieldSourceSecondPassThreadTime;

	/** Number of transition distance scatters during the distance field search pass. */
	uint64 NumSignedDistanceFieldScatters;

	/** Thread seconds spent searching the source data for the closest transition distance. */
	float SignedDistanceFieldSearchThreadTime;

	/** Number of cell - mesh queries processed. */
	uint64 NumPrecomputedVisibilityQueries;

	/** Number of cell - mesh queries considered visible because the mesh was very close to the cell. */
	uint64 NumQueriesVisibleByDistanceRatio;

	/** Number of cell - mesh queries determined visible from explicitly sampling the mesh's bounds. */
	uint64 NumQueriesVisibleExplicitSampling;

	/** Number of cell - mesh queries determined visible from importance sampling. */
	uint64 NumQueriesVisibleImportanceSampling;

	/** Number of rays traced for precomputed visibility. */
	uint64 NumPrecomputedVisibilityRayTraces;

	/** Number of visibility cells processed on this agent. */
	int32 NumPrecomputedVisibilityCellsProcessed;

	/** Thread seconds processing visibility cells. */
	float PrecomputedVisibilityThreadTime;

	/** Thread seconds generating visibility sample positions. */
	float PrecomputedVisibilitySampleSetupThreadTime;

	/** Thread seconds tracing visibility rays. */
	float PrecomputedVisibilityRayTraceThreadTime;

	/** Thread seconds importance sampling visibility queries. */
	float PrecomputedVisibilityImportanceSampleThreadTime;

	/** Number of visibility queries on groups. */
	uint64 NumPrecomputedVisibilityGroupQueries;

	/** Number of mesh queries that were trivially occluded because their group was already determined to be occluded. */
	uint64 NumPrecomputedVisibilityMeshQueriesSkipped;

	/** Thread seconds calculating static shadow depth maps. */
	float StaticShadowDepthMapThreadTime;

	/** Longest time spent on a single static shadow depth map. */
	float MaxStaticShadowDepthMapThreadTime;

	/** Thread seconds calculating the volume distance field. */
	float VolumeDistanceFieldThreadTime;

	/** Amount of time the mapping processing threads spent either processing indirect lighting cache tasks, or waiting on another thread to finish its cache task. */
	float BlockOnIndirectLightingCacheTasksTime;

	/** Amount of time the mapping processing threads spent either processing indirect lighting interpolate tasks, or waiting on another thread to finish its interpolate task. */
	float BlockOnIndirectLightingInterpolateTasksTime;

	/** Time taken to calculate indirect lighting */
	float IndirectLightingCacheTaskThreadTime;

	float IndirectLightingCacheTaskThreadTimeSeparateTask;

	/** Time taken to gather photons which are used for importance sampling the final gather */
	float ImportancePhotonGatherTime;

	/** Number of importance photons found */
	uint64 TotalFoundImportancePhotons;

	/** Time taken to generate a sample direction based on the importance photons, and then calculate the PDF for the generated sample direction. */
	float CalculateImportanceSampleTime;

	/** Number of elements contributing to the PDF times the number of samples */
	uint64 NumImportancePDFCalculations;

	/** Time taken to calculate the exitant radiance at the end of each final gather ray */
	float CalculateExitantRadianceTime;

	/** Number of final gather rays */
	uint64 NumFirstBounceRaysTraced;

	/** Time taken to trace final gather rays */
	float FirstBounceRayTraceTime;

	/** Number of shadow rays traced for direct lighting. */
	uint64 NumDirectLightingShadowRays;

	/** Number of times the nearest irradiance photon was searched for */
	uint64 NumIrradiancePhotonMapSearches;

	/** Number of unique irradiance photons cached on surfaces.  */
	int32 NumFoundIrradiancePhotons;

	/** Number of nearest irradiance photon samples that were cached on surfaces */
	uint64 NumCachedIrradianceSamples;

	/** Time taken for irradiance cache interpolation for the final shading pass */
	float SecondPassIrradianceCacheInterpolationTime;

	float SecondPassIrradianceCacheInterpolationTimeSeparateTask;

	/** Number of rays traced to determine visibility of irradiance photons while caching them on surfaces */
	uint64 NumIrradiancePhotonSearchRays;

	/** Time spent caching irradiance photons on surfaces */
	float IrradiancePhotonCachingThreadTime;

	float RadiositySetupThreadTime;
	float RadiosityIterationThreadTime;

	/** Time taken traversing the irradiance photon octree */
	float IrradiancePhotonOctreeTraversalTime;

	/** Time taken to trace rays determining the visibility of irradiance photons */
	float IrradiancePhotonSearchRayTime;

	/** The number of final gather samples done at the base resolution. */
	uint64 NumBaseFinalGatherSamples;

	/** The number of final gather samples done at each refinement depth. */
	uint64 NumRefiningFinalGatherSamples[MaxNumRefiningDepths];

	/** The number of final gather samples done due to brightness differences between neighbors. */
	uint64 NumRefiningSamplesDueToBrightness;

	/** The number of final gather samples done due to intersecting importance photons. */
	uint64 NumRefiningSamplesDueToImportancePhotons;

	uint64 NumRefiningSamplesOther;

	/** Amount of time spent computing base final gather samples. */
	float BaseFinalGatherSampleTime;

	/** Amount of time spent computing refining final gather samples. */
	float RefiningFinalGatherSampleTime;

	int32 NumVolumetricLightmapSamples;

	float VolumetricLightmapVoxelizationTime;

	float VolumetricLightmapGatherImportancePhotonsTime;

	float VolumetricLightmapDirectLightingTime;

	float VolumetricLightmapFinalGatherTime;

	FStaticLightingMappingStats() :
		TotalTextureMappingLightingThreadTime(0),
		TotalVolumeSampleLightingThreadTime(0),
		TotalVolumetricLightmapLightingThreadTime(0),
		TexelRasterizationTime(0),
		VertexSampleCreationTime(0),
		NumMappedTexels(0),
		NumVertexSamples(0),
		DirectLightingTime(0),
		AreaShadowsThreadTime(0),
		AreaLightingThreadTime(0),
		AccumulatedSignedDistanceFieldUpsampleFactors(0),
		NumSignedDistanceFieldCalculations(0),
		NumSignedDistanceFieldAdaptiveSourceRaysFirstPass(0),
		NumSignedDistanceFieldAdaptiveSourceRaysSecondPass(0),
		SignedDistanceFieldSourceFirstPassThreadTime(0),
		SignedDistanceFieldSourceSecondPassThreadTime(0),
		NumSignedDistanceFieldScatters(0),
		SignedDistanceFieldSearchThreadTime(0),
		NumPrecomputedVisibilityQueries(0),
		NumQueriesVisibleByDistanceRatio(0),
		NumQueriesVisibleExplicitSampling(0),
		NumQueriesVisibleImportanceSampling(0),
		NumPrecomputedVisibilityRayTraces(0),
		NumPrecomputedVisibilityCellsProcessed(0),
		PrecomputedVisibilityThreadTime(0),
		PrecomputedVisibilitySampleSetupThreadTime(0),
		PrecomputedVisibilityRayTraceThreadTime(0),
		PrecomputedVisibilityImportanceSampleThreadTime(0),
		NumPrecomputedVisibilityGroupQueries(0),
		NumPrecomputedVisibilityMeshQueriesSkipped(0),
		StaticShadowDepthMapThreadTime(0),
		MaxStaticShadowDepthMapThreadTime(0),
		VolumeDistanceFieldThreadTime(0),
		BlockOnIndirectLightingCacheTasksTime(0),
		BlockOnIndirectLightingInterpolateTasksTime(0),
		IndirectLightingCacheTaskThreadTime(0),
		IndirectLightingCacheTaskThreadTimeSeparateTask(0),
		ImportancePhotonGatherTime(0),
		TotalFoundImportancePhotons(0),
		CalculateImportanceSampleTime(0),
		NumImportancePDFCalculations(0),
		CalculateExitantRadianceTime(0),
		NumFirstBounceRaysTraced(0),
		FirstBounceRayTraceTime(0),
		NumDirectLightingShadowRays(0),
		NumIrradiancePhotonMapSearches(0),
		NumFoundIrradiancePhotons(0),
		NumCachedIrradianceSamples(0),
		SecondPassIrradianceCacheInterpolationTime(0),
		SecondPassIrradianceCacheInterpolationTimeSeparateTask(0),
		NumIrradiancePhotonSearchRays(0),
		IrradiancePhotonCachingThreadTime(0),
		RadiositySetupThreadTime(0),
		RadiosityIterationThreadTime(0),
		IrradiancePhotonOctreeTraversalTime(0),
		IrradiancePhotonSearchRayTime(0),
		NumBaseFinalGatherSamples(0),
		NumRefiningSamplesDueToBrightness(0),
		NumRefiningSamplesDueToImportancePhotons(0),
		NumRefiningSamplesOther(0),
		BaseFinalGatherSampleTime(0),
		RefiningFinalGatherSampleTime(0),
		NumVolumetricLightmapSamples(0),
		VolumetricLightmapVoxelizationTime(0),
		VolumetricLightmapGatherImportancePhotonsTime(0),
		VolumetricLightmapDirectLightingTime(0),
		VolumetricLightmapFinalGatherTime(0)
	{
		for (int32 i = 0; i < UE_ARRAY_COUNT(NumRefiningFinalGatherSamples); i++)
		{
			NumRefiningFinalGatherSamples[i] = 0;
		}
	}

	FStaticLightingMappingStats& operator+=(const FStaticLightingMappingStats& B)
	{
		TotalTextureMappingLightingThreadTime += B.TotalTextureMappingLightingThreadTime;
		TotalVolumeSampleLightingThreadTime += B.TotalVolumeSampleLightingThreadTime;
		TotalVolumetricLightmapLightingThreadTime += B.TotalVolumetricLightmapLightingThreadTime;
		TexelRasterizationTime += B.TexelRasterizationTime;
		VertexSampleCreationTime += B.VertexSampleCreationTime;
		NumMappedTexels += B.NumMappedTexels;
		NumVertexSamples += B.NumVertexSamples;
		DirectLightingTime += B.DirectLightingTime;
		AreaShadowsThreadTime += B.AreaShadowsThreadTime;
		AreaLightingThreadTime += B.AreaLightingThreadTime;
		AccumulatedSignedDistanceFieldUpsampleFactors += B.AccumulatedSignedDistanceFieldUpsampleFactors;
		NumSignedDistanceFieldCalculations += B.NumSignedDistanceFieldCalculations;
		NumSignedDistanceFieldAdaptiveSourceRaysFirstPass += B.NumSignedDistanceFieldAdaptiveSourceRaysFirstPass;
		NumSignedDistanceFieldAdaptiveSourceRaysSecondPass += B.NumSignedDistanceFieldAdaptiveSourceRaysSecondPass;
		SignedDistanceFieldSourceFirstPassThreadTime += B.SignedDistanceFieldSourceFirstPassThreadTime;
		SignedDistanceFieldSourceSecondPassThreadTime += B.SignedDistanceFieldSourceSecondPassThreadTime;
		NumSignedDistanceFieldScatters += B.NumSignedDistanceFieldScatters;
		SignedDistanceFieldSearchThreadTime += B.SignedDistanceFieldSearchThreadTime;
		NumPrecomputedVisibilityQueries += B.NumPrecomputedVisibilityQueries;
		NumQueriesVisibleByDistanceRatio += B.NumQueriesVisibleByDistanceRatio;
		NumQueriesVisibleExplicitSampling += B.NumQueriesVisibleExplicitSampling;
		NumQueriesVisibleImportanceSampling += B.NumQueriesVisibleImportanceSampling;
		NumPrecomputedVisibilityRayTraces += B.NumPrecomputedVisibilityRayTraces;
		NumPrecomputedVisibilityCellsProcessed += B.NumPrecomputedVisibilityCellsProcessed;
		PrecomputedVisibilityThreadTime += B.PrecomputedVisibilityThreadTime;
		PrecomputedVisibilitySampleSetupThreadTime += B.PrecomputedVisibilitySampleSetupThreadTime;
		PrecomputedVisibilityRayTraceThreadTime += B.PrecomputedVisibilityRayTraceThreadTime;
		PrecomputedVisibilityImportanceSampleThreadTime += B.PrecomputedVisibilityImportanceSampleThreadTime;
		NumPrecomputedVisibilityGroupQueries += B.NumPrecomputedVisibilityGroupQueries;
		NumPrecomputedVisibilityMeshQueriesSkipped += B.NumPrecomputedVisibilityMeshQueriesSkipped;
		StaticShadowDepthMapThreadTime += B.StaticShadowDepthMapThreadTime;
		MaxStaticShadowDepthMapThreadTime = FMath::Max(MaxStaticShadowDepthMapThreadTime, B.MaxStaticShadowDepthMapThreadTime);
		VolumeDistanceFieldThreadTime += B.VolumeDistanceFieldThreadTime;
		BlockOnIndirectLightingCacheTasksTime += B.BlockOnIndirectLightingCacheTasksTime;
		BlockOnIndirectLightingInterpolateTasksTime += B.BlockOnIndirectLightingInterpolateTasksTime;
		IndirectLightingCacheTaskThreadTime += B.IndirectLightingCacheTaskThreadTime;
		IndirectLightingCacheTaskThreadTimeSeparateTask += B.IndirectLightingCacheTaskThreadTimeSeparateTask;
		ImportancePhotonGatherTime += B.ImportancePhotonGatherTime;
		TotalFoundImportancePhotons += B.TotalFoundImportancePhotons;
		CalculateImportanceSampleTime += B.CalculateImportanceSampleTime;
		NumImportancePDFCalculations += B.NumImportancePDFCalculations;
		CalculateExitantRadianceTime += B.CalculateExitantRadianceTime;
		NumFirstBounceRaysTraced += B.NumFirstBounceRaysTraced;
		FirstBounceRayTraceTime += B.FirstBounceRayTraceTime;
		NumDirectLightingShadowRays += B.NumDirectLightingShadowRays;
		NumIrradiancePhotonMapSearches += B.NumIrradiancePhotonMapSearches;
		NumFoundIrradiancePhotons += B.NumFoundIrradiancePhotons;
		NumCachedIrradianceSamples += B.NumCachedIrradianceSamples;
		SecondPassIrradianceCacheInterpolationTime += B.SecondPassIrradianceCacheInterpolationTime;
		SecondPassIrradianceCacheInterpolationTimeSeparateTask += B.SecondPassIrradianceCacheInterpolationTimeSeparateTask;
		NumIrradiancePhotonSearchRays += B.NumIrradiancePhotonSearchRays;
		IrradiancePhotonCachingThreadTime += B.IrradiancePhotonCachingThreadTime;
		RadiositySetupThreadTime += B.RadiositySetupThreadTime;
		RadiosityIterationThreadTime += B.RadiosityIterationThreadTime;
		IrradiancePhotonOctreeTraversalTime += B.IrradiancePhotonOctreeTraversalTime;
		IrradiancePhotonSearchRayTime += B.IrradiancePhotonSearchRayTime;

		NumBaseFinalGatherSamples += B.NumBaseFinalGatherSamples;
		NumRefiningSamplesDueToBrightness += B.NumRefiningSamplesDueToBrightness;
		NumRefiningSamplesDueToImportancePhotons += B.NumRefiningSamplesDueToImportancePhotons;
		NumRefiningSamplesOther += B.NumRefiningSamplesOther;
		BaseFinalGatherSampleTime += B.BaseFinalGatherSampleTime;
		RefiningFinalGatherSampleTime += B.RefiningFinalGatherSampleTime;

		for (int32 i = 0; i < UE_ARRAY_COUNT(NumRefiningFinalGatherSamples); i++)
		{
			NumRefiningFinalGatherSamples[i] += B.NumRefiningFinalGatherSamples[i];
		}

		NumVolumetricLightmapSamples += B.NumVolumetricLightmapSamples;
		VolumetricLightmapVoxelizationTime += B.VolumetricLightmapVoxelizationTime;
		VolumetricLightmapGatherImportancePhotonsTime += B.VolumetricLightmapGatherImportancePhotonsTime;
		VolumetricLightmapDirectLightingTime += B.VolumetricLightmapDirectLightingTime;
		VolumetricLightmapFinalGatherTime += B.VolumetricLightmapFinalGatherTime;

		return *this;
	}
};

/** Stats collected by FLightingSystem::FindNearbyPhotons*() */
struct FFindNearbyPhotonStats
{
	/** Number of photon map searches using the iterative search process. */
	uint64 NumIterativePhotonMapSearches;

	/** Number of size-increasing photon map search iterations until enough photons were found. */
	uint64 NumSearchIterations;

	/** Thread seconds spent traversing the photon map octree and pushing child nodes onto the traversal stack. */
	float PushingOctreeChildrenThreadTime;

	/** Thread seconds spent processing photons found in the photon map. */
	float ProcessingOctreeElementsThreadTime;

	/** Thread seconds spent finding the furthest found photon in order to replace it with the incoming one. */
	float FindingFurthestPhotonThreadTime;

	/** Number of octree nodes tested during all photon map searches. */
	uint64 NumOctreeNodesTested;

	/** Number of octree nodes that passed testing and had their elements processed during all photon map searches. */
	uint64 NumOctreeNodesVisited;

	/** Number of elements tested during all photon map searches. */
	uint64 NumElementsTested;

	/** Number of elements that passed testing during all photon map searches. */
	uint64 NumElementsAccepted;

	FFindNearbyPhotonStats() :
		NumIterativePhotonMapSearches(0),
		NumSearchIterations(0),
		PushingOctreeChildrenThreadTime(0),
		ProcessingOctreeElementsThreadTime(0),
		FindingFurthestPhotonThreadTime(0),
		NumOctreeNodesTested(0),
		NumOctreeNodesVisited(0),
		NumElementsTested(0),
		NumElementsAccepted(0)
	{}

	FFindNearbyPhotonStats& operator+=(const FFindNearbyPhotonStats& B)
	{
		NumIterativePhotonMapSearches += B.NumIterativePhotonMapSearches;
		NumSearchIterations += B.NumSearchIterations;
		PushingOctreeChildrenThreadTime += B.PushingOctreeChildrenThreadTime;
		ProcessingOctreeElementsThreadTime += B.ProcessingOctreeElementsThreadTime;
		FindingFurthestPhotonThreadTime += B.FindingFurthestPhotonThreadTime;
		NumOctreeNodesTested += B.NumOctreeNodesTested;
		NumOctreeNodesVisited += B.NumOctreeNodesVisited;
		NumElementsTested += B.NumElementsTested;
		NumElementsAccepted += B.NumElementsAccepted;
		return *this;
	}
};

/** Stats collected by FStaticLightingSystem::CalculateIrradiancePhotonsThreadLoop() */
struct FCalculateIrradiancePhotonStats : public FFindNearbyPhotonStats
{
	/** Thread seconds spent calculating irradiance once the relevant photons have been found. */
	float CalculateIrradianceThreadTime;

	FCalculateIrradiancePhotonStats() :
		FFindNearbyPhotonStats(),
		CalculateIrradianceThreadTime(0)
	{}

	FCalculateIrradiancePhotonStats& operator+=(const FCalculateIrradiancePhotonStats& B)
	{
		(FFindNearbyPhotonStats&)(*this) += (const FFindNearbyPhotonStats&)B;
		CalculateIrradianceThreadTime += B.CalculateIrradianceThreadTime;
		return *this;
	}
};

/** Stats for the whole lighting system, which belong to the main thread.  Other threads must use synchronization to access them. */
class FStaticLightingStats : public FStaticLightingMappingStats
{
public:
	
	/** Main thread seconds setting up the scene */
	float SceneSetupTime;

	/** Main thread seconds setting up mesh area lights */
	float MeshAreaLightSetupTime;
	
	/** Thread seconds spent processing mappings for the final lighting pass */
	float TotalLightingThreadTime;

	/** Main thread seconds until the final lighting pass was complete */
	float MainThreadLightingTime;

	/** Number of mappings processed */
	int32 NumMappings;

	/** Number of texels processed */
	int32 NumTexelsProcessed;

	/** Number of lights in the scene. */
	int32 NumLights;

	/** Number of meshes that created mesh area lights. */
	int32 NumMeshAreaLightMeshes;

	/** Number of mesh area lights created from emissive areas in the scene. */
	int32 NumMeshAreaLights;

	/** Number of mesh area light primitives before simplification. */
	uint64 NumMeshAreaLightPrimitives;

	/** Number of simplified mesh area light primitives that are used for lighting. */
	uint64 NumSimplifiedMeshAreaLightPrimitives;

	/** Number of volume lighting samples created off of surfaces. */
	int32 NumDynamicObjectSurfaceSamples;

	/** Number of volume lighting samples created based on the importance volume. */
	int32 NumDynamicObjectVolumeSamples;

	/** Single threaded time setup up sPVS structures for parallel task processing. */
	float PrecomputedVisibilitySetupTime;

	/** Total number of cells which visibility will be computed for. */
	int32 NumPrecomputedVisibilityCellsTotal;

	/** Number of visibility cells generated on camera tracks. */
	int32 NumPrecomputedVisibilityCellsCamaraTracks;

	/** Number of meshes with valid visibility Ids in the scene. */
	int32 NumPrecomputedVisibilityMeshes;

	/** Number of meshes excluded from visibility groups due to their size. */
	int32 NumPrecomputedVisibilityMeshesExcludedFromGroups;

	/** Size of the raw visibility data exported. */
	SIZE_T PrecomputedVisibilityDataBytes;

	/** Main thread time emitting direct photons */
	float EmitDirectPhotonsTime;

	/** Thread time emitting direct photons. */
	float EmitDirectPhotonsThreadTime;

	/** Thread time tracing rays for direct photons. */
	float DirectPhotonsTracingThreadTime;

	/** Thread time generating light samples for direct photons. */
	float DirectPhotonsLightSamplingThreadTime;

	/** Thread time spent applying non-physical attenuation to direct photons. */
	float DirectCustomAttenuationThreadTime;

	/** Thread seconds spent processing gathered direct photons and adding them to spatial data structures. */
	float ProcessDirectPhotonsThreadTime;

	/** Number of direct photons that were deposited on surfaces. */
	int32 NumDirectPhotonsGathered;

	/** Main thread time letting lights cache information about the indirect photon paths. */
	float CachingIndirectPhotonPathsTime;

	/** Main thread time emitting indirect photons */
	float EmitIndirectPhotonsTime;

	/** Thread time emitting indirect photons. */
	float EmitIndirectPhotonsThreadTime;

	/** Thread seconds spent processing gathered indirect photons and adding them to spatial data structures. */
	float ProcessIndirectPhotonsThreadTime;

	/** Thread time sampling lights while emitting indirect photons. */
	float LightSamplingThreadTime;

	/** Thread time spent applying non-physical attenuation to indirect photons. */
	float IndirectCustomAttenuationThreadTime;

	/** Thread time tracing indirect photons from a light. */
	float IntersectLightRayThreadTime;

	/** Thread time tracing photons bouncing off of surfaces in the scene. */
	float PhotonBounceTracingThreadTime;

	/** Number of indirect photons that were deposited on surfaces. */
	int32 NumIndirectPhotonsGathered;

	/** Main thread time marking photons as having direct lighting influence or not. */
	float IrradiancePhotonMarkingTime;

	/** Thread time marking photons. */
	float IrradiancePhotonMarkingThreadTime;

	/** Main thread time calculating irradiance photons */
	float IrradiancePhotonCalculatingTime;

	/** Thread time calculating irradiance photons */
	float IrradiancePhotonCalculatingThreadTime;

	/** Main thread time caching irradiance photons on surfaces */
	float CacheIrradiancePhotonsTime;

	/** Number of irradiance photons */
	int32 NumIrradiancePhotons;

	/** Number of irradiance photons created off of direct photons. */
	int32 NumDirectIrradiancePhotons;

	/** Number of times a photon map was searched, excluding irradiance photon searches */
	volatile int32 NumPhotonGathers;

	/** Number of photons emitted in the first pass */
	uint64 NumFirstPassPhotonsEmitted;

	/** Number of photons requested to be emitted in the first pass */
	uint64 NumFirstPassPhotonsRequested;

	/** Number of photons emitted in the second pass */
	uint64 NumSecondPassPhotonsEmitted;

	/** Number of photons requested to be emitted in the second pass */
	uint64 NumSecondPassPhotonsRequested;

	/** Total number of first hit rays traced */
	uint64 NumFirstHitRaysTraced;

	/** Total number of boolean visibility rays traced */
	uint64 NumBooleanRaysTraced;

	/** Thread seconds spent tracing first hit rays */
	float FirstHitRayTraceThreadTime;

	/** Thread seconds spent tracing shadow rays */
	float BooleanRayTraceThreadTime;

	/** Thread seconds spent placing volume lighting samples and then calculating their lighting. */
	float VolumeSamplePlacementThreadTime;

	/** Irradiance cache stats */
	FIrradianceCacheStats Cache[NumTrackedBounces];

	/** Stats from irradiance photon calculations. */
	FCalculateIrradiancePhotonStats CalculateIrradiancePhotonStats;

	/** Critical section that worker threads must acquire before writing to members of this class */
	FCriticalSection StatsSync;

	FStaticLightingStats() :
		SceneSetupTime(0),
		MeshAreaLightSetupTime(0),
		TotalLightingThreadTime(0),
		MainThreadLightingTime(0),
		NumMappings(0),
		NumTexelsProcessed(0),
		NumLights(0),
		NumMeshAreaLightMeshes(0),
		NumMeshAreaLights(0),
		NumMeshAreaLightPrimitives(0),
		NumSimplifiedMeshAreaLightPrimitives(0),
		NumDynamicObjectSurfaceSamples(0),
		NumDynamicObjectVolumeSamples(0),
		PrecomputedVisibilitySetupTime(0),
		NumPrecomputedVisibilityCellsTotal(0),
		NumPrecomputedVisibilityCellsCamaraTracks(0),
		NumPrecomputedVisibilityMeshes(0),
		NumPrecomputedVisibilityMeshesExcludedFromGroups(0),
		PrecomputedVisibilityDataBytes(0),
		EmitDirectPhotonsTime(0),
		EmitDirectPhotonsThreadTime(0),
		DirectPhotonsTracingThreadTime(0),
		DirectPhotonsLightSamplingThreadTime(0),
		DirectCustomAttenuationThreadTime(0),
		ProcessDirectPhotonsThreadTime(0),
		NumDirectPhotonsGathered(0),
		CachingIndirectPhotonPathsTime(0),
		EmitIndirectPhotonsTime(0),
		EmitIndirectPhotonsThreadTime(0),
		ProcessIndirectPhotonsThreadTime(0),
		LightSamplingThreadTime(0),
		IndirectCustomAttenuationThreadTime(0),
		IntersectLightRayThreadTime(0),
		PhotonBounceTracingThreadTime(0),
		NumIndirectPhotonsGathered(0),
		IrradiancePhotonMarkingTime(0),
		IrradiancePhotonMarkingThreadTime(0),
		IrradiancePhotonCalculatingTime(0),
		IrradiancePhotonCalculatingThreadTime(0),
		CacheIrradiancePhotonsTime(0),
		NumIrradiancePhotons(0),
		NumDirectIrradiancePhotons(0),
		NumPhotonGathers(0),
		NumFirstPassPhotonsEmitted(0),
		NumFirstPassPhotonsRequested(0),
		NumSecondPassPhotonsEmitted(0),
		NumSecondPassPhotonsRequested(0),
		NumFirstHitRaysTraced(0),
		NumBooleanRaysTraced(0),
		FirstHitRayTraceThreadTime(0),
		BooleanRayTraceThreadTime(0),
		VolumeSamplePlacementThreadTime(0)
	{
		for (int32 i = 0; i < NumTrackedBounces; i++)
		{
			Cache[i] = FIrradianceCacheStats();
		}
	}
};

template<typename ElementType>
class FSimpleQuadTreeNode
{
public:
	ElementType Element;

	FSimpleQuadTreeNode* Children[4];

	FSimpleQuadTreeNode()
	{
		Children[0] = NULL;
		Children[1] = NULL;
		Children[2] = NULL;
		Children[3] = NULL;
	}

	void AddChild(int32 Index, FSimpleQuadTreeNode* Child)
	{
		Children[Index] = Child;
	}
};

template<typename ElementType>
class FSimpleQuadTree
{
public:

	const ElementType& GetLeafElement(float U, float V) const
	{
		const FSimpleQuadTreeNode<ElementType>* CurrentNode = &RootNode;

		while (1)
		{
			const int32 ChildX = U > .5f;
			const int32 ChildY = V > .5f;
			const int32 ChildIndex = ChildX * 2 + ChildY;

			if (CurrentNode->Children[ChildIndex])
			{
				U = U * 2 - ChildX;
				V = V * 2 - ChildY;

				CurrentNode = CurrentNode->Children[ChildIndex];
			}
			else
			{
				break;
			}
		}

		return CurrentNode->Element;
	}

	FSimpleQuadTreeNode<ElementType> RootNode;

	~FSimpleQuadTree()
	{
	}

	void ReturnToFreeList(TArray<FSimpleQuadTreeNode<ElementType>*>& OutNodes)
	{
		ReturnToFreeListRecursive(&RootNode, OutNodes);
	}

private:

	void ReturnToFreeListRecursive(FSimpleQuadTreeNode<ElementType>* Node, TArray<FSimpleQuadTreeNode<ElementType>*>& OutNodes) const
	{
		for (int32 ChildIndex = 0; ChildIndex < UE_ARRAY_COUNT(Node->Children); ChildIndex++)
		{
			if (Node->Children[ChildIndex])
			{
				ReturnToFreeListRecursive(Node->Children[ChildIndex], OutNodes);
			}
		}

		if (Node != &RootNode)
		{
			OutNodes.Add(Node);
		}
	}
};

/** Lighting payload used by the adaptive refinement in final gathering. */
class FLightingAndOcclusion
{
public:

	FLinearColor Lighting;
	FVector3f UnoccludedSkyVector;
	FLinearColor StationarySkyLighting;
	float NumSamplesOccluded;

	FLightingAndOcclusion() :
		Lighting(ForceInit),
		UnoccludedSkyVector(FVector3f(0)),
		StationarySkyLighting(FLinearColor::Black),
		NumSamplesOccluded(0)
	{}

	FLightingAndOcclusion(const FLinearColor& InLighting, FVector3f InUnoccludedSkyVector, const FLinearColor& InStationarySkyLighting, float InNumSamplesOccluded) : 
		Lighting(InLighting),
		UnoccludedSkyVector(InUnoccludedSkyVector),
		StationarySkyLighting(InStationarySkyLighting),
		NumSamplesOccluded(InNumSamplesOccluded)
	{}

	friend inline FLightingAndOcclusion operator+ (const FLightingAndOcclusion& A, const FLightingAndOcclusion& B)
	{
		return FLightingAndOcclusion(A.Lighting + B.Lighting, A.UnoccludedSkyVector + B.UnoccludedSkyVector, A.StationarySkyLighting + B.StationarySkyLighting, A.NumSamplesOccluded + B.NumSamplesOccluded);
	}

	friend inline FLightingAndOcclusion operator/ (const FLightingAndOcclusion& A, float Divisor)
	{
		return FLightingAndOcclusion(A.Lighting / Divisor, A.UnoccludedSkyVector / Divisor, A.StationarySkyLighting / Divisor, A.NumSamplesOccluded / Divisor);
	}
};

/** Data stored for a sample that may need to be refined. */
class FRefinementElement
{
public:
	FLightingAndOcclusion Lighting;
	FVector2f Uniforms;
	int32 HitPointIndex;

	FRefinementElement() :
		Uniforms(FVector2f(0, 0)),
		HitPointIndex(-1)
	{}

	FRefinementElement(FLightingAndOcclusion InLighting, FVector2f InUniforms, int32 InHitPointIndex) :
		Lighting(InLighting),
		Uniforms(InUniforms),
		HitPointIndex(InHitPointIndex)
	{}
};

/** Local state for a mapping, accessed only by the owning thread. */
class FStaticLightingMappingContext
{
public:
	FStaticLightingMappingStats Stats;

	/** Lighting caches for the mapping */
	TLightingCache<FFinalGatherSample> FirstBounceCache;

	FCoherentRayCache RayCache;

	TArray<FDebugLightingCacheRecord> DebugCacheRecords;

	TArray<FSimpleQuadTreeNode<FRefinementElement>* > RefinementTreeFreePool;

	class FStaticLightingSystem& System;

	/** This mapping's debug output, if any. */
	FDebugLightingOutput* DebugOutput;

public:
	FStaticLightingMappingContext(const FStaticLightingMesh* InSubjectMesh, class FStaticLightingSystem& InSystem, FDebugLightingOutput* InDebugOutput = nullptr);

	~FStaticLightingMappingContext();
};

/** Information about the power distribution of lights in the scene. */
class FSceneLightPowerDistribution
{
public:
	/** Stores an unnormalized step 1D probability distribution function of emitting a photon from a given light */
	TArray<float> LightPDFs;

	/** Stores the cumulative distribution function of LightPDFs */
	TArray<float> LightCDFs;

	/** Stores the integral of LightPDFs */
	float UnnormalizedIntegral;
};

/** The static lighting data for a texture mapping. */
struct FTextureMappingStaticLightingData
{
	FStaticLightingTextureMapping* Mapping;
	FLightMapData2D* LightMapData;
	TMap<const FLight*,FShadowMapData2D*> ShadowMaps;
	TMap<const FLight*,FSignedDistanceFieldShadowMapData2D*> SignedDistanceFieldShadowMaps;
	FDebugLightingOutput DebugOutput;

	/** Stores the time this mapping took to process */
	double ExecutionTime;
};

/** Visibility output data from a single visibility task. */
struct FPrecomputedVisibilityData
{
	FGuid Guid;
	TArray<FPrecomputedVisibilityCell> PrecomputedVisibilityCells;
	TArray<FDebugStaticLightingRay> DebugVisibilityRays;
};

struct FIrradianceBrickData
{
	FGuid IntersectingLevelGuid;

	/** Position in the global indirection texture.  Used for mapping brick positions back to world space. */
	FIntVector IndirectionTexturePosition;

	/** Depth in the refinement tree, where 0 is the root. */
	int32 TreeDepth;

	float AverageClosestGeometryDistance;

	TArray<FFloat3Packed> AmbientVector;
	TArray<FColor> SHCoefficients[6];
	TArray<FColor> SkyBentNormal;
	TArray<uint8> DirectionalLightShadowing;

	TArray<FIrradianceVoxelImportProcessingData> VoxelImportProcessingData;

	void SetFromVolumeLightingSample(int32 Index, const FVolumeLightingSample& Sample, bool bInsideGeometry, float MinDistanceToSurface, bool bBorderVoxel);
};

/** Output data from a single volumetric lightmap task. */
struct FVolumetricLightmapTaskData
{
	FGuid Guid;
	TArray<FIrradianceBrickData> BrickData;
	FDebugLightingOutput DebugOutput;
};


/** A thread which processes static lighting mappings. */
class FStaticLightingThreadRunnable : public FRunnable
{
public:

	FRunnableThread* Thread;

	/** Seconds that the thread spent in Run() */
	float ExecutionTime;

	float IdleTime;

	/** Seconds since GStartupTime that the thread exited Run() */
	double EndTime;

	int32	ThreadIndex;

	FThreadStatistics ThreadStatistics;

	/** Initialization constructor. */
	FStaticLightingThreadRunnable(FStaticLightingSystem* InSystem, int32 InThreadIndex) :
		Thread(NULL),
		IdleTime(0),
		ThreadIndex(InThreadIndex),
		System(InSystem),
		bTerminatedByError(false)
	{}

	/** Checks the thread's health, and passes on any errors that have occurred.  Called by the main thread. */
	bool CheckHealth(bool bReportError = true) const;

	void FixThreadGroupAffinity() const;

protected:
	FStaticLightingSystem* System;

	/** If the thread has been terminated by an unhandled exception, this contains the error message. */
	FString ErrorMessage;

	/** true if the thread has been terminated by an unhandled exception. */
	bool bTerminatedByError;
};

/** Input required to emit direct photons. */
class FDirectPhotonEmittingInput
{
public:
	const FBoxSphereBounds3f& ImportanceBounds;
	const FSceneLightPowerDistribution& LightDistribution;
	
	FDirectPhotonEmittingInput(
		const FBoxSphereBounds3f& InImportanceBounds,
		const FSceneLightPowerDistribution& InLightDistribution)
		:
		ImportanceBounds(InImportanceBounds),
		LightDistribution(InLightDistribution)
	{}
};

/** A work range for emitting direct photons, which is the smallest unit that can be parallelized. */
class FDirectPhotonEmittingWorkRange
{
public:
	const int32 RangeIndex;
	const int32 NumDirectPhotonsToEmit;
	const int32 TargetNumIndirectPhotonPaths;

	FDirectPhotonEmittingWorkRange(
		int32 InRangeIndex,
		int32 InNumDirectPhotonsToEmit,
		int32 InTargetNumIndirectPhotonPaths)
		:
		RangeIndex(InRangeIndex),
		NumDirectPhotonsToEmit(InNumDirectPhotonsToEmit),
		TargetNumIndirectPhotonPaths(InTargetNumIndirectPhotonPaths)
	{}
};

/** Direct photon emitting output for a single FDirectPhotonEmittingWorkRange. */
class FDirectPhotonEmittingOutput
{
public:
	/** 
	 * A worker thread will increment this counter once the output is complete, 
	 * so that the main thread can process it while the worker thread moves on. 
	 */
	volatile int32 OutputComplete;
	int32 NumPhotonsEmittedDirect;
	TArray<FPhoton> DirectPhotons;
	TArray<FIrradiancePhoton>* IrradiancePhotons;
	TArray<TArray<FIndirectPathRay> > IndirectPathRays;
	int32 NumPhotonsEmitted;
	float DirectPhotonsTracingThreadTime;
	float DirectPhotonsLightSamplingThreadTime;
	float DirectCustomAttenuationThreadTime;

	FDirectPhotonEmittingOutput(TArray<FIrradiancePhoton>* InIrradiancePhotons) :
		OutputComplete(0),
		NumPhotonsEmittedDirect(0),
		IrradiancePhotons(InIrradiancePhotons),
		NumPhotonsEmitted(0),
		DirectPhotonsTracingThreadTime(0),
		DirectPhotonsLightSamplingThreadTime(0),
		DirectCustomAttenuationThreadTime(0)
	{}
};

/** Thread used to parallelize indirect photon emitting. */
class FDirectPhotonEmittingThreadRunnable : public FStaticLightingThreadRunnable
{
public:

	/** Initialization constructor. */
	FDirectPhotonEmittingThreadRunnable(
		FStaticLightingSystem* InSystem,
		int32 InThreadIndex,
		const FDirectPhotonEmittingInput& InInput)
		:
		FStaticLightingThreadRunnable(InSystem, InThreadIndex),
		Input(InInput)
	{}

	// FRunnable interface.
	virtual bool Init(void) { return true; }
	virtual void Exit(void) {}
	virtual void Stop(void) {}
	virtual uint32 Run(void);

protected:
	const FDirectPhotonEmittingInput& Input;
};

/** Input required to emit indirect photons. */
class FIndirectPhotonEmittingInput
{
public:
	const FBoxSphereBounds3f& ImportanceBounds;
	const FSceneLightPowerDistribution& LightDistribution;
	const TArray<TArray<FIndirectPathRay> >& IndirectPathRays;
	
	FIndirectPhotonEmittingInput(
		const FBoxSphereBounds3f& InImportanceBounds,
		const FSceneLightPowerDistribution& InLightDistribution,
		const TArray<TArray<FIndirectPathRay> >& InIndirectPathRays)
		:
		ImportanceBounds(InImportanceBounds),
		LightDistribution(InLightDistribution),
		IndirectPathRays(InIndirectPathRays)
	{}
};

/** A work range for emitting indirect photons, which is the smallest unit that can be parallelized. */
class FIndirectPhotonEmittingWorkRange
{
public:
	const int32 RangeIndex;
	const int32 NumIndirectPhotonsToEmit;

	FIndirectPhotonEmittingWorkRange(
		int32 InRangeIndex,
		int32 InNumIndirectPhotonsToEmit)
		:
		RangeIndex(InRangeIndex),
		NumIndirectPhotonsToEmit(InNumIndirectPhotonsToEmit)
	{}
};

/** Indirect photon emitting output for a single FIndirectPhotonEmittingWorkRange. */
class FIndirectPhotonEmittingOutput
{
public:
	/** 
	 * A worker thread will increment this counter once the output is complete, 
	 * so that the main thread can process it while the worker thread moves on. 
	 */
	volatile int32 OutputComplete;
	int32 NumPhotonsEmittedFirstBounce;
	TArray<FPhoton> FirstBouncePhotons;
	TArray<FPhoton> FirstBounceEscapedPhotons;
	int32 NumPhotonsEmittedSecondBounce;
	TArray<FPhoton> SecondBouncePhotons;
	TArray<FIrradiancePhoton>* IrradiancePhotons;
	int32 NumPhotonsEmitted;
	float LightSamplingThreadTime;
	float IndirectCustomAttenuationThreadTime;
	float IntersectLightRayThreadTime;
	float PhotonBounceTracingThreadTime;

	FIndirectPhotonEmittingOutput(TArray<FIrradiancePhoton>* InIrradiancePhotons) :
		OutputComplete(0),
		NumPhotonsEmittedFirstBounce(0),
		NumPhotonsEmittedSecondBounce(0),
		IrradiancePhotons(InIrradiancePhotons),
		NumPhotonsEmitted(0),
		LightSamplingThreadTime(0),
		IndirectCustomAttenuationThreadTime(0),
		IntersectLightRayThreadTime(0),
		PhotonBounceTracingThreadTime(0)
	{}
};

/** Thread used to parallelize indirect photon emitting. */
class FIndirectPhotonEmittingThreadRunnable : public FStaticLightingThreadRunnable
{
public:

	/** Initialization constructor. */
	FIndirectPhotonEmittingThreadRunnable(
		FStaticLightingSystem* InSystem, 
		int32 InThreadIndex,
		const FIndirectPhotonEmittingInput& InInput)
		:
		FStaticLightingThreadRunnable(InSystem, InThreadIndex),
		Input(InInput)
	{}

	// FRunnable interface.
	virtual bool Init(void) { return true; }
	virtual void Exit(void) {}
	virtual void Stop(void) {}
	virtual uint32 Run(void);

protected:
	const FIndirectPhotonEmittingInput& Input;
};

/** Smallest unit of irradiance photon marking work that can be done in parallel. */
class FIrradianceMarkingWorkRange
{
public:
	const int32 RangeIndex;
	/** Index into IrradiancePhotons that should be processed for this work range. */
	const int32 IrradiancePhotonArrayIndex;

	FIrradianceMarkingWorkRange(
		int32 InRangeIndex,
		int32 InIrradiancePhotonArrayIndex)
		:
		RangeIndex(InRangeIndex),
		IrradiancePhotonArrayIndex(InIrradiancePhotonArrayIndex)
	{}
};

class FIrradiancePhotonMarkingThreadRunnable : public FStaticLightingThreadRunnable
{
public:

	/** Initialization constructor. */
	FIrradiancePhotonMarkingThreadRunnable(FStaticLightingSystem* InSystem, int32 InThreadIndex, TArray<TArray<FIrradiancePhoton>>& InIrradiancePhotons) :
		FStaticLightingThreadRunnable(InSystem, InThreadIndex),
		IrradiancePhotons(InIrradiancePhotons)
	{}

	// FRunnable interface.
	virtual bool Init(void) { return true; }
	virtual void Exit(void) {}
	virtual void Stop(void) {}
	virtual uint32 Run(void);

private:
	/** Irradiance photons to operate on */
	TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons;
};

/** Smallest unit of irradiance photon calculating work that can be done in parallel. */
class FIrradianceCalculatingWorkRange
{
public:
	const int32 RangeIndex;
	/** Index into IrradiancePhotons that should be processed for this work range. */
	const int32 IrradiancePhotonArrayIndex;

	FIrradianceCalculatingWorkRange(
		int32 InRangeIndex,
		int32 InIrradiancePhotonArrayIndex)
		:
		RangeIndex(InRangeIndex),
		IrradiancePhotonArrayIndex(InIrradiancePhotonArrayIndex)
	{}
};

class FIrradiancePhotonCalculatingThreadRunnable : public FStaticLightingThreadRunnable
{
public:

	/** Stats for this thread's operations. */
	FCalculateIrradiancePhotonStats Stats;

	/** Initialization constructor. */
	FIrradiancePhotonCalculatingThreadRunnable(FStaticLightingSystem* InSystem, int32 InThreadIndex, TArray<TArray<FIrradiancePhoton>>& InIrradiancePhotons) :
		FStaticLightingThreadRunnable(InSystem, InThreadIndex),
		IrradiancePhotons(InIrradiancePhotons)
	{}

	// FRunnable interface.
	virtual bool Init(void) { return true; }
	virtual void Exit(void) {}
	virtual void Stop(void) {}
	virtual uint32 Run(void);

private:
	/** Irradiance photons to operate on */
	TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons;
};

/** Indicates which type of task a FMappingProcessingThreadRunnable should execute */
enum EStaticLightingTaskType
{
	StaticLightingTask_ProcessMappings,
	StaticLightingTask_CacheIrradiancePhotons,
	StaticLightingTask_RadiositySetup,
	StaticLightingTask_RadiosityIterations,
	StaticLightingTask_FinalizeSurfaceCache
};

/** A thread which processes static lighting mappings. */
class FMappingProcessingThreadRunnable : public FStaticLightingThreadRunnable
{
	/** > 0 this thread has finished working */
	FThreadSafeCounter FinishedCounter;

public:

	EStaticLightingTaskType TaskType;

	/** Initialization constructor. */
	FMappingProcessingThreadRunnable(FStaticLightingSystem* InSystem, int32 ThreadIndex, EStaticLightingTaskType InTaskType) :
		FStaticLightingThreadRunnable(InSystem, ThreadIndex),
		FinishedCounter(0),
		TaskType(InTaskType)
	{}

	// FRunnable interface.
	virtual bool Init(void) { return true; }
	virtual void Exit(void) {}
	virtual void Stop(void) {}
	virtual uint32 Run(void);

	bool IsComplete()
	{
		return FinishedCounter.GetValue() > 0;
	}
};

/** Encapsulates a list of mappings which static lighting has been computed for, but not yet applied. */
template<typename StaticLightingDataType>
class TCompleteStaticLightingList
{
public:

	/** Initialization constructor. */
	TCompleteStaticLightingList():
		FirstElement(NULL)
	{}

	/** Adds an element to the list. */
	void AddElement(TList<StaticLightingDataType>* Element)
	{
		// Link the element at the beginning of the list.
		TList<StaticLightingDataType>* LocalFirstElement;
		do 
		{
			LocalFirstElement = FirstElement;
			Element->Next = LocalFirstElement;
		}
		while(FPlatformAtomics::InterlockedCompareExchangePointer((void**)&FirstElement,Element,LocalFirstElement) != LocalFirstElement);
	}

	/**
	 * Applies the static lighting to the mappings in the list, and clears the list.
	 * Also reports back to Unreal after each mapping has been exported.
	 * @param LightingSystem - Reference to the static lighting system
	 */
	void ApplyAndClear(FStaticLightingSystem& LightingSystem);

protected:

	TList<StaticLightingDataType>* FirstElement;
};

template<typename DataType>
class TCompleteTaskList : public TCompleteStaticLightingList<DataType>
{
public:
	void ApplyAndClear(FStaticLightingSystem& LightingSystem);
};

/** Base class for a task that operates on a rectangle of a texture mapping. */
class FBaseTextureTaskDescription
{
public:
	int32 StartX;
	int32 StartY;
	int32 SizeX;
	int32 SizeY;
	bool bDebugThisMapping;
	FStaticLightingTextureMapping* TextureMapping;
	FStaticLightingMappingContext MappingContext;
	FGatheredLightMapData2D* LightMapData;
	const class FTexelToVertexMap* TexelToVertexMap;
	bool bProcessedOnMainThread;

	FBaseTextureTaskDescription(const FStaticLightingMesh* InSubjectMesh, class FStaticLightingSystem& InSystem) :
		StartX(0),
		StartY(0),
		SizeX(0),
		SizeY(0),
		bDebugThisMapping(false),
		TextureMapping(NULL),
		MappingContext(InSubjectMesh, InSystem),
		LightMapData(NULL),
		TexelToVertexMap(NULL),
		bProcessedOnMainThread(false)
	{}
};

/** Class for a task that populates the irradiance cache for a texture mapping. */
class FCacheIndirectTaskDescription : public FBaseTextureTaskDescription
{
public:

	FCacheIndirectTaskDescription(const FStaticLightingMesh* InSubjectMesh, class FStaticLightingSystem& InSystem) :
		FBaseTextureTaskDescription(InSubjectMesh, InSystem)
	{}
};

/** Class for a task that interpolates from the irradiance cache for a texture mapping. */
class FInterpolateIndirectTaskDescription : public FBaseTextureTaskDescription
{
public:
	TLightingCache<FFinalGatherSample>* FirstBounceCache;

	FInterpolateIndirectTaskDescription(const FStaticLightingMesh* InSubjectMesh, class FStaticLightingSystem& InSystem) :
		FBaseTextureTaskDescription(InSubjectMesh, InSystem),
		FirstBounceCache(NULL)
	{}
};

/**  */
class FVolumeSamplesTaskDescription
{
public:
	FGuid LevelId;
	int32 StartIndex;
	int32 NumSamples;
	
	FVolumeSamplesTaskDescription(FGuid InLevelId, int32 InStartIndex, int32 InNumSamples) :
		LevelId(InLevelId),
		StartIndex(InStartIndex),
		NumSamples(InNumSamples)
	{}
};

class FVisibilityMeshGroup
{
public:
	FBox3f GroupBounds;

	/** Array of all the meshes contained in the group.  These entries index into VisibilityMeshes. */
	TArray<int32> VisibilityIds;
};

class FVisibilityMesh
{
public:
	bool bInGroup;
	TArray<FStaticLightingMesh*> Meshes;
};

enum EFinalGatherRayBiasMode
{
	RBM_ConstantNormalOffset,
	RBM_ScaledNormalOffset
};

/** The state of the static lighting system. */
class FStaticLightingSystem
{
public:

	/** Debug data generated by startup passes (photon mapping) to transfer back to Unreal. */
	mutable FDebugLightingOutput StartupDebugOutput;

	FVolumeLightingDebugOutput VolumeLightingDebugOutput;

	/** Threads must acquire this critical section before reading or writing to StartupDebugOutput or VolumeLightingDebugOutput, if contention is possible. */
	mutable FCriticalSection DebugOutputSync;

	/**
	 * Initializes this static lighting system, and builds static lighting based on the provided options.
	 * @param InOptions		- The static lighting build options.
	 * @param InScene		- The scene containing all the lights and meshes
	 * @param InExporter	- The exporter used to send completed data back to Unreal
	 * @param InNumThreads	- Number of concurrent threads to use for lighting building
	 */
	FStaticLightingSystem( const FLightingBuildOptions& InOptions, class FScene& InScene, class FLightmassSolverExporter& InExporter, int32 InNumThreads );

	~FStaticLightingSystem();

	/**
	 * Returns the Lightmass exporter (back to Unreal)
	 * @return	Lightmass exporter
	 */
	class FLightmassSolverExporter& GetExporter()
	{
		return Exporter;
	}

	FGuid GetDebugGuid() const { return Scene.DebugInput.MappingGuid; }

	/**
	 * Whether the Lighting System is in debug mode or not. When in debug mode, Swarm is not completely hooked up
	 * and the system will process all mappings in the scene file on its own.
	 * @return	true if in debug mode
	 */
	bool IsDebugMode() const
	{
		return GDebugMode;
	}

	bool HasSkyShadowing() const
	{
		for (int32 SkylightIndex = 0; SkylightIndex < SkyLights.Num(); SkylightIndex++)
		{
			// Indicating sky shadowing is needed even if the sky lights do not have shadow casting enabled,
			// So that shadow casting can be toggled without rebuilding lighting
			// This does mean that skylights with shadow casting disabled will generate unused sky occlusion textures
			if (!(SkyLights[SkylightIndex]->LightFlags & GI_LIGHT_HASSTATICLIGHTING))
			{
				return true;
			}
		}

		return false;
	}

	/** Rasterizes Mesh into TexelToCornersMap */
	void CalculateTexelCorners(const FStaticLightingMesh* Mesh, class FTexelToCornersMap& TexelToCornersMap, int32 UVIndex, bool bDebugThisMapping) const;

	/** Rasterizes Mesh into TexelToCornersMap, with extra parameters like which material index to rasterize and UV scale and bias. */
	void CalculateTexelCorners(
		const TArray<int32>& TriangleIndices, 
		const TArray<FStaticLightingVertex>& Vertices, 
		FTexelToCornersMap& TexelToCornersMap, 
		const TArray<int32>& ElementIndices,
		int32 MaterialIndex,
		int32 UVIndex, 
		bool bDebugThisMapping, 
		FVector2f UVBias, 
		FVector2f UVScale) const;

	FStaticLightingAggregateMeshType& GetAggregateMesh()
	{
		return *AggregateMesh;
	}

private:

	/** Exports tasks that are not mappings, if they are ready. */
	void ExportNonMappingTasks();

	/** Internal accessors */
	int32 GetNumShadowRays(int32 BounceNumber, bool bPenumbra=false) const;
	int32 GetNumUniformHemisphereSamples(int32 BounceNumber) const;
	int32 GetNumPhotonImportanceHemisphereSamples() const;

	FBoxSphereBounds3f GetImportanceBounds(bool bClampToScene = true) const;

	/** Returns true if the specified position is inside any of the importance volumes. */
	bool IsPointInImportanceVolume(const FVector4f& Position, float Tolerance = 0.0f) const;

	/** Changes the scene's settings if necessary so that only valid combinations are used */
	void ValidateSettings(FScene& InScene);

	/** Logs solver stats */
	void DumpStats(float TotalStaticLightingTime) const;

	/** Logs a solver message */
	void LogSolverMessage(const FString& Message) const;

	/** Logs a progress update message when appropriate */
	void UpdateInternalStatus(int32 OldNumTexelsCompleted) const;

	/** Caches samples for any sampling distributions that are known ahead of time, which greatly reduces noise in those estimates. */
	void CacheSamples();

	/** Sets up photon mapping settings. */
	void InitializePhotonSettings();

	/** Emits photons, builds data structures to accelerate photon map lookups, and does any other photon preprocessing required. */
	void EmitPhotons();

	/** Gathers direct photons and generates indirect photon paths. */
	void EmitDirectPhotons(
		const FBoxSphereBounds3f& ImportanceBounds, 
		TArray<TArray<FIndirectPathRay> >& IndirectPathRays,
		TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons);

	/** Entrypoint for all threads emitting direct photons. */
	void EmitDirectPhotonsThreadLoop(const FDirectPhotonEmittingInput& Input, int32 ThreadIndex);

	/** Emits direct photons for a given work range. */
	void EmitDirectPhotonsWorkRange(
		const FDirectPhotonEmittingInput& Input, 
		FDirectPhotonEmittingWorkRange WorkRange, 
		FDirectPhotonEmittingOutput& Output);

	void BuildPhotonSegmentMap(const FPhotonOctree& SourcePhotonMap, FPhotonSegmentOctree& OutPhotonSegementMap, float AddToSegmentMapChance);

	/** Gathers indirect photons based on the indirect photon paths. */
	void EmitIndirectPhotons(
		const FBoxSphereBounds3f& ImportanceBounds,
		const TArray<TArray<FIndirectPathRay> >& IndirectPathRays, 
		TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons);

	/** Entrypoint for all threads emitting indirect photons. */
	void EmitIndirectPhotonsThreadLoop(const FIndirectPhotonEmittingInput& Input, int32 ThreadIndex);

	/** Emits indirect photons for a given work range. */
	void EmitIndirectPhotonsWorkRange(
		const FIndirectPhotonEmittingInput& Input, 
		FIndirectPhotonEmittingWorkRange WorkRange, 
		FIndirectPhotonEmittingOutput& Output);

	/** Iterates through all irradiance photons, searches for nearby direct photons, and marks the irradiance photon has having direct photon influence if necessary. */
	void MarkIrradiancePhotons(const FBoxSphereBounds3f& ImportanceBounds, TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons);

	/** Entry point for all threads marking irradiance photons. */
	void MarkIrradiancePhotonsThreadLoop(
		int32 ThreadIndex, 
		TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons);

	/** Marks irradiance photons specified by a single work range. */
	void MarkIrradiancePhotonsWorkRange(
		TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons, 
		FIrradianceMarkingWorkRange WorkRange);

	/** Calculates irradiance for photons randomly chosen to precalculate irradiance. */
	void CalculateIrradiancePhotons(const FBoxSphereBounds3f& ImportanceBounds, TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons);

	/** Main loop that all threads access to calculate irradiance photons. */
	void CalculateIrradiancePhotonsThreadLoop(
		int32 ThreadIndex, 
		TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons, 
		FCalculateIrradiancePhotonStats& Stats);

	/** Calculates irradiance for the photons specified by a single work range. */
	void CalculateIrradiancePhotonsWorkRange(
		TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons, 
		FIrradianceCalculatingWorkRange WorkRange,
		FCalculateIrradiancePhotonStats& Stats);

    /** Cache irradiance photons on surfaces. */
	void CacheIrradiancePhotons();

	/** Main loop that all threads access to cache irradiance photons. */
	void CacheIrradiancePhotonsThreadLoop(int32 ThreadIndex, bool bIsMainThread);

	/** Caches irradiance photons on a single texture mapping. */
	void CacheIrradiancePhotonsTextureMapping(FStaticLightingTextureMapping* TextureMapping);

	void SetupRadiosity();
	void RadiositySetupThreadLoop(int32 ThreadIndex, bool bIsMainThread);
	void RadiositySetupTextureMapping(FStaticLightingTextureMapping* TextureMapping);
	void RunRadiosityIterations();

	void RadiosityIterationThreadLoop(int32 ThreadIndex, bool bIsMainThread);

	/** Cache irradiance photons on surfaces. */
	void FinalizeSurfaceCache();

	/** Main loop that all threads access to cache irradiance photons. */
	void FinalizeSurfaceCacheThreadLoop(int32 ThreadIndex, bool bIsMainThread);

	/** Caches irradiance photons on a single texture mapping. */
	void FinalizeSurfaceCacheTextureMapping(FStaticLightingTextureMapping* TextureMapping);

	void RasterizeToSurfaceCacheTextureMapping(FStaticLightingTextureMapping* TextureMapping, bool bDebugThisMapping, FTexelToVertexMap& TexelToVertexMap);

	void AdjustRepresentativeSurfelForTexelsTextureMapping(FStaticLightingTextureMapping* TextureMapping, FTexelToVertexMap& TexelToVertexMap, FTexelToCornersMap& TexelToCornersMap, FGatheredLightMapData2D* LightMapData, FStaticLightingMappingContext& MappingContext,	bool bDebugThisMapping) const;

	void RadiosityIterationTextureMapping(FStaticLightingTextureMapping* TextureMapping, int32 PassIndex);

	void RadiosityIterationCachedHitpointsTextureMapping(const FTexelToVertexMap& TexelToVertexMap, FStaticLightingTextureMapping* TextureMapping, int32 PassIndex);

	/** Returns true if a photon was found within MaxPhotonSearchDistance. */
	bool FindAnyNearbyPhoton(
		const FPhotonOctree& PhotonMap, 
		const FVector4f& SearchPosition, 
		float MaxPhotonSearchDistance,
		bool bDebugThisLookup) const;

	/** 
	 * Searches the given photon map for the nearest NumPhotonsToFind photons to SearchPosition using an iterative process, 
	 * Unless the start and max search distances are the same, in which case all photons in that distance will be returned.
	 * The iterative search starts at StartPhotonSearchDistance and doubles the search distance until enough photons are found or the distance is greater than MaxPhotonSearchDistance.
	 * @return - the furthest found photon's distance squared from SearchPosition, unless the start and max search distances are the same,
	 *		in which case FMath::Square(MaxPhotonSearchDistance) will be returned.
	 */
	float FindNearbyPhotonsIterative(
		const FPhotonOctree& PhotonMap, 
		const FVector4f& SearchPosition, 
		const FVector4f& SearchNormal, 
		int32 NumPhotonsToFind,
		float StartPhotonSearchDistance, 
		float MaxPhotonSearchDistance,
		bool bDebugSearchResults,
		bool bDebugSearchProcess,
		TArray<FPhoton>& FoundPhotons,
		FFindNearbyPhotonStats& SearchStats) const;

	float FindNearbyPhotonsInVolumeIterative(
		const FPhotonSegmentOctree& PhotonSegmentMap,
		const FVector4f& SearchPosition,
		int32 NumPhotonsToFind,
		float StartPhotonSearchDistance,
		float MaxPhotonSearchDistance,
		TArray<FPhotonSegmentElement>& FoundPhotonSegments,
		bool bDebugThisLookup) const;

	/** 
	 * Searches the given photon map for the nearest NumPhotonsToFind photons to SearchPosition by sorting octree nodes nearest to furthest.
	 * @return - the furthest found photon's distance squared from SearchPosition.
	 */
	float FindNearbyPhotonsSorted(
		const FPhotonOctree& PhotonMap, 
		const FVector4f& SearchPosition, 
		const FVector4f& SearchNormal, 
		int32 NumPhotonsToFind, 
		float MaxPhotonSearchDistance,
		bool bDebugSearchResults,
		bool bDebugSearchProcess,
		TArray<FPhoton>& FoundPhotons,
		FFindNearbyPhotonStats& SearchStats) const;

	/** Finds the nearest irradiance photon, if one exists. */
	FIrradiancePhoton* FindNearestIrradiancePhoton(
		const FMinimalStaticLightingVertex& Vertex, 
		FStaticLightingMappingContext& MappingContext, 
		TArray<FIrradiancePhoton*>& TempIrradiancePhotons,
		bool bVisibleOnly, 
		bool bDebugThisLookup) const;

	/** Calculates the irradiance for an irradiance photon */
	FLinearColor CalculatePhotonIrradiance(
		const FPhotonOctree& PhotonMap,
		int32 NumPhotonsEmitted, 
		int32 NumPhotonsToFind,
		float SearchDistance,
		const FIrradiancePhoton& IrradiancePhoton,
		bool bDebugThisCalculation,
		TArray<FPhoton>& TempFoundPhotons,
		FCalculateIrradiancePhotonStats& Stats) const;

	/** Calculates incident radiance at a vertex from the given photon map. */
	FGatheredLightSample CalculatePhotonIncidentRadiance(
		const FPhotonOctree& PhotonMap,
		int32 NumPhotonsEmitted, 
		float SearchDistance,
		const FStaticLightingVertex& Vertex,
		bool bDebugThisDensityEstimation) const;

	/** Calculates exitant radiance at a vertex from the given photon map. */
	FLinearColor CalculatePhotonExitantRadiance(
		const FPhotonOctree& PhotonMap,
		int32 NumPhotonsEmitted, 
		float SearchDistance,
		const FStaticLightingMesh* Mesh,
		const FMinimalStaticLightingVertex& Vertex,
		int32 ElementIndex,
		const FVector4f& OutgoingDirection,
		bool bDebugThisDensityEstimation) const;

	/** Places volume lighting samples and calculates lighting for them. */
	void BeginCalculateVolumeSamples();

	/** 
	 * Interpolates lighting from the volume lighting samples to a vertex. 
	 * This mirrors FPrecomputedLightVolume::InterpolateIncidentRadiance in Unreal, used for visualizing interpolation from the lighting volume on surfaces.
	 */
	FGatheredLightSample InterpolatePrecomputedVolumeIncidentRadiance(
		const FStaticLightingVertex& Vertex, 
		float SampleRadius, 
		FCoherentRayCache& RayCache, 
		bool bDebugThisTexel) const;

	/** 
	 * Computes direct lighting for a volume point. 
	 * Caller is responsible for initializing the outputs to something valid.
	 */
	template<int32 SHOrder>
	void CalculateApproximateDirectLighting(
		const FStaticLightingVertex& Vertex,
		float SampleRadius,
		const TArray<FVector3f, TInlineAllocator<1>>& VertexOffsets,
		float LightSampleFraction,
		/** Whether to composite all direct lighting into OutStaticDirectLighting, regardless of static or toggleable. */
		bool bCompositeAllLights,
		bool bCalculateForIndirectLighting,
		bool bDebugThisSample,
		FStaticLightingMappingContext& MappingContext,
		TGatheredLightSample<SHOrder>& OutStaticDirectLighting,
		float& OutToggleableDirectionalLightShadowing) const;

	void GatherVolumeImportancePhotonDirections(
		const FVector3f WorldPosition,
		FVector3f FirstHemisphereNormal,
		FVector3f SecondHemisphereNormal,
		TArray<FVector4f>& FirstHemisphereImportancePhotonDirections,
		TArray<FVector4f>& SecondHemisphereImportancePhotonDirections,
		bool bDebugThisSample) const;

	/** Calculates incident radiance for a given world space position. */
	void CalculateVolumeSampleIncidentRadiance(
		const TArray<FVector4f>& UniformHemisphereSamples,
		const TArray<FVector2f>& UniformHemisphereSampleUniforms,
		float MaxUnoccludedLength,
		const TArray<FVector3f, TInlineAllocator<1>>& VertexOffsets,
		FVolumeLightingSample& LightingSample,
		float& OutBackfacingHitsFraction,
		float& OutMinDistanceToSurface,
		FLMRandomStream& RandomStream,
		FStaticLightingMappingContext& MappingContext,
		bool bDebugThisSample
		) const;

	/** Determines visibility cell placement, called once at startup. */
	void SetupPrecomputedVisibility();

	int32 GetGroupCellIndex(FVector3f BoxCenter) const;

	/** Calculates visibility for a given group of cells, called from all threads. */
	void CalculatePrecomputedVisibility(int32 BucketIndex);
	
	void RecursivelyBuildBrickTree(
		int32 StartCellIndex,
		int32 NumCells,
		FIntVector CellCoordinate, 
		int32 TreeDepth, 
		bool bCoveringDebugPosition,
		const FBox3f& TopLevelCellBounds, 
		const TArray<FVector3f>& VoxelTestPositions,
		const FGuid& IntersectingLevelGuid,
		TArray<struct FIrradianceBrickBuildData>& OutBrickBuildData);

	void ProcessVolumetricLightmapBrickTask(class FVolumetricLightmapBrickTaskDescription* Task);

	bool ProcessVolumetricLightmapTaskIfAvailable();

	void GenerateVoxelTestPositions(TArray<FVector3f>& VoxelTestPositions) const;

	void CalculateAdaptiveVolumetricLightmap(int32 TaskIndex);

	bool DoesVoxelIntersectSceneGeometry(const FBox3f& CellBounds, FGuid& OutIntersectingLevelGuid) const;

	bool ShouldRefineVoxel(int32 TreeDepth, const FBox3f& AABB, const TArray<FVector3f>& VoxelTestPositions, bool bDebugThisVoxel, FGuid& OutIntersectingLevelGuid) const;

	/** Computes a shadow depth map for a stationary light. */
	void CalculateStaticShadowDepthMap(FGuid LightGuid);

	/** Prepares for multithreaded generation of VolumeDistanceField. */
	void BeginCalculateVolumeDistanceField();

	/** Generates a single z layer of VolumeDistanceField. */
	void CalculateVolumeDistanceFieldWorkRange(int32 ZIndex);

	/**
	 * Calculates shadowing for a given mapping surface point and light.
	 * @param Mapping - The mapping the point comes from.
	 * @param WorldSurfacePoint - The point to check shadowing at.
	 * @param Light - The light to check shadowing from.
	 * @param CacheGroup - The calling thread's collision cache.
	 * @return true if the surface point is shadowed from the light.
	 */
	bool CalculatePointShadowing(
		const FStaticLightingMapping* Mapping,
		const FVector4f& WorldSurfacePoint,
		const FLight* Light,
		FStaticLightingMappingContext& MappingContext,
		bool bDebugThisSample) const;

	/** Calculates area shadowing from a light for the given vertex. */
	FVector2f CalculatePointAreaShadowing(
		const FStaticLightingMapping* Mapping,
		const FStaticLightingVertex& Vertex,
		int32 ElementIndex,
		float SampleRadius,
		const FLight* Light,
		FStaticLightingMappingContext& MappingContext,
		FLMRandomStream& RandomStream,
		FLinearColor& UnnormalizedTransmission,
		const TArray<FLightSurfaceSample>& LightPositionSamples,
		bool bDebugThisSample
		) const;

	/** Calculates the lighting contribution of a light to a mapping vertex. */
	FGatheredLightSample CalculatePointLighting(
		const FStaticLightingMapping* Mapping, 
		const FStaticLightingVertex& Vertex, 
		int32 ElementIndex,
		const FLight* Light,
		const FLinearColor& InLightIntensity,
		const FLinearColor& InTransmission
		) const;

	/** Returns environment lighting for the given direction. */
	FLinearColor EvaluateEnvironmentLighting(const FVector4f& IncomingDirection) const;

	/** Evaluates the incoming sky lighting from the scene. */
	void EvaluateSkyLighting(const FVector4f& IncomingDirection, float PathSolidAngle, bool bShadowed, bool bForDirectLighting, FLinearColor& OutStaticLighting, FLinearColor& OutStationaryLighting) const;

	float EvaluateSkyVariance(const FVector4f& IncomingDirection, float PathSolidAngle) const;

	/** Returns a light sample that represents the material attribute specified by MaterialSettings.ViewMaterialAttribute at the intersection. */
	FGatheredLightSample GetVisualizedMaterialAttribute(const FStaticLightingMapping* Mapping, const FLightRayIntersection& Intersection) const;

	/** Calculates exitant radiance at a vertex. */
	FLinearColor CalculateExitantRadiance(
		const FStaticLightingMapping* HitMapping,
		const FStaticLightingMesh* HitMesh,
		const FMinimalStaticLightingVertex& Vertex,
		int32 ElementIndex,
		const FVector4f& OutgoingDirection,
		int32 BounceNumber,
		EHemisphereGatherClassification GatherClassification,
		FStaticLightingMappingContext& MappingContext,
		bool bDebugThisTexel) const;

	void IntersectLightRay(
		const FStaticLightingMapping* Mapping,
		const FFullStaticLightingVertex& Vertex,
		float SampleRadius,
		const FVector4f& WorldPathDirection,
		const FVector4f& TangentPathDirection,
		EFinalGatherRayBiasMode RayBiasMode,
		FStaticLightingMappingContext& MappingContext,
		FLightRay& OutLightRay,
		FLightRayIntersection& OutLightRayIntersection) const;

	/** Takes a final gather sample in the given tangent space direction. */
	FLinearColor FinalGatherSample(
		const FStaticLightingMapping* Mapping,
		const FFullStaticLightingVertex& Vertex,
		const FVector4f& WorldPathDirection,
		const FVector4f& TangentPathDirection,
		const FLightRay& PathRay,
		const FLightRayIntersection& RayIntersection,
		float PathSolidAngle,
		int32 BounceNumber,
		EHemisphereGatherClassification GatherClassification,
		bool bGatheringForCachedDirectLighting,
		bool bDebugThisTexel,
		FStaticLightingMappingContext& MappingContext,
		FLMRandomStream& RandomStream,
		FLightingCacheGatherInfo& GatherInfo,
		FFinalGatherInfo& FinalGatherInfo,
		FFinalGatherHitPoint& HitPoint,
		FVector3f& OutUnoccludedSkyVector,
		FLinearColor& OutStationarySkyLighting) const;

	/** 
	 * Final gather using adaptive sampling to estimate the incident radiance function. 
	 * Adaptive refinement is done on brightness differences and anywhere that a first bounce photon determined lighting was coming from.
	 */
	template<class SampleType>
	SampleType IncomingRadianceAdaptive(
		const FStaticLightingMapping* Mapping,
		const FFullStaticLightingVertex& Vertex,
		float SampleRadius,
		bool bIntersectingSurface,
		int32 ElementIndex,
		int32 BounceNumber,
		EFinalGatherRayBiasMode RayBiasMode,
		EHemisphereGatherClassification GatherClassification,
		int32 NumAdaptiveRefinementLevels,
		float BrightnessThresholdScale,
		const TArray<FVector4f>& UniformHemisphereSamples,
		const TArray<FVector2f>& UniformHemisphereSampleUniforms,
		float MaxUnoccludedLength,
		const TArray<FVector4f>& ImportancePhotonDirections,
		FStaticLightingMappingContext& MappingContext,
		FLMRandomStream& RandomStream,
		FLightingCacheGatherInfo& GatherInfo,
		bool bGatheringForCachedDirectLighting,
		bool bDebugThisTexel) const;

	/** Calculates irradiance gradients for a sample position that will be cached. */
	void CalculateIrradianceGradients(
		int32 BounceNumber,
		const FLightingCacheGatherInfo& GatherInfo,
		FVector4f& RotationalGradient,
		FVector4f& TranslationalGradient) const;

	/** 
	 * Interpolates incoming radiance from the lighting cache if possible,
	 * otherwise estimates incoming radiance for this sample point and adds it to the cache. 
	 */
	FFinalGatherSample CachePointIncomingRadiance(
		const FStaticLightingMapping* Mapping,
		const FFullStaticLightingVertex& Vertex,
		int32 ElementIndex,
		float TexelRadius,
		float SampleRadius,
		bool bIntersectingSurface,
		FStaticLightingMappingContext& MappingContext,
		FLMRandomStream& RandomStream,
		bool bDebugThisTexel) const;

	/**
	 * Builds lighting for a texture mapping.
	 * @param TextureMapping - The mapping to build lighting for.
	 */
	void ProcessTextureMapping(FStaticLightingTextureMapping* TextureMapping);

	/** Traces a ray to the corner of a texel. */
	void TraceToTexelCorner(
		const FVector4f& TexelCenterOffset, 
		const FFullStaticLightingVertex& FullVertex, 
		FVector2f CornerSigns,
		float TexelRadius, 
		FStaticLightingMappingContext& MappingContext, 
		FLightRayIntersection& Intersection,
		bool& bHitBackface,
		bool bDebugThisTexel) const;

	/** Calculates TexelToVertexMap and initializes each texel's light sample as mapped or not. */
	void SetupTextureMapping(
		FStaticLightingTextureMapping* TextureMapping, 
		FGatheredLightMapData2D& LightMapData, 
		class FTexelToVertexMap& TexelToVertexMap, 
		class FTexelToCornersMap& TexelToCornersMap,
		FStaticLightingMappingContext& MappingContext,
		bool bDebugThisMapping) const;

	/** Calculates direct lighting as if all lights were non-area lights, then filters the results in texture space to create approximate soft shadows. */
	void CalculateDirectLightingTextureMappingFiltered(
		FStaticLightingTextureMapping* TextureMapping, 
		FStaticLightingMappingContext& MappingContext,
		FGatheredLightMapData2D& LightMapData, 
		TMap<const FLight*, FShadowMapData2D*>& ShadowMaps,
		const FTexelToVertexMap& TexelToVertexMap, 
		bool bDebugThisMapping,
		const FLight* Light) const;

	/**
	 * Calculate lighting from area lights, no filtering in texture space.  
	 * Shadow penumbras will be correctly shaped and will be softer for larger light sources and distant shadow casters.
	 */
	void CalculateDirectAreaLightingTextureMapping(
		FStaticLightingTextureMapping* TextureMapping, 
		FStaticLightingMappingContext& MappingContext,
		FGatheredLightMapData2D& LightMapData, 
		FShadowMapData2D*& ShadowMapData,
		const FTexelToVertexMap& TexelToVertexMap, 
		bool bDebugThisMapping,
		const FLight* Light) const;

	/** 
	 * Calculate signed distance field shadowing from a single light,  
	 * Based on the paper "Improved Alpha-Tested Magnification for Vector Textures and Special Effects" by Valve.
	 */
	void CalculateDirectSignedDistanceFieldLightingTextureMappingTextureSpace(
		FStaticLightingTextureMapping* TextureMapping, 
		FStaticLightingMappingContext& MappingContext,
		FGatheredLightMapData2D& LightMapData, 
		TMap<const FLight*, FSignedDistanceFieldShadowMapData2D*>& ShadowMaps,
		const FTexelToVertexMap& TexelToVertexMap, 
		const FTexelToCornersMap& TexelToCornersMap,
		bool bDebugThisMapping,
		const FLight* Light) const;

	/** Experimental method that avoids artifacts due to lightmap seams. */
	void CalculateDirectSignedDistanceFieldLightingTextureMappingLightSpace(
		FStaticLightingTextureMapping* TextureMapping, 
		FStaticLightingMappingContext& MappingContext,
		FGatheredLightMapData2D& LightMapData, 
		TMap<const FLight*, FSignedDistanceFieldShadowMapData2D*>& ShadowMaps,
		const FTexelToVertexMap& TexelToVertexMap, 
		const FTexelToCornersMap& TexelToCornersMap,
		bool bDebugThisMapping,
		const FLight* Light) const;

	/**
	 * Estimate direct lighting using the direct photon map.
	 * This is only useful for debugging what the final gather rays see.
	 */
	void CalculateDirectLightingTextureMappingPhotonMap(
		FStaticLightingTextureMapping* TextureMapping, 
		FStaticLightingMappingContext& MappingContext,
		FGatheredLightMapData2D& LightMapData, 
		TMap<const FLight*, FShadowMapData2D*>& ShadowMaps,
		const FTexelToVertexMap& TexelToVertexMap, 
		bool bDebugThisMapping) const;

	/** 
	 * Builds an irradiance cache for a given mapping task.  
	 * This can be called from any thread, not just the thread that owns the mapping, so called code must be thread safe in that manner.
	 */
	void ProcessCacheIndirectLightingTask(FCacheIndirectTaskDescription* Task, bool bProcessedByMappingThread);

	/** 
	 * Interpolates from the irradiance cache for a given mapping task.  
	 * This can be called from any thread, not just the thread that owns the mapping, so called code must be thread safe in that manner.
	 */
	void ProcessInterpolateTask(FInterpolateIndirectTaskDescription* Task, bool bProcessedByMappingThread);

	void ProcessVolumeSamplesTask(const FVolumeSamplesTaskDescription& Task);

	/** Handles indirect lighting calculations for a single texture mapping. */
	void CalculateIndirectLightingTextureMapping(
		FStaticLightingTextureMapping* TextureMapping,
		FStaticLightingMappingContext& MappingContext,
		FGatheredLightMapData2D& LightMapData, 
		const FTexelToVertexMap& TexelToVertexMap, 
		bool bDebugThisMapping);

	/** Overrides LightMapData with material attributes if MaterialSettings.ViewMaterialAttribute != VMA_None */
	void ViewMaterialAttributesTextureMapping(
		FStaticLightingTextureMapping* TextureMapping,
		FStaticLightingMappingContext& MappingContext,
		FGatheredLightMapData2D& LightMapData, 
		const FTexelToVertexMap& TexelToVertexMap, 
		bool bDebugThisMapping) const;

	/** Colors texels with invalid lightmap UVs to make it obvious that they are wrong. */
	void ColorInvalidLightmapUVs(
		const FStaticLightingTextureMapping* TextureMapping, 
		FGatheredLightMapData2D& LightMapData, 
		bool bDebugThisMapping) const;

	/** Adds a texel of padding around texture mappings and copies the nearest texel into the padding. */
	void PadTextureMapping(
		const FStaticLightingTextureMapping* TextureMapping,
		const FGatheredLightMapData2D& LightMapData,
		FGatheredLightMapData2D& PaddedLightMapData,
		TMap<const FLight*, FShadowMapData2D*>& ShadowMaps,
		TMap<const FLight*, FSignedDistanceFieldShadowMapData2D*>& SignedDistanceFieldShadowMaps) const;

	/**
	 * Retrieves the next task from Swarm. Optionally blocking, thread-safe function call. Returns NULL when there are no more tasks.
	 * @param WaitTime The timeout period in milliseconds for the request
	 * @param bWaitTimedOut Output parameter; true if the request timed out, false if not
	 * @return	The next mapping task to process.
	 */
	FStaticLightingMapping*	ThreadGetNextMapping( 
		FThreadStatistics& 
		ThreadStatistics, 
		FGuid& TaskGuid, 
		uint32 WaitTime, 
		bool& bWaitTimedOut, 
		bool& bDynamicObjectTask, 
		int32& PrecomputedVisibilityTaskIndex,
		int32& VolumetricLightmapTaskIndex,
		bool& DominantShadowTask,
		bool& bMeshAreaLightDataTask,
		bool& bVolumeDataTask);

	/**
	 * The processing loop for a static lighting thread.
	 * @param bIsMainThread		- true if this is running in the main thread.
	 * @param ThreadStatistics	- [out] Various thread statistics
	 */
	void ThreadLoop(bool bIsMainThread, int32 ThreadIndex, FThreadStatistics& ThreadStatistics, float& IdleTime);

	/**
	 * Creates multiple worker threads and starts the process locally.
	 */
	void MultithreadProcess();

	/** The lights in the world which the system is building, excluding sky lights. */
	TArray<FLight*> Lights;

	TArray<FSkyLight*> SkyLights;

	/** Mesh area lights in the world. */
	TIndirectArray<FMeshAreaLight> MeshAreaLights;

	/** The options the system is building lighting with. */
	const FLightingBuildOptions Options;

	/** Critical section to synchronize the access to Mappings (used only when GDebugMode is true). */
	FCriticalSection CriticalSection;

	/** References to the scene's settings, for convenience */
	FStaticLightingSettings& GeneralSettings;
	FStaticLightingSceneConstants& SceneConstants;
	FSceneMaterialSettings& MaterialSettings;
	FMeshAreaLightSettings& MeshAreaLightSettings;
	FDynamicObjectSettings& DynamicObjectSettings;
	FVolumetricLightmapSettings& VolumetricLightmapSettings;
	FPrecomputedVisibilitySettings& PrecomputedVisibilitySettings;
	FVolumeDistanceFieldSettings& VolumeDistanceFieldSettings;
	FAmbientOcclusionSettings& AmbientOcclusionSettings;
	FStaticShadowSettings& ShadowSettings;
	FImportanceTracingSettings& ImportanceTracingSettings;
	FPhotonMappingSettings& PhotonMappingSettings;
	FIrradianceCachingSettings& IrradianceCachingSettings;

	/** Stats of the system */
	mutable FStaticLightingStats Stats;

	/** 
	 * Counts the number of mapping tasks that have begun, and might need help from other threads with tasks that they generate. 
	 * This is used to keep completed mapping threads running so they can check for tasks. 
	 */
	volatile int32 TasksInProgressThatWillNeedHelp;

	/** List of tasks to cache indirect lighting, used by all mapping threads. */
	// consider changing this from FIFO to Unordered, which may be faster
	TLockFreePointerListLIFO<FCacheIndirectTaskDescription> CacheIndirectLightingTasks;

	/** List of tasks to interpolate indirect lighting, used by all mapping threads. */
	// consider changing this from FIFO to Unordered, which may be faster
	TLockFreePointerListLIFO<FInterpolateIndirectTaskDescription> InterpolateIndirectLightingTasks;

	TLockFreePointerListLIFO<FVolumetricLightmapBrickTaskDescription> VolumetricLightmapBrickTasks;

	TArray<FVolumeSamplesTaskDescription> VolumeSampleTasks;

	volatile int32 bHasVolumeSampleTasks;
	volatile int32 NextVolumeSampleTaskIndex;
	volatile int32 NumVolumeSampleTasksOutstanding;
	volatile int32 bShouldExportVolumeSampleData;
	/** Bounds that VolumeLightingSamples were generated in. */
	FBoxSphereBounds3f VolumeBounds;
	/** Octree used for interpolating the volume lighting samples if DynamicObjectSettings.bVisualizeVolumeLightInterpolation is true. */
	FVolumeLightingInterpolationOctree VolumeLightingInterpolationOctree;
	/** Map from Level Guid to array of volume lighting samples generated. */
	TMap<FGuid,TArray<FVolumeLightingSample> > VolumeLightingSamples;

	/** All precomputed visibility cells in the scene.  Some of these may be processed on other agents. */
	TArray<FPrecomputedVisibilityCell> AllPrecomputedVisibilityCells;

	/** Threads must acquire this critical section before reading or writing to CompletedStaticShadowDepthMaps. */
	FCriticalSection CompletedStaticShadowDepthMapsSync;

	/** Static shadow depth maps ready to be exported by the main thread. */
	TMap<const FLight*, FStaticShadowDepthMap*> CompletedStaticShadowDepthMaps;

	/** Non-zero if the mesh area light data task should be exported. */
	volatile int32 bShouldExportMeshAreaLightData;

	/** Non-zero if the volume distance field task should be exported. */
	volatile int32 bShouldExportVolumeDistanceField;

	/** Number of direct photons to emit */
	int32 NumDirectPhotonsToEmit;
	/** Number of photons that were emitted until enough direct photons were gathered */
	int32 NumPhotonsEmittedDirect;
	/** Photon map for direct photons */
	FPhotonOctree DirectPhotonMap;

	/** The target number of indirect photon paths to gather. */
	int32 NumIndirectPhotonPaths;
	/** Number of indirect photons to emit */
	int32 NumIndirectPhotonsToEmit;
	/** Number of photons that were emitted until enough first bounce photons were gathered */
	int32 NumPhotonsEmittedFirstBounce;
	/** 
	 * Photon map for first bounce indirect photons.  
	 * This is separate from other indirect photons so we can access just first bounce photons and use them for guiding the final gather.
	 */
	FPhotonOctree FirstBouncePhotonMap;

	/** Tracks first bounce photons that did not intersect a surface and escaped.  Used when lighting volumes. */
	FPhotonOctree FirstBounceEscapedPhotonMap;

	/** Stores photon segments, which allows finding photons which travelled near a certain point in space. */
	FPhotonSegmentOctree FirstBouncePhotonSegmentMap;

	/** Number of photons that were emitted until enough second bounce photons were gathered */
	int32 NumPhotonsEmittedSecondBounce;
	/** Photon map for second and up bounce photons. */
	FPhotonOctree SecondBouncePhotonMap;

	/** Fraction of direct photons deposited to calculate irradiance at. */
	float DirectIrradiancePhotonFraction;
	/** Fraction of indirect photons deposited to calculate irradiance at. */
	float IndirectIrradiancePhotonFraction;
	/** Photon map storing irradiance photons */
	FIrradiancePhotonOctree IrradiancePhotonMap;

	/** 
	 * Irradiance photons generated by photon emission.  
	 * Each array was generated on a separate thread, so these are stored as an array of irradiance photon arrays,
	 * Which avoids copying to one large array, since that can take a while due to the large irradiance photon memory size.
	 */
	TArray<TArray<FIrradiancePhoton>> mIrradiancePhotons;

	/** Maximum distance to trace a ray through the scene. */
	float MaxRayDistance;

	/** Cached direction samples for hemisphere gathers. */
	TArray<FVector4f> CachedHemisphereSamples;

	/** Length of all the hemisphere samples averaged, which is also the max length that a bent normal can be. */
	float CachedSamplesMaxUnoccludedLength;

	TArray<FVector2f> CachedHemisphereSampleUniforms;

	TArray<FVector4f> CachedHemisphereSamplesForRadiosity[3];
	TArray<FVector2f> CachedHemisphereSamplesForRadiosityUniforms[3];

	TArray<FVector4f> CachedVolumetricLightmapUniformHemisphereSamples;
	TArray<FVector2f> CachedVolumetricLightmapUniformHemisphereSampleUniforms;
	float CachedVolumetricLightmapMaxUnoccludedLength;
	TArray<FVector3f, TInlineAllocator<1>> CachedVolumetricLightmapVertexOffsets;

	/** The aggregate mesh used for raytracing. */
	FStaticLightingAggregateMeshType* AggregateMesh;

	FDefaultAggregateMesh* VoxelizationSurfaceAggregateMesh;
	FDefaultAggregateMesh* VoxelizationVolumeAggregateMesh;
	FDefaultAggregateMesh* LandscapeCullingVoxelizationAggregateMesh;

	/** The input scene describing geometry, materials and lights. */
	const FScene& Scene; 

	/** All meshes in the system. */
	TArray< FStaticLightingMesh* > Meshes;

	/** All meshes involved in sPVS indexed by their visibility id, setup at scene setup time. */
	TArray<FVisibilityMesh> VisibilityMeshes;

	/** Visibility groups which are clusters of meshes, generated at PVS startup time. */
	TArray<FVisibilityMeshGroup> VisibilityGroups;

	/** X and Y dimensions of GroupGrid. */
	int32 GroupVisibilityGridSizeXY;

	/** Z dimension of GroupGrid. */
	int32 GroupVisibilityGridSizeZ;

	/** World space bounding box of GroupGrid. */
	FBox3f VisibilityGridBounds;

	/** Grid of indices into VisibilityGroups. */
	TArray<int32> GroupGrid;

	/** All mappings in the system. */
	TArray< FStaticLightingMapping* > AllMappings;

	/** All mappings in the system for which lighting will be built. */
	TMap< FGuid, FStaticLightingMapping* > Mappings;

	/** The next index into Mappings which processing hasn't started for yet. */
	FThreadSafeCounter NextMappingToProcess;

	/** Stats on how many texels and vertices have been completed, written and read by all threads. */
	volatile int32 NumTexelsCompleted;

	/** A list of the texture mappings which static lighting has been computed for, but not yet applied.  This is accessed by multiple threads and should be written to using interlocked functions. */
	TCompleteStaticLightingList<FTextureMappingStaticLightingData> CompleteTextureMappingList;

	/** List of complete visibility task data. */
	TCompleteTaskList<FPrecomputedVisibilityData> CompleteVisibilityTaskList;

	TCompleteTaskList<FVolumetricLightmapTaskData> CompleteVolumetricLightmapTaskList;

	// Landscape mapping for Lighting Sample number estimation...
	TArray< FStaticLightingMapping* > LandscapeMappings;

	int32 VolumeSizeX;
	int32 VolumeSizeY;
	int32 VolumeSizeZ;
	FVector4f::FReal DistanceFieldVoxelSize;
	FBox3f DistanceFieldVolumeBounds;
	TArray<FColor> VolumeDistanceField;

	/** */
	volatile int32 NumOutstandingVolumeDataLayers;
	/** */
	volatile int32 OutstandingVolumeDataLayerIndex;

	/** Number of threads to use for static lighting */
	const int32 NumStaticLightingThreads;

	/** The threads spawned by the static lighting system for processing mappings. */
	TIndirectArray<FMappingProcessingThreadRunnable> Threads;

	/** Index of the next entry in DirectPhotonEmittingWorkRanges to process. */
	FThreadSafeCounter DirectPhotonEmittingWorkRangeIndex;
	TArray<FDirectPhotonEmittingWorkRange> DirectPhotonEmittingWorkRanges;
	TArray<FDirectPhotonEmittingOutput> DirectPhotonEmittingOutputs;

	/** Index of the next entry in IndirectPhotonEmittingWorkRanges to process. */
	FThreadSafeCounter IndirectPhotonEmittingWorkRangeIndex;
	TArray<FIndirectPhotonEmittingWorkRange> IndirectPhotonEmittingWorkRanges;
	TArray<FIndirectPhotonEmittingOutput> IndirectPhotonEmittingOutputs;

	/** Index of the next entry in IrradianceMarkWorkRanges to process. */
	FThreadSafeCounter IrradianceMarkWorkRangeIndex;
	TArray<FIrradianceMarkingWorkRange> IrradianceMarkWorkRanges;

	/** Index of the next entry in IrradianceCalculationWorkRanges to process. */
	FThreadSafeCounter IrradianceCalcWorkRangeIndex;
	TArray<FIrradianceCalculatingWorkRange> IrradianceCalculationWorkRanges;

	/** Index of the next mapping in AllMappings to cache irradiance photons on */
	FThreadSafeCounter NextMappingToCacheIrradiancePhotonsOn;
	/** Index into IrradiancePhotons of the array containing the photon being debugged, or INDEX_NONE if no photon is being debugged. */
	int32 DebugIrradiancePhotonCalculationArrayIndex;
	/** Index into IrradiancePhotons(DebugIrradiancePhotonCalculationArrayIndex) of the photon being debugged. */
	int32 DebugIrradiancePhotonCalculationPhotonIndex;

    TIndirectArray<FMappingProcessingThreadRunnable> IrradiancePhotonCachingThreads;

	FThreadSafeCounter NextMappingToProcessRadiositySetup;
	FThreadSafeCounter NextMappingToProcessRadiosityIterations;
	TArray<FThreadSafeCounter> NumCompletedRadiosityIterationMappings;

	FThreadSafeCounter NextMappingToFinalizeSurfaceCache;

	TIndirectArray<FMappingProcessingThreadRunnable> RadiositySetupThreads;
	TIndirectArray<FMappingProcessingThreadRunnable> RadiosityIterationThreads;
	TIndirectArray<FMappingProcessingThreadRunnable> FinalizeSurfaceCacheThreads;

	/** Lightmass exporter (back to Unreal) */
	class FLightmassSolverExporter& Exporter;

	// allow the vertex mapper access to the private functions
	friend class FStaticLightingMappingContext;
	friend class FLightingCacheBase;
	friend class FStaticLightingThreadRunnable;
	friend class FDirectPhotonEmittingThreadRunnable;
	friend class FIndirectPhotonEmittingThreadRunnable;
	friend class FIrradiancePhotonMarkingThreadRunnable;
	friend class FIrradiancePhotonCalculatingThreadRunnable;
	friend class FMappingProcessingThreadRunnable;
	friend class FVolumeSamplePlacementRasterPolicy;
	friend class FUniformHemisphereRefinementGrid;
	friend class FStaticLightingTextureMapping;
};

/**
 * Checks if a light is behind a triangle.
 * @param TrianglePoint - Any point on the triangle.
 * @param TriangleNormal - The (not necessarily normalized) triangle surface normal.
 * @param Light - The light to classify.
 * @return true if the light is behind the triangle.
 */
extern bool IsLightBehindSurface(const FVector4f& TrianglePoint, const FVector4f& TriangleNormal, const FLight* Light);

/**
 * Culls lights that are behind a triangle.
 * @param bTwoSidedMaterial - true if the triangle has a two-sided material.  If so, lights behind the surface are not culled.
 * @param TrianglePoint - Any point on the triangle.
 * @param TriangleNormal - The (not necessarily normalized) triangle surface normal.
 * @param Lights - The lights to cull.
 * @return A map from Lights index to a boolean which is true if the light is in front of the triangle.
 */
extern TBitArray<> CullBackfacingLights(bool bTwoSidedMaterial, const FVector4f& TrianglePoint, const FVector4f& TriangleNormal, const TArray<FLight*>& Lights);

#include "LightingSystem.inl" // IWYU pragma: export

} //namespace Lightmass

