// Copyright Epic Games, Inc. All Rights Reserved.

template<int32 SHOrder>
TGatheredLightSample<SHOrder> FGatheredLightSampleUtil::AmbientLight(const FLinearColor& Color)
{
	TGatheredLightSample<SHOrder> Result;
	Result.SHVector.AddAmbient(Color);

	// Compute SHCorrection as if all the lighting was coming in along the normal
	FVector4f TangentDirection(0, 0, 1);

	FSHVector2 SH = FSHVector2::SHBasisFunction(FVector4(TangentDirection));
	Result.SHCorrection = Color.GetLuminance() * (0.282095f * SH.V[0] + 0.325735f * SH.V[2]);

	Result.IncidentLighting = Color;

	checkSlow(Result.SHCorrection >= 0 && Result.IncidentLighting.GetMin() >= 0);

	return Result;
}

template<int32 SHOrder>
TGatheredLightSample<SHOrder> FGatheredLightSampleUtil::PointLightWorldSpace(const FLinearColor& Color, const FVector4f& TangentDirection, const FVector4f& WorldDirection)
{
	TGatheredLightSample<SHOrder> Result;

	if (TangentDirection.Z >= 0.0f)
	{
		Result.SHVector.AddIncomingRadiance(Color, 1, FVector4(WorldDirection));

		FSHVector2 SH = FSHVector2::SHBasisFunction(FVector4(TangentDirection));
		// Evaluate lighting along the smoothed vertex normal direction, so that later we can guarantee an SH intensity of 1 along the normal
		// These scaling coefficients are SHBasisFunction and CalcDiffuseTransferSH baked down
		// 0.325735f = 0.488603f from SHBasisFunction * 2/3 from CalcDiffuseTransferSH
		// Only using V[2] which is the tangent space Z
		Result.SHCorrection = Color.GetLuminance() * (0.282095f * SH.V[0] + 0.325735f * SH.V[2]);
		Result.IncidentLighting = Color * FMath::Max(0.0f, TangentDirection.Z);

		checkSlow(Result.SHCorrection >= 0 && Result.IncidentLighting.GetMin() >= 0);
	}

	return Result;
}

template<int32 SHOrder>
void TGatheredLightSample<SHOrder>::AddWeighted(const TGatheredLightSample<SHOrder>& OtherSample, float Weight)
{
	SHVector += OtherSample.SHVector * Weight;
	SHCorrection += OtherSample.SHCorrection * Weight;
	IncidentLighting += OtherSample.IncidentLighting * Weight;
	SkyOcclusion += OtherSample.SkyOcclusion * Weight;
	AOMaterialMask += OtherSample.AOMaterialMask * Weight;

}
template void TGatheredLightSample<2>::AddWeighted(const TGatheredLightSample<2>& OtherSample, float Weight);
template void TGatheredLightSample<3>::AddWeighted(const TGatheredLightSample<3>& OtherSample, float Weight);

template<int32 SHOrder>
void TGatheredLightSample<SHOrder>::ApplyOcclusion(float Occlusion)
{
	SHVector *= Occlusion;
	SHCorrection *= Occlusion;
	IncidentLighting *= Occlusion;
}
template void TGatheredLightSample<2>::ApplyOcclusion(float Occlusion);
template void TGatheredLightSample<3>::ApplyOcclusion(float Occlusion);

template<int32 SHOrder>
bool TGatheredLightSample<SHOrder>::AreFloatsValid() const
{
	return SHVector.AreFloatsValid()
		&& FMath::IsFinite(SHCorrection) && !FMath::IsNaN(SHCorrection)
		&& FLinearColorUtils::AreFloatsValid(IncidentLighting);
}
template bool TGatheredLightSample<2>::AreFloatsValid() const;
template bool TGatheredLightSample<3>::AreFloatsValid() const;

template<int32 SHOrder>
void TFinalGatherSample<SHOrder>::AddWeighted(const TFinalGatherSample<SHOrder>& OtherSample, float Weight)
{
	TGatheredLightSample<SHOrder>::AddWeighted(OtherSample, Weight);
	Occlusion += OtherSample.Occlusion * Weight;
	StationarySkyLighting = StationarySkyLighting + OtherSample.StationarySkyLighting * Weight;
}

template<int32 SHOrder>
bool TFinalGatherSample<SHOrder>::AreFloatsValid() const
{
	return TGatheredLightSample<SHOrder>::AreFloatsValid() && FMath::IsFinite(Occlusion) && !FMath::IsNaN(Occlusion);
}

template<int32 SHOrder>
void FStaticLightingSystem::CalculateApproximateDirectLighting(
	const FStaticLightingVertex& Vertex,
	float SampleRadius,
	const TArray<FVector3f, TInlineAllocator<1>>& VertexOffsets,
	float LightSampleFraction,
	bool bCompositeAllLights,
	bool bCalculateForIndirectLighting,
	bool bDebugThisSample,
	FStaticLightingMappingContext& MappingContext,
	TGatheredLightSample<SHOrder>& OutStaticDirectLighting,
	float& OutToggleableDirectionalLightShadowing) const
{
	check(VertexOffsets.Num() > 0);

	for (int32 LightIndex = 0; LightIndex < Lights.Num(); LightIndex++)
	{
		const FLight* Light = Lights[LightIndex];

		if (Light->AffectsBounds(FBoxSphereBounds3f(FSphere3f(Vertex.WorldPosition, SampleRadius))))
		{
			FLinearColor LightIntensity(0, 0, 0, 0);

			for (int32 OffsetIndex = 0; OffsetIndex < VertexOffsets.Num(); OffsetIndex++)
			{
				LightIntensity += Light->GetDirectIntensity(Vertex.WorldPosition + VertexOffsets[OffsetIndex] * SampleRadius, bCalculateForIndirectLighting);
			}

			LightIntensity /= (float)VertexOffsets.Num();

			FLinearColor Transmission = FLinearColor::Black;

			if ((Light->LightFlags & GI_LIGHT_CASTSHADOWS) && (Light->LightFlags & GI_LIGHT_CASTSTATICSHADOWS))
			{
				int32 UnShadowedRays = 0;
				FLinearColor UnnormalizedTransmission = FLinearColor::Black;

				const TArray<FLightSurfaceSample>& LightSurfaceSamples = Light->GetCachedSurfaceSamples(0, false);
				const int32 NumSamplesToTrace = FMath::Max(FMath::TruncToInt(LightSurfaceSamples.Num() * LightSampleFraction), 1);

				for (int32 RayIndex = 0; RayIndex < NumSamplesToTrace; RayIndex++)
				{
					FLightSurfaceSample CurrentSample = LightSurfaceSamples[RayIndex];
					// Allow the light to modify the surface position for this receiving position
					Light->ValidateSurfaceSample(Vertex.WorldPosition, CurrentSample);

					// Construct a line segment between the light and the volume point.
					const FVector4f LightVector = CurrentSample.Position - Vertex.WorldPosition;

					FVector4f NormalForOffset = Vertex.WorldTangentZ;

					const FVector4f StartOffset = LightVector.GetSafeNormal() * SceneConstants.VisibilityRayOffsetDistance
						+ NormalForOffset * SampleRadius * SceneConstants.VisibilityNormalOffsetSampleRadiusScale;

					const FLightRay LightRay(
						// Offset the start of the ray by some fraction along the direction of the ray and some fraction along the vertex normal.
						Vertex.WorldPosition
						+ StartOffset,
						Vertex.WorldPosition + LightVector,
						NULL,
						Light
						);

					// Check the line segment for intersection with the static lighting meshes.
					FLightRayIntersection Intersection;
					//@todo - change this back to request boolean visibility once transmission is supported with boolean visibility ray intersections
					AggregateMesh->IntersectLightRay(LightRay, true, true, true, MappingContext.RayCache, Intersection);

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
					if (bDebugThisSample)
					{
						FDebugStaticLightingRay DebugRay(LightRay.Start, LightRay.End, Intersection.bIntersects);
						if (Intersection.bIntersects)
						{
							DebugRay.End = Intersection.IntersectionVertex.WorldPosition;
						}
						MappingContext.DebugOutput->ShadowRays.Add(DebugRay);
					}
#endif

					if (!Intersection.bIntersects)
					{
						UnnormalizedTransmission += Intersection.Transmission;
						UnShadowedRays++;
					}
				}

				if (UnShadowedRays > 0)
				{
					Transmission = UnnormalizedTransmission / UnShadowedRays;
				}
			}
			else
			{
				// Shadow casting disabled on this light
				Transmission = FLinearColor::White;
			}

			{
				// Calculate the direction from the vertex to the light.
				const FVector4f WorldLightVector = Light->GetDirectLightingDirection(Vertex.WorldPosition, Vertex.WorldTangentZ);

				// Transform the light vector to tangent space.
				const FVector4f TangentLightVector =
					FVector4f(
					Dot3(WorldLightVector, Vertex.WorldTangentX),
					Dot3(WorldLightVector, Vertex.WorldTangentY),
					Dot3(WorldLightVector, Vertex.WorldTangentZ),
					0
					).GetSafeNormal();

				// Compute the incident lighting of the light on the vertex.
				const FLinearColor FinalIntensity = LightIntensity * Transmission;

				// Compute the light-map sample for the front-face of the vertex.
				TGatheredLightSample<SHOrder> Lighting = FGatheredLightSampleUtil::PointLightWorldSpace<SHOrder>(FinalIntensity, TangentLightVector, WorldLightVector.GetSafeNormal());

				if (Light->UseStaticLighting() || bCompositeAllLights)
				{
					OutStaticDirectLighting = OutStaticDirectLighting + Lighting;
				}
				else if (Light->GetDirectionalLight() != NULL)
				{
					OutToggleableDirectionalLightShadowing = Transmission.GetLuminance();
				}
			}
		}
	}
}