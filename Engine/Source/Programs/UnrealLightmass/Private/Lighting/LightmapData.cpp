// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightmapData.h"
#include "Exporter.h"
#include "LightmassSwarm.h"
#include "Misc/Compression.h"

/** Maximum light intensity stored in vertex/ texture lightmaps. */
#define MAX_LIGHT_INTENSITY	16.f

namespace Lightmass
{
	/**
	 * Perform compression on 1D or 2D lightmap data
	 *
	 * @param UncompressedBuffer Source buffer
	 * @param UncompressedDataSize Size of source buffer
	 * @param [out] CompressedBuffer Output buffer
	 * @param [out] Size of data in CompressedBuffer (actual buffer will be larger)
	 */
	void CompressData(uint8* UncompressedBuffer, uint32 UncompressedDataSize, uint8*& CompressedBuffer, uint32& CompressedDataSize)
	{
		// don't compress zero data
		if (UncompressedDataSize == 0)
		{
			CompressedDataSize = 0;
			return;
		}

		check(UncompressedDataSize <= INT_MAX);

		// allocate all of the input space for the output, with extra space for max overhead of zlib (when compressed > uncompressed)
		int32 CompressedSize = FCompression::CompressMemoryBound(NAME_Zlib, (int32)UncompressedDataSize);
		CompressedBuffer = (uint8*)FMemory::Malloc(CompressedSize);

		// compress the data		
		bool bSucceeded = FCompression::CompressMemory(NAME_Zlib, CompressedBuffer, CompressedSize, UncompressedBuffer, UncompressedDataSize);

		// if it failed send the data uncompressed, which we mark by setting compressed size to 0
		checkf(bSucceeded, TEXT("zlib failed to compress, which is very unexpected"));

		// cache the compressed size in the header so the other side knows how much to read
		CompressedDataSize = CompressedSize;
	}


	/**
	 * Compresses the raw lightmap data to a buffer for writing over Swarm
	 */
	void FLightMapData2D::Compress(int32 DebugSampleIndex)
	{
		// make sure the data has been quantized already
		Quantize(DebugSampleIndex);

		// calculate the uncompressed size
		UncompressedDataSize = sizeof(FQuantizedLightSampleData) * QuantizedData.Num();

		// compress the array
		CompressData((uint8*)QuantizedData.GetData(), UncompressedDataSize, CompressedData, CompressedDataSize);

		// we no longer need the source data now that we're compressed
		QuantizedData.Empty();
	}

	static void GetLUVW( const float RGB[3], float& L, float& U, float& V, float& W )
	{
		float R = FMath::Max( 0.0f, RGB[0] );
		float G = FMath::Max( 0.0f, RGB[1] );
		float B = FMath::Max( 0.0f, RGB[2] );

		L = 0.3f * R + 0.59f * G + 0.11f * B;
		if( L < 1e-4f )
		{
			U = 1.0f;
			V = 1.0f;
			W = 1.0f;
		}
		else
		{
			U = R / L;
			V = G / L;
			W = B / L;
		}
	}

	/**
	 * Quantizes floating point light samples down to byte samples with a scale applied to all samples
	 *
	 * @param InLightSamples Floating point light sample coefficients
	 * @param OutLightSamples Quantized light sample coefficients
	 * @param OutScale Scale applied to each quantized sample (to get it back near original floating point value)
	 * @param bUseMappedFlag Whether or not to pay attention to the bIsMapped flag for each sample when calculating max scale
	 *
	 * TODO Calculate residual after compression, not just quantization.
	 * TODO Factor out error from directionality compression and push it to color. This requires knowing a representative normal.
	 * Best way is probably to create a new texture compression type and do error correcting during compression.
	 */
	void QuantizeLightSamples(
		TArray<FLightSample>& InLightSamples, 
		TArray<FQuantizedLightSampleData>& OutLightSamples, 
		float OutMultiply[LM_NUM_STORED_LIGHTMAP_COEF][4], 
		float OutAdd[LM_NUM_STORED_LIGHTMAP_COEF][4],
		int32 DebugSampleIndex,
		bool bUseMappedFlag)
	{
		//const float EncodeExponent = .5f;
		const float LogScale = 11.5f;
		const float LogBlackPoint = FMath::Pow( 2.0f, -0.5f * LogScale );
		const float SimpleLogScale = 16.0f;
		const float SimpleLogBlackPoint = FMath::Pow( 2.0f, -0.5f * SimpleLogScale );

		float MinCoefficient[LM_NUM_STORED_LIGHTMAP_COEF][4];
		float MaxCoefficient[LM_NUM_STORED_LIGHTMAP_COEF][4];

		for( int32 CoefficientIndex = 0; CoefficientIndex < LM_NUM_STORED_LIGHTMAP_COEF; CoefficientIndex += 2 )
		{
			for( int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++ )
			{
				// Color
				MinCoefficient[ CoefficientIndex ][ ColorIndex ] = 10000.0f;
				MaxCoefficient[ CoefficientIndex ][ ColorIndex ] = -10000.0f;

				// Direction
				MinCoefficient[ CoefficientIndex + 1 ][ ColorIndex ] = 10000.0f;
				MaxCoefficient[ CoefficientIndex + 1 ][ ColorIndex ] = -10000.0f;
			}
		}

		// go over all samples looking for min and max values
		for (int32 SampleIndex = 0; SampleIndex < InLightSamples.Num(); SampleIndex++)
		{
			const FLightSample& SourceSample = InLightSamples[SampleIndex];
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (SampleIndex == DebugSampleIndex)
			{
				int32 TempBreak = 0;
			}
#endif
			if(SourceSample.bIsMapped)
			{
				{
					// Complex
					float L, U, V, W;
					GetLUVW( SourceSample.Coefficients[0], L, U, V, W );

					float LogL = FMath::Log2( L + LogBlackPoint );

					MinCoefficient[0][0] = FMath::Min( MinCoefficient[0][0], U );
					MaxCoefficient[0][0] = FMath::Max( MaxCoefficient[0][0], U );

					MinCoefficient[0][1] = FMath::Min( MinCoefficient[0][1], V );
					MaxCoefficient[0][1] = FMath::Max( MaxCoefficient[0][1], V );
				
					MinCoefficient[0][2] = FMath::Min( MinCoefficient[0][2], W );
					MaxCoefficient[0][2] = FMath::Max( MaxCoefficient[0][2], W );
				
					MinCoefficient[0][3] = FMath::Min( MinCoefficient[0][3], LogL );
					MaxCoefficient[0][3] = FMath::Max( MaxCoefficient[0][3], LogL );

					// Dampen dark texel's contribution on the directionality min and max
					float DampenDirectionality = FMath::Clamp(L * 100, 0.0f, 1.0f);

					for( int32 ColorIndex = 0; ColorIndex < 3; ColorIndex++ )
					{
						MinCoefficient[1][ColorIndex] = FMath::Min( MinCoefficient[1][ColorIndex], DampenDirectionality * SourceSample.Coefficients[1][ColorIndex] );
						MaxCoefficient[1][ColorIndex] = FMath::Max( MaxCoefficient[1][ColorIndex], DampenDirectionality * SourceSample.Coefficients[1][ColorIndex] );
					}
				}

				{
					// Simple
					float L, U, V, W;
					GetLUVW( SourceSample.Coefficients[2], L, U, V, W );

					float LogL = FMath::Log2( L + SimpleLogBlackPoint ) / SimpleLogScale + 0.5f;

					float LogR = LogL * U;
					float LogG = LogL * V;
					float LogB = LogL * W;

					MinCoefficient[2][0] = FMath::Min( MinCoefficient[2][0], LogR );
					MaxCoefficient[2][0] = FMath::Max( MaxCoefficient[2][0], LogR );

					MinCoefficient[2][1] = FMath::Min( MinCoefficient[2][1], LogG );
					MaxCoefficient[2][1] = FMath::Max( MaxCoefficient[2][1], LogG );
				
					MinCoefficient[2][2] = FMath::Min( MinCoefficient[2][2], LogB );
					MaxCoefficient[2][2] = FMath::Max( MaxCoefficient[2][2], LogB );

					// Dampen dark texel's contribution on the directionality min and max
					float DampenDirectionality = FMath::Clamp(L * 100, 0.0f, 1.0f);

					for( int32 ColorIndex = 0; ColorIndex < 3; ColorIndex++ )
					{
						MinCoefficient[3][ColorIndex] = FMath::Min( MinCoefficient[3][ColorIndex], DampenDirectionality * SourceSample.Coefficients[3][ColorIndex] );
						MaxCoefficient[3][ColorIndex] = FMath::Max( MaxCoefficient[3][ColorIndex], DampenDirectionality * SourceSample.Coefficients[3][ColorIndex] );
					}
				}
			}
		}

		// If no sample mapped make range sane.
		// Or if very dark no directionality is added. Make range sane.
		for (int32 CoefficientIndex = 0; CoefficientIndex < LM_NUM_STORED_LIGHTMAP_COEF; CoefficientIndex++)
		{
			for( int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++ )
			{
				if( MinCoefficient[CoefficientIndex][ColorIndex] > MaxCoefficient[CoefficientIndex][ColorIndex] )
				{
					MinCoefficient[CoefficientIndex][ColorIndex] = 0.0f;
					MaxCoefficient[CoefficientIndex][ColorIndex] = 0.0f;
				}
			}
		}

		// Calculate the scale/bias for the light-map coefficients.
		float CoefficientMultiply[LM_NUM_STORED_LIGHTMAP_COEF][4];
		float CoefficientAdd[LM_NUM_STORED_LIGHTMAP_COEF][4];

		for (int32 CoefficientIndex = 0; CoefficientIndex < LM_NUM_STORED_LIGHTMAP_COEF; CoefficientIndex++)
		{
			for (int32 ColorIndex = 0; ColorIndex < 4; ColorIndex++)
			{
				// Calculate scale and bias factors to pack into the desired range
				// y = (x - Min) / (Max - Min)
				// Mul = 1 / (Max - Min)
				// Add = -Min / (Max - Min)
				CoefficientMultiply[CoefficientIndex][ColorIndex] = 1.0f / FMath::Max<float>(MaxCoefficient[CoefficientIndex][ColorIndex] - MinCoefficient[CoefficientIndex][ColorIndex], DELTA);
				CoefficientAdd[CoefficientIndex][ColorIndex] = -MinCoefficient[CoefficientIndex][ColorIndex] / FMath::Max<float>(MaxCoefficient[CoefficientIndex][ColorIndex] - MinCoefficient[CoefficientIndex][ColorIndex], DELTA);

				// Output the values used to undo this packing
				OutMultiply[CoefficientIndex][ColorIndex] = 1.0f / CoefficientMultiply[CoefficientIndex][ColorIndex];
				OutAdd[CoefficientIndex][ColorIndex] = -CoefficientAdd[CoefficientIndex][ColorIndex] / CoefficientMultiply[CoefficientIndex][ColorIndex];
			}
		}

		// Bias to avoid divide by zero in shader
		for (int32 ColorIndex = 0; ColorIndex < 3; ColorIndex++)
		{
			OutAdd[2][ColorIndex] = FMath::Max( OutAdd[2][ColorIndex], 1e-2f );
		}

		// Force SH constant term to 0.282095f. Avoids add in shader.
		OutMultiply[1][3] = 0.0f;
		OutAdd[1][3] = 0.282095f;
		OutMultiply[3][3] = 0.0f;
		OutAdd[3][3] = 0.282095f;

		// allocate space in the output
		OutLightSamples.Empty(InLightSamples.Num());
		OutLightSamples.AddUninitialized(InLightSamples.Num());

		// quantize each sample using the above scaling
		for (int32 SampleIndex = 0; SampleIndex < InLightSamples.Num(); SampleIndex++)
		{
			const FLightSample& SourceSample = InLightSamples[SampleIndex];
			FQuantizedLightSampleData& DestCoefficients = OutLightSamples[SampleIndex];
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (SampleIndex == DebugSampleIndex)
			{
				int32 TempBreak = 0;
			}
#endif
			DestCoefficients.Coverage = SourceSample.bIsMapped ? 255 : 0;

			const FVector3f BentNormal(SourceSample.SkyOcclusion[0], SourceSample.SkyOcclusion[1], SourceSample.SkyOcclusion[2]);
			const float BentNormalLength = BentNormal.Size();
			const FVector3f NormalizedBentNormal = BentNormal.GetSafeNormal() * FVector3f(.5f) + FVector3f(.5f);

			DestCoefficients.SkyOcclusion[0] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( NormalizedBentNormal[0] * 255.0f ), 0, 255 );
			DestCoefficients.SkyOcclusion[1] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( NormalizedBentNormal[1] * 255.0f ), 0, 255 );
			DestCoefficients.SkyOcclusion[2] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( NormalizedBentNormal[2] * 255.0f ), 0, 255 );
			// Sqrt on length to allocate more precision near 0
			DestCoefficients.SkyOcclusion[3] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( FMath::Sqrt(BentNormalLength) * 255.0f ), 0, 255 );

			// Sqrt to allocate more precision near 0
			DestCoefficients.AOMaterialMask = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( FMath::Sqrt(SourceSample.AOMaterialMask) * 255.0f ), 0, 255 ); 

			{
				float L, U, V, W;
				GetLUVW( SourceSample.Coefficients[0], L, U, V, W );

				// LogLUVW encode color
				float LogL = FMath::Log2( L + LogBlackPoint );

				U = U * CoefficientMultiply[0][0] + CoefficientAdd[0][0];
				V = V * CoefficientMultiply[0][1] + CoefficientAdd[0][1];
				W = W * CoefficientMultiply[0][2] + CoefficientAdd[0][2];
				LogL = LogL * CoefficientMultiply[0][3] + CoefficientAdd[0][3];

				float Residual = LogL * 255.0f - FMath::RoundToFloat( LogL * 255.0f ) + 0.5f;

				// U, V, W, LogL
				// UVW stored in gamma space
				DestCoefficients.Coefficients[0][0] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( FMath::Sqrt(U) * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[0][1] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( FMath::Sqrt(V) * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[0][2] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( FMath::Sqrt(W) * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[0][3] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( LogL * 255.0f ), 0, 255 );
				
				float Dx = SourceSample.Coefficients[1][0] * CoefficientMultiply[1][0] + CoefficientAdd[1][0];
				float Dy = SourceSample.Coefficients[1][1] * CoefficientMultiply[1][1] + CoefficientAdd[1][1];
				float Dz = SourceSample.Coefficients[1][2] * CoefficientMultiply[1][2] + CoefficientAdd[1][2];

				// Dx, Dy, Dz, Residual
				DestCoefficients.Coefficients[1][0] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( Dx * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[1][1] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( Dy * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[1][2] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( Dz * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[1][3] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( Residual * 255.0f ), 0, 255 );
			}

			{
				float L, U, V, W;
				GetLUVW( SourceSample.Coefficients[2], L, U, V, W );

				// LogRGB encode color
				float LogL = FMath::Log2( L + SimpleLogBlackPoint ) / SimpleLogScale + 0.5f;

				float LogR = LogL * U * CoefficientMultiply[2][0] + CoefficientAdd[2][0];
				float LogG = LogL * V * CoefficientMultiply[2][1] + CoefficientAdd[2][1];
				float LogB = LogL * W * CoefficientMultiply[2][2] + CoefficientAdd[2][2];
				
				// LogR, LogG, LogB
				DestCoefficients.Coefficients[2][0] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( LogR * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[2][1] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( LogG * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[2][2] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( LogB * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[2][3] = 255;

				float Dx = SourceSample.Coefficients[3][0] * CoefficientMultiply[3][0] + CoefficientAdd[3][0];
				float Dy = SourceSample.Coefficients[3][1] * CoefficientMultiply[3][1] + CoefficientAdd[3][1];
				float Dz = SourceSample.Coefficients[3][2] * CoefficientMultiply[3][2] + CoefficientAdd[3][2];

				// Dx, Dy, Dz
				DestCoefficients.Coefficients[3][0] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( Dx * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[3][1] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( Dy * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[3][2] = (uint8)FMath::Clamp<int32>( FMath::RoundToInt( Dz * 255.0f ), 0, 255 );
				DestCoefficients.Coefficients[3][3] = 255;
			}
		}

		InLightSamples.Empty();
	}

	/**
	 * Quantize the full-res FLightSamples into FQuantizedLightSampleDatas
	 */
	void FLightMapData2D::Quantize(int32 DebugSampleIndex)
	{
		QuantizeLightSamples(Data, QuantizedData, Multiply, Add, DebugSampleIndex, true);
	}

	void FShadowMapData2D::Quantize(int32 DebugSampleIndex)
	{
		// Allocate space in the output array
		QuantizedData.Empty(Data.Num());
		QuantizedData.AddUninitialized(Data.Num());

		for (int32 SampleIndex = 0; SampleIndex < Data.Num(); SampleIndex++)
		{
			const FShadowSample& Value = Data[SampleIndex];
			FQuantizedShadowSampleData& QuantizedValue = QuantizedData[SampleIndex];

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (SampleIndex == DebugSampleIndex)
			{
				int32 TempBreak = 0;
			}
#endif
			// Convert linear input values to gamma space before quantizing, which preserves more detail in the darks where banding would be noticeable otherwise
			QuantizedValue.Visibility = (uint8)FMath::Clamp<int32>(FMath::TruncToInt(FMath::Pow(Value.Visibility, 1.0f / 2.2f) * 255.0f), 0, 255);
			QuantizedValue.Coverage = Value.bIsMapped ? 255 : 0;
		}
		Data.Empty();
	}

	void FShadowMapData2D::Compress(int32 DebugSampleIndex)
	{
		// Make sure the data has been quantized already
		Quantize(DebugSampleIndex);

		// Calculate the uncompressed size
		UncompressedDataSize = sizeof(FQuantizedShadowSampleData) * QuantizedData.Num();

		// Compress the array
		CompressData((uint8*)QuantizedData.GetData(), UncompressedDataSize, CompressedData, CompressedDataSize);

		// Discard the source data now that we're compressed
		QuantizedData.Empty();
	}

	void FSignedDistanceFieldShadowMapData2D::Quantize(int32 DebugSampleIndex)
	{
		// Allocate space in the output array
		QuantizedData.Empty(Data.Num());
		QuantizedData.AddUninitialized(Data.Num());

		for (int32 SampleIndex = 0; SampleIndex < Data.Num(); SampleIndex++)
		{
			const FSignedDistanceFieldShadowSample& Value = Data[SampleIndex];
			FQuantizedSignedDistanceFieldShadowSampleData& QuantizedValue = QuantizedData[SampleIndex];

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (SampleIndex == DebugSampleIndex)
			{
				int32 TempBreak = 0;
			}
#endif
			QuantizedValue.Distance = (uint8)FMath::Clamp<int32>(FMath::TruncToInt(Value.Distance * 255.0f), 0, 255);
			QuantizedValue.PenumbraSize = (uint8)FMath::Clamp<int32>(FMath::TruncToInt(Value.PenumbraSize * 255.0f), 0, 255);
			QuantizedValue.Coverage = Value.bIsMapped ? 255 : 0;
		}
		Data.Empty();
	}

	void FSignedDistanceFieldShadowMapData2D::Compress(int32 DebugSampleIndex)
	{
		// Make sure the data has been quantized already
		Quantize(DebugSampleIndex);

		// Calculate the uncompressed size
		UncompressedDataSize = sizeof(FQuantizedSignedDistanceFieldShadowSampleData) * QuantizedData.Num();

		// Compress the array
		CompressData((uint8*)QuantizedData.GetData(), UncompressedDataSize, CompressedData, CompressedDataSize);

		// Discard the source data now that we're compressed
		QuantizedData.Empty();
	}

} //namespace Lightmass
