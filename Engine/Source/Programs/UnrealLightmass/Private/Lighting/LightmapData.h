// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ImportExport.h"
#include "LightmassScene.h"

namespace Lightmass
{
	/**
	 * The raw data which is used to construct a 2D light-map.
	 */
	class FLightMapData2D : public FLightMapData2DData
	{
	public:

		/** The lights which this light-map stores. */
		TArray<const FLight*> Lights;

		FLightMapData2D(uint32 InSizeX,uint32 InSizeY)
			: FLightMapData2DData(InSizeX, InSizeY)
			, CompressedData(NULL)
		{
			Data.Empty(SizeX * SizeY);
			Data.AddZeroed(SizeX * SizeY);
		}

		~FLightMapData2D()
		{
			delete CompressedData;
		}

		// Accessors.
		const FLightSample& operator()(uint32 X,uint32 Y) const { return Data[SizeX * Y + X]; }
		FLightSample& operator()(uint32 X,uint32 Y) { return Data[SizeX * Y + X]; }
		uint32 GetSizeX() const { return SizeX; }
		uint32 GetSizeY() const { return SizeY; }

		void AddLight(const FLight* NewLight)
		{
			Lights.AddUnique(NewLight);
		}

		/**
		* Quantize the full-res FLightSamples into FQuantizedLightSampleDatas
		*/
		void Quantize(int32 DebugSampleIndex);

		const FLightSample* GetData() { return Data.GetData(); }
		const FQuantizedLightSampleData* GetQuantizedData() { return QuantizedData.GetData(); }

		/**
		 * Compresses the raw lightmap data to a buffer for writing over Swarm
		 */
		void Compress(int32 DebugSampleIndex);

		/**
		 * @return the compressed data, or NULL if not compressed 
		 */
		uint8* GetCompressedData()
		{
			return CompressedData;
		}

	private:
		TArray<FLightSample>				Data;
		TArray<FQuantizedLightSampleData>	QuantizedData;

		/** zlib compressed lightmap data */
		uint8*								CompressedData;
	};

	/**
	 * A sample of the visibility factor between a light and a single point.
	 */
	class FShadowSample : public FShadowSampleData
	{
	public:
		FShadowSample operator-(const FShadowSample& SampleB) const
		{
			FShadowSample Result;
			Result.bIsMapped = bIsMapped;
			Result.Visibility = Visibility - SampleB.Visibility;
			return Result;
		}
		FShadowSample operator*(const float& Scalar) const
		{
			FShadowSample Result;
			Result.bIsMapped = bIsMapped;
			Result.Visibility = Visibility * Scalar;
			return Result;
		}
	};

	/**
	 * The raw data which is used to construct a 2D light-map.
	 */
	class FShadowMapData2D : public FShadowMapData2DData
	{
	public:
		FShadowMapData2D(uint32 InSizeX,uint32 InSizeY)
			: FShadowMapData2DData(InSizeX, InSizeY)
			, CompressedData(NULL)
		{
			Data.Empty(InSizeX * InSizeY);
			Data.AddZeroed(InSizeX * InSizeY);
		}

		virtual ~FShadowMapData2D(){ }

		// Accessors.
		const FShadowSample& operator()(uint32 X,uint32 Y) const { return Data[SizeX * Y + X]; }
		FShadowSample& operator()(uint32 X,uint32 Y) { return Data[SizeX * Y + X]; }
		uint32 GetSizeX() const { return SizeX; }
		uint32 GetSizeY() const { return SizeY; }

		void Quantize(int32 DebugSampleIndex);

		const FShadowSample* GetData() { return Data.GetData(); }
		const FQuantizedShadowSampleData* GetQuantizedData() { return QuantizedData.GetData(); }

		void Compress(int32 DebugSampleIndex);
		uint8* GetCompressedData()
		{
			return CompressedData;
		}

	private:
		TArray<FShadowSample>				Data;
		TArray<FQuantizedShadowSampleData>	QuantizedData;
		uint8*								CompressedData;
	};

	class FSignedDistanceFieldShadowSample : public FSignedDistanceFieldShadowSampleData
	{
	public:
		FSignedDistanceFieldShadowSample operator-(const FSignedDistanceFieldShadowSample& SampleB) const
		{
			FSignedDistanceFieldShadowSample Result;
			Result.bIsMapped = bIsMapped;
			Result.Distance = Distance - SampleB.Distance;
			Result.PenumbraSize = PenumbraSize - SampleB.PenumbraSize;
			return Result;
		}
		FSignedDistanceFieldShadowSample operator*(const float& Scalar) const
		{
			FSignedDistanceFieldShadowSample Result;
			Result.bIsMapped = bIsMapped;
			Result.Distance = Distance * Scalar;
			Result.PenumbraSize = PenumbraSize * Scalar;
			return Result;
		}
	};

	/**
	 * The raw data which is used to construct a 2D signed distance field shadow map.
	 */
	class FSignedDistanceFieldShadowMapData2D : public FSignedDistanceFieldShadowMapData2DData
	{
	public:
		FSignedDistanceFieldShadowMapData2D(uint32 InSizeX,uint32 InSizeY)
			: FSignedDistanceFieldShadowMapData2DData(InSizeX, InSizeY)
			, CompressedData(NULL)
		{
			Data.Empty(InSizeX * InSizeY);
			Data.AddZeroed(InSizeX * InSizeY);
		}

		virtual ~FSignedDistanceFieldShadowMapData2D(){ }

		// Accessors.
		const FSignedDistanceFieldShadowSample& operator()(uint32 X,uint32 Y) const { return Data[SizeX * Y + X]; }
		FSignedDistanceFieldShadowSample& operator()(uint32 X,uint32 Y) { return Data[SizeX * Y + X]; }
		uint32 GetSizeX() const { return SizeX; }
		uint32 GetSizeY() const { return SizeY; }

		void Quantize(int32 DebugSampleIndex);

		const FSignedDistanceFieldShadowSample* GetData() { return Data.GetData(); }
		const FQuantizedSignedDistanceFieldShadowSampleData* GetQuantizedData() { return QuantizedData.GetData(); }

		void Compress(int32 DebugSampleIndex);
		uint8* GetCompressedData()
		{
			return CompressedData;
		}

	private:
		TArray<FSignedDistanceFieldShadowSample>				Data;
		TArray<FQuantizedSignedDistanceFieldShadowSampleData>	QuantizedData;
		uint8*													CompressedData;
	};

} //namespace Lightmass
