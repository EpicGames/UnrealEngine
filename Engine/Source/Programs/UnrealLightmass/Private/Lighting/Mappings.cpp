// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mappings.h"
#include "Importer.h"
#include "StaticMesh.h"

namespace Lightmass
{
/** A static indicating that debug borders should be used around padded mappings. */
bool FStaticLightingMapping::s_bShowLightmapBorders = false;

void FStaticLightingMapping::Import( class FLightmassImporter& Importer )
{
	Importer.ImportData( (FStaticLightingMappingData*) this );
	Mesh = Importer.GetStaticMeshInstances().FindRef(Guid);
}

void FStaticLightingTextureMapping::Import( class FLightmassImporter& Importer )
{
	FStaticLightingMapping::Import( Importer );
	Importer.ImportData( (FStaticLightingTextureMappingData*) this );
	CachedSizeX = SizeX;
	CachedSizeY = SizeY;
	SurfaceCacheSizeX = 0;
	SurfaceCacheSizeY = 0;
}

} //namespace Lightmass
