// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/LockFreeList.h"
#include "Templates/RefCounting.h"
#include "SceneExport.h"
#include "LightingCache.h"
#include "GatheredLightingSample.h"

namespace Lightmass
{

class FCacheIndirectTaskDescription;
class FInterpolateIndirectTaskDescription;

enum EHemisphereGatherClassification
{
	GLM_None = 0,
	GLM_GatherRadiosityBuffer0 = 1,
	GLM_GatherRadiosityBuffer1 = 2,
	GLM_GatherLightEmitted = 4,
	GLM_GatherLightFinalBounced = 8,
	GLM_FinalGather = GLM_GatherLightEmitted | GLM_GatherLightFinalBounced
};

class FCompressedGatherHitPoints
{
public:

	FCompressedGatherHitPoints() :
		GatherHitPointRangesUncompressedSize(0),
		GatherHitPointDataUncompressedSize(0)
	{}

	uint32 GatherHitPointRangesUncompressedSize;
	TArray<uint8> GatherHitPointRanges;

	uint32 GatherHitPointDataUncompressedSize;
	TArray<uint8> GatherHitPointData;

	void Compress(const FGatherHitPoints& Source);
	void Decompress(FGatherHitPoints& Dest) const;

	size_t GetAllocatedSize() const { return GatherHitPointRanges.GetAllocatedSize() + GatherHitPointData.GetAllocatedSize(); }
};

class FGatherHitPoints
{
public:

	TArray<FArrayRange> GatherHitPointRanges;
	TArray<FFinalGatherHitPoint> GatherHitPointData;

	size_t GetAllocatedSize() const { return GatherHitPointRanges.GetAllocatedSize() + GatherHitPointData.GetAllocatedSize(); }
};

class FCompressedInfluencingRecords
{
public:

	FCompressedInfluencingRecords() :
		RangesUncompressedSize(0),
		DataUncompressedSize(0)
	{}

	uint32 RangesUncompressedSize;
	TArray<uint8> Ranges;

	uint32 DataUncompressedSize;
	TArray<uint8> Data;

	void Compress(const FInfluencingRecords& Source);
	void Decompress(FInfluencingRecords& Dest) const;

	size_t GetAllocatedSize() const { return Ranges.GetAllocatedSize() + Data.GetAllocatedSize(); }
};

/** A mapping between world-space surfaces and a static lighting cache. */
class FStaticLightingMapping : public virtual FRefCountedObject, public FStaticLightingMappingData
{
public:

	/** The mesh associated with the mapping, guaranteed to be valid (non-NULL) after import. */
	class FStaticLightingMesh* Mesh;

	/** Whether the mapping has been processed. */
	volatile int32 bProcessed;

	/** If true, the mapping is being padded */
	bool bPadded;

	/** A static indicating that debug borders should be used around padded mappings. */
	static bool s_bShowLightmapBorders;

	/** Index of this mapping in FStaticLightingSystem::AllMappings */
	int32 SceneMappingIndex;

protected:

    /** The irradiance photons which are cached on this mapping */
	TArray<const class FIrradiancePhoton*> CachedIrradiancePhotons;

	/** Approximate lighting cached on this mapping, used by final gather rays. */
	TArray<FLinearColor> SurfaceCacheLighting;

	TArray<FLinearColor> RadiositySurfaceCache[2];

	/** Indexed by texel coordinate */
	FCompressedInfluencingRecords CompressedInfluencingRecords;
	FInfluencingRecords InfluencingRecordsSurfaceCache;

	FCompressedGatherHitPoints CompressedGatherHitPoints;
	FGatherHitPoints UncompressedGatherHitPoints;

public:

 	/** Initialization constructor. */
	FStaticLightingMapping() :
		  bProcessed(false)
		, bPadded(false)
		, SceneMappingIndex(-1)
	{
	}

	/** Virtual destructor. */
	virtual ~FStaticLightingMapping() 
	{
	}

	/** @return If the mapping is a texture mapping, returns a pointer to this mapping as a texture mapping.  Otherwise, returns NULL. */
	virtual class FStaticLightingTextureMapping* GetTextureMapping() 
	{
		return NULL;
	}

	virtual const FStaticLightingTextureMapping* GetTextureMapping() const
	{
		return NULL;
	}

	virtual class FStaticLightingGlobalVolumeMapping* GetVolumeMapping() 
	{
		return NULL;
	}

	virtual const FStaticLightingGlobalVolumeMapping* GetVolumeMapping() const
	{
		return NULL;
	}

	virtual const class FLandscapeStaticLightingGlobalVolumeMapping* GetLandscapeVolumeMapping()
	{
		return NULL;
	}

	virtual const class FLandscapeStaticLightingGlobalVolumeMapping* GetLandscapeVolumeMapping() const
	{
		return NULL;
	}

	/**
	 * Returns the relative processing cost used to sort tasks from slowest to fastest.
	 *
	 * @return	relative processing cost or 0 if unknown
	 */
	virtual float GetProcessingCost() const
	{
		return 0;
	}

	virtual FLinearColor GetSurfaceCacheLighting(const FMinimalStaticLightingVertex& Vertex) const = 0;

	virtual int32 GetSurfaceCacheIndex(const struct FMinimalStaticLightingVertex& Vertex) const = 0;
	FLinearColor GetCachedRadiosity(int32 RadiosityBufferIndex, int32 SurfaceCacheIndex) const;
	size_t FreeRadiosityTemporaries();

	uint32 GetIrradiancePhotonCacheBytes() const { return SurfaceCacheLighting.GetAllocatedSize(); }

	virtual void Import( class FLightmassImporter& Importer );

	virtual void Initialize(FStaticLightingSystem& System) = 0;

	friend class FStaticLightingSystem;
};

/** A mapping between world-space surfaces and static lighting cache textures. */
class FStaticLightingTextureMapping : public FStaticLightingMapping, public FStaticLightingTextureMappingData
{
public:

	FStaticLightingTextureMapping() : 
		NumOutstandingCacheTasks(0),
		NumOutstandingInterpolationTasks(0)
	{}

	// FStaticLightingMapping interface.
	virtual FStaticLightingTextureMapping* GetTextureMapping() override
	{
		return this;
	}

	virtual const FStaticLightingTextureMapping* GetTextureMapping() const override
	{
		return this;
	}

	/**
	 * Returns the relative processing cost used to sort tasks from slowest to fastest.
	 *
	 * @return	relative processing cost or 0 if unknown
	 */
	virtual float GetProcessingCost() const
	{
		return SizeX * SizeY;
	}

	virtual FLinearColor GetSurfaceCacheLighting(const FMinimalStaticLightingVertex& Vertex) const override;

	virtual int32 GetSurfaceCacheIndex(const struct FMinimalStaticLightingVertex& Vertex) const;

	virtual void Import( class FLightmassImporter& Importer );

	virtual void Initialize(FStaticLightingSystem& System);

	/** The padded size of the mapping */
	int32 CachedSizeX;
	int32 CachedSizeY;

	/** The sizes that CachedIrradiancePhotons were stored with */
	int32 SurfaceCacheSizeX;
	int32 SurfaceCacheSizeY;

	/** Counts how many cache tasks this mapping needs completed. */
	volatile int32 NumOutstandingCacheTasks;

	/** List of completed cache tasks for this mapping. */
	TLockFreePointerListLIFO<FCacheIndirectTaskDescription> CompletedCacheIndirectLightingTasks;

	/** Counts how many interpolation tasks this mapping needs completed. */
	volatile int32 NumOutstandingInterpolationTasks;

	/** List of completed interpolation tasks for this mapping. */
	TLockFreePointerListLIFO<FInterpolateIndirectTaskDescription> CompletedInterpolationTasks;
};

/** 
 * A mapping that represents an object which is going to use the global volumetric lightmap.
 * Hack: currently represented as a texture lightmap to Lightmass for the purposes of surface caching of light (radiosity + direct lighting).
 */
class FStaticLightingGlobalVolumeMapping : public FStaticLightingTextureMapping
{
public:

	FStaticLightingGlobalVolumeMapping() 
	{}

	virtual class FStaticLightingGlobalVolumeMapping* GetVolumeMapping() override
	{
		return this;
	}

	virtual const FStaticLightingGlobalVolumeMapping* GetVolumeMapping() const override
	{
		return this;
	}
};
 
} //namespace Lightmass
