// Copyright Epic Games, Inc. All Rights Reserved.

#include "MonteCarlo.h"

namespace Lightmass
{

/** Generates valid X and Y axes of a coordinate system, given the Z axis. */
void GenerateCoordinateSystem(const FVector4f& ZAxis, FVector4f& XAxis, FVector4f& YAxis)
{
	// Use the vector perpendicular to ZAxis and the Y axis as the XAxis
	const FVector4f XAxisCandidate = ZAxis ^ FVector4f(0,1,0);
	if (XAxisCandidate.SizeSquared3() < KINDA_SMALL_NUMBER)
	{
		// The vector was nearly equal to the Y axis, use the X axis instead
		XAxis = (ZAxis ^ FVector4f(1,0,0)).GetUnsafeNormal3();
	}
	else
	{
		XAxis = XAxisCandidate.GetUnsafeNormal3();
	}

	YAxis = ZAxis ^ XAxis;
	checkSlow(YAxis.IsUnit3());
}

/** Generates valid X and Y axes of a coordinate system, given the Z axis. */
void GenerateCoordinateSystem2(const FVector4f& ZAxis, FVector4f& XAxis, FVector4f& YAxis)
{
	// This implementation is based off of the one from 'Physically Based Rendering'
	if (FMath::Abs(ZAxis.X) > FMath::Abs(ZAxis.Y))
 	{
		const float InverseLength = FMath::InvSqrt(ZAxis.X * ZAxis.X + ZAxis.Z * ZAxis.Z);
		XAxis = FVector4f(-ZAxis.Z * InverseLength, 0.0f, ZAxis.X * InverseLength);
 	}
 	else
 	{
		const float InverseLength = FMath::InvSqrt(ZAxis.Y * ZAxis.Y + ZAxis.Z * ZAxis.Z);
		XAxis = FVector4f(0.0f, ZAxis.Z * InverseLength, -ZAxis.Y * InverseLength);
 	}

	YAxis = ZAxis ^ XAxis;
 	checkSlow(YAxis.IsUnit3());
	checkSlow(FMath::Abs(Dot3(XAxis, ZAxis)) <= THRESH_NORMALS_ARE_ORTHOGONAL);
	checkSlow(FMath::Abs(Dot3(YAxis, ZAxis)) <= THRESH_NORMALS_ARE_ORTHOGONAL);
	checkSlow(FMath::Abs(Dot3(XAxis, YAxis)) <= THRESH_NORMALS_ARE_ORTHOGONAL);
}

/** Generates a pseudo-random unit vector, uniformly distributed over all directions. */
FVector4f GetUnitVector(FLMRandomStream& RandomStream)
{
	return GetUnitPosition(RandomStream).GetUnsafeNormal3();
}

/** Generates a pseudo-random position inside the unit sphere, uniformly distributed over the volume of the sphere. */
FVector4f GetUnitPosition(FLMRandomStream& RandomStream)
{
	FVector4f Result;
	// Use rejection sampling to generate a valid sample
	do
	{
		Result.X = RandomStream.GetFraction() * 2 - 1;
		Result.Y = RandomStream.GetFraction() * 2 - 1;
		Result.Z = RandomStream.GetFraction() * 2 - 1;
	} while( Result.SizeSquared3() > 1.f );
	return Result;
}

/** 
 * Generates a pseudo-random unit vector in the Z > 0 hemisphere whose PDF == 1 / (2 * PI) in solid angles,
 * Or sin(theta) / (2 * PI) in hemispherical coordinates, which is a uniform distribution over the area of the hemisphere.
 */
FVector4f GetUniformHemisphereVector(FLMRandomStream& RandomStream, float MaxTheta)
{
	const float Theta = FMath::Min(FMath::Acos(RandomStream.GetFraction()), MaxTheta - DELTA);
	const float Phi = 2.0f * (float)PI * RandomStream.GetFraction();
	checkSlow(Theta >= 0 && Theta <= (float)HALF_PI);
	checkSlow(Phi >= 0 && Phi <= 2.0f * (float)PI);
	const float SinTheta = FMath::Sin(Theta);
	// Convert to Cartesian
	return FVector4f(FMath::Cos(Phi) * SinTheta, FMath::Sin(Phi) * SinTheta, FMath::Cos(Theta));
}

/** 
 * Generates a pseudo-random unit vector in the Z > 0 hemisphere whose PDF == cos(theta) / PI in solid angles,
 * Which is sin(theta)cos(theta) / PI in hemispherical coordinates.
 */
FVector4f GetCosineHemisphereVector(FLMRandomStream& RandomStream, float MaxTheta)
{
	const float Theta = FMath::Min(FMath::Acos(FMath::Sqrt(RandomStream.GetFraction())), MaxTheta - DELTA);
	const float Phi = 2.0f * (float)PI * RandomStream.GetFraction();
	checkSlow(Theta >= 0 && Theta <= (float)HALF_PI);
	checkSlow(Phi >= 0 && Phi <= 2.0f * (float)PI);
	const float SinTheta = FMath::Sin(Theta);
	// Convert to Cartesian
	return FVector4f(FMath::Cos(Phi) * SinTheta, FMath::Sin(Phi) * SinTheta, FMath::Cos(Theta));
}

/** 
 * Generates a pseudo-random unit vector in the Z > 0 hemisphere,
 * Whose PDF == (SpecularPower + 1) / (2.0f * PI) * cos(Alpha) ^ SpecularPower in solid angles,
 * Where Alpha is the angle between the perfect specular direction and the outgoing direction.
 */
FVector4f GetModifiedPhongSpecularVector(FLMRandomStream& RandomStream, const FVector4f& TangentSpecularDirection, float SpecularPower)
{
	checkSlow(TangentSpecularDirection.Z >= 0.0f);
	checkSlow(SpecularPower > 0.0f);

	FVector4f GeneratedTangentVector;
	do
	{
		// Generate hemispherical coordinates in the local frame of the perfect specular direction
		// Don't allow a value of 0, since that results in a PDF of 0 with large specular powers due to floating point imprecision
		const float Alpha = FMath::Min(FMath::Acos(FMath::Pow(FMath::Max(RandomStream.GetFraction(), DELTA), 1.0f / (SpecularPower + 1.0f))), (float)HALF_PI - DELTA);
		const float Phi = 2.0f * (float)PI * RandomStream.GetFraction();
		
		// Convert to Cartesian, still in the coordinate space of the perfect specular direction
		const float SinTheta = FMath::Sin(Alpha);
		const FVector4f GeneratedSpecularTangentVector(FMath::Cos(Phi) * SinTheta, FMath::Sin(Phi) * SinTheta, FMath::Cos(Alpha));

		// Generate the X and Y axes of the coordinate space whose Z is the perfect specular direction
		FVector4f SpecularTangentX = (TangentSpecularDirection ^ FVector4f(0,1,0)).GetUnsafeNormal3();
		if (SpecularTangentX.SizeSquared3() < KINDA_SMALL_NUMBER)
		{
			// The specular direction was nearly equal to the Y axis, use the X axis instead
			SpecularTangentX = (TangentSpecularDirection ^ FVector4f(1,0,0)).GetUnsafeNormal3();
		}
		else
		{
			SpecularTangentX = SpecularTangentX.GetUnsafeNormal3();
		}
		const FVector4f SpecularTangentY = TangentSpecularDirection ^ SpecularTangentX;

		// Rotate the generated coordinates into the local frame of the tangent space normal (0,0,1)
		const FVector4f SpecularTangentRow0(SpecularTangentX.X, SpecularTangentY.X, TangentSpecularDirection.X);
		const FVector4f SpecularTangentRow1(SpecularTangentX.Y, SpecularTangentY.Y, TangentSpecularDirection.Y);
		const FVector4f SpecularTangentRow2(SpecularTangentX.Z, SpecularTangentY.Z, TangentSpecularDirection.Z);
		GeneratedTangentVector = FVector4f(
			Dot3(SpecularTangentRow0, GeneratedSpecularTangentVector),
			Dot3(SpecularTangentRow1, GeneratedSpecularTangentVector),
			Dot3(SpecularTangentRow2, GeneratedSpecularTangentVector)
			);
	}
	// Regenerate an Alpha as long as the direction is outside of the tangent space Z > 0 hemisphere, 
	// Since some part of the cosine lobe around the specular direction can be outside of the hemisphere around the surface normal.
	while (GeneratedTangentVector.Z < DELTA);
	return GeneratedTangentVector;
}

/** 
 * Generates a pseudo-random position within a unit disk,
 * Whose PDF == 1 / PI, which is a uniform distribution over the area of the disk.
 */
FVector2f GetUniformUnitDiskPosition(FLMRandomStream& RandomStream)
{
	const float Theta = 2.0f * (float)PI * RandomStream.GetFraction();
	const float Radius = FMath::Sqrt(RandomStream.GetFraction());
	return FVector2f(Radius * FMath::Cos(Theta), Radius * FMath::Sin(Theta));
}

/** 
 * Generates a pseudo-random direction within a cone,
 * Whose PDF == 1 / (2 * PI * (1 - CosMaxConeTheta)), which is a uniform distribution over the directions in the cone. 
 */
FVector4f UniformSampleCone(FLMRandomStream& RandomStream, float CosMaxConeTheta, const FVector4f& XAxis, const FVector4f& YAxis, const FVector4f& ZAxis)
{
	checkSlow(CosMaxConeTheta >= 0.0f && CosMaxConeTheta <= 1.0f);
	const float CosTheta = FMath::Lerp(CosMaxConeTheta, 1.0f, RandomStream.GetFraction());
	const float SinTheta = FMath::Sqrt(1.0f - CosTheta * CosTheta);
	const float Phi = RandomStream.GetFraction() * 2.0f * (float)PI;
	return FMath::Cos(Phi) * SinTheta * XAxis + FMath::Sin(Phi) * SinTheta * YAxis + CosTheta * ZAxis;
}

FVector4f UniformSampleCone(float CosMaxConeTheta, const FVector4f& XAxis, const FVector4f& YAxis, const FVector4f& ZAxis, float Uniform1, float Uniform2)
{
	checkSlow(CosMaxConeTheta >= 0.0f && CosMaxConeTheta <= 1.0f);
	const float CosTheta = FMath::Lerp(CosMaxConeTheta, 1.0f, Uniform1);
	const float SinTheta = FMath::Sqrt(1.0f - CosTheta * CosTheta);
	const float Phi = Uniform2 * 2.0f * (float)PI;
	return FMath::Cos(Phi) * SinTheta * XAxis + FMath::Sin(Phi) * SinTheta * YAxis + CosTheta * ZAxis;
}

/** Calculates the PDF for a sample generated by UniformSampleCone */
float UniformConePDF(float CosMaxConeTheta)
{
	checkSlow(CosMaxConeTheta >= 0.0f && CosMaxConeTheta <= 1.0f);
	return 1.0f / (2.0f * (float)PI * (1.0f - CosMaxConeTheta));
}

FVector4f UniformSampleHemisphere(float Uniform1, float Uniform2)
{
	const float R = FMath::Sqrt(1.0f - Uniform1 * Uniform1);
	const float Phi = 2.0f * (float)PI * Uniform2;

	// Convert to Cartesian
	return FVector4f(FMath::Cos(Phi) * R, FMath::Sin(Phi) * R, Uniform1);
}

/** Generates unit length, stratified and uniformly distributed direction samples in a hemisphere. */
void GenerateStratifiedUniformHemisphereSamples(int32 NumThetaSteps, int32 NumPhiSteps, FLMRandomStream& RandomStream, TArray<FVector4f>& Samples, TArray<FVector2f>& Uniforms)
{
	Samples.Empty(NumThetaSteps * NumPhiSteps);
	for (int32 ThetaIndex = 0; ThetaIndex < NumThetaSteps; ThetaIndex++)
	{
		for (int32 PhiIndex = 0; PhiIndex < NumPhiSteps; PhiIndex++)
		{
			const float U1 = RandomStream.GetFraction();
			const float U2 = RandomStream.GetFraction();

			const float Fraction1 = (ThetaIndex + U1) / (float)NumThetaSteps;
			const float Fraction2 = (PhiIndex + U2) / (float)NumPhiSteps;

			const float R = FMath::Sqrt(1.0f - Fraction1 * Fraction1);

			const float Phi = 2.0f * (float)PI * Fraction2;
			// Convert to Cartesian
			Samples.Add(FVector4f(FMath::Cos(Phi) * R, FMath::Sin(Phi) * R, Fraction1));
			Uniforms.Add(FVector2f(Fraction1, Fraction2));
		}
	}
}

void GenerateStratifiedCosineHemisphereSamples(int32 NumThetaSteps, int32 NumPhiSteps, FLMRandomStream& RandomStream, TArray<FVector4f>& Samples)
{
	Samples.Empty(NumThetaSteps * NumPhiSteps);

	for (int32 ThetaIndex = 0; ThetaIndex < NumThetaSteps; ThetaIndex++)
	{
		for (int32 PhiIndex = 0; PhiIndex < NumPhiSteps; PhiIndex++)
		{
			const float U1 = RandomStream.GetFraction();
			const float U2 = RandomStream.GetFraction();

			const float Fraction1 = (ThetaIndex + U1) / (float)NumThetaSteps;
			const float Fraction2 = (PhiIndex + U2) / (float)NumPhiSteps;

			const float Theta = FMath::Acos(FMath::Sqrt(Fraction1));
			const float Phi = 2.0f * (float)PI * Fraction2;
			checkSlow(Theta >= 0 && Theta <= (float)HALF_PI);
			checkSlow(Phi >= 0 && Phi <= 2.0f * (float)PI);
			const float SinTheta = FMath::Sin(Theta);
			// Convert to Cartesian
			Samples.Add(FVector4f(FMath::Cos(Phi) * SinTheta, FMath::Sin(Phi) * SinTheta, FMath::Cos(Theta)));
		}
	}
}

/** 
 * Multiple importance sampling power heuristic of two functions with a power of two. 
 * From Veach's PHD thesis titled "Robust Monte Carlo Methods for Light Transport Simulation", page 273.
 */
float PowerHeuristic(int32 NumF, float fPDF, int32 NumG, float gPDF)
{
	const float fWeight = NumF * fPDF;
	const float gWeight = NumG * gPDF;
	return fWeight * fWeight / (fWeight * fWeight + gWeight * gWeight);
}

/** Calculates the step 1D cumulative distribution function for the given unnormalized probability distribution function. */
void CalculateStep1dCDF(const TArray<float>& PDF, TArray<float>& CDF, float& UnnormalizedIntegral)
{
	CDF.Empty(PDF.Num());
	float RunningUnnormalizedIntegral = 0;
	CDF.Add(0.0f);
	for (int32 i = 1; i < PDF.Num(); i++)
	{
		RunningUnnormalizedIntegral += PDF[i - 1];
		CDF.Add(RunningUnnormalizedIntegral);
	}
	UnnormalizedIntegral = RunningUnnormalizedIntegral + PDF.Last();
	if (UnnormalizedIntegral > 0.0f)
	{
		// Normalize the CDF
		for (int32 i = 1; i < CDF.Num(); i++)
		{
			CDF[i] /= UnnormalizedIntegral;
		}
	}
	check(CDF.Num() == PDF.Num());
}

/** Generates a Sample from the given step 1D probability distribution function. */
void Sample1dCDF(const TArray<float>& PDFArray, const TArray<float>& CDFArray, float UnnormalizedIntegral, FLMRandomStream& RandomStream, float& PDF, float& Sample)
{
	checkSlow(PDFArray.Num() > 0);
	checkSlow(PDFArray.Num() == CDFArray.Num());
	
	// See pages 641-644 of the "Physically Based Rendering" book for an excellent description of 
	// How to sample from a piecewise-constant 1d function, which this implementation is based on.
	if (PDFArray.Num() > 1)
	{
		// Get a uniformly distributed pseudo-random number
		const float RandomFraction = RandomStream.GetFraction();
		int32 GreaterElementIndex = -1;
		// Find the index of where the step function becomes greater or equal to the generated number
		//@todo - CDFArray is monotonically increasing so we can do better than a linear time search
		for (int32 i = 1; i < CDFArray.Num(); i++)
		{
			if (CDFArray[i] >= RandomFraction)
			{
				GreaterElementIndex = i;
				break;
			}
		}
		if (GreaterElementIndex >= 0)
		{
			check(GreaterElementIndex >= 1 && GreaterElementIndex < CDFArray.Num());
			// Find the fraction that the generated number is from the element before the greater or equal element.
			const float OffsetAlongCDFSegment = (RandomFraction - CDFArray[GreaterElementIndex - 1]) / (CDFArray[GreaterElementIndex] - CDFArray[GreaterElementIndex - 1]);
			// Access the probability that this element was selected and normalize it 
			PDF = PDFArray[GreaterElementIndex - 1] / UnnormalizedIntegral;
			Sample = (GreaterElementIndex - 1 + OffsetAlongCDFSegment) / (float)CDFArray.Num();
		}
		else
		{
			// The last element in the 1d CDF was selected
			const float OffsetAlongCDFSegment = (RandomFraction - CDFArray.Last()) / (1.0f - CDFArray.Last());
			PDF = PDFArray.Last() / UnnormalizedIntegral;
			Sample = FMath::Clamp((CDFArray.Num() - 1 + OffsetAlongCDFSegment) / (float)CDFArray.Num(), 0.0f, 1.0f - DELTA);
		}
	}
	else
	{
		PDF = 1.0f;
		Sample = 0;
	}
}

}
