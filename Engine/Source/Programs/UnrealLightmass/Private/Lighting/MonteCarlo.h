// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LMMath.h"

namespace Lightmass
{
	/** Generates valid X and Y axes of a coordinate system, given the Z axis. */
	void GenerateCoordinateSystem(const FVector4f& ZAxis, FVector4f& XAxis, FVector4f& YAxis);
		
	/** Generates a pseudo-random unit vector, uniformly distributed over all directions. */
	FVector4f GetUnitVector(FLMRandomStream& RandomStream);

	/** Generates a pseudo-random position inside the unit sphere, uniformly distributed over the volume of the sphere. */
	FVector4f GetUnitPosition(FLMRandomStream& RandomStream);

	/** 
	 * Generates a pseudo-random unit vector in the Z > 0 hemisphere whose PDF == 1 / (2 * PI) in solid angles,
	 * Or sin(theta) / (2 * PI) in hemispherical coordinates, which is a uniform distribution over the area of the hemisphere.
	 */
	FVector4f GetUniformHemisphereVector(FLMRandomStream& RandomStream, float MaxTheta = (float)HALF_PI);

	/** 
	 * Generates a pseudo-random unit vector in the Z > 0 hemisphere whose PDF == cos(theta) / PI in solid angles,
	 * Which is sin(theta)cos(theta) / PI in hemispherical coordinates.
	 */
	FVector4f GetCosineHemisphereVector(FLMRandomStream& RandomStream, float MaxTheta = (float)HALF_PI);

	/** 
	 * Generates a pseudo-random unit vector in the Z > 0 hemisphere,
	 * Whose PDF == (SpecularPower + 1) / (2.0f * PI) * cos(Alpha) ^ SpecularPower in solid angles,
	 * Where Alpha is the angle between the perfect specular direction and the outgoing direction.
	 */
	FVector4f GetModifiedPhongSpecularVector(FLMRandomStream& RandomStream, const FVector4f& TangentSpecularDirection, float SpecularPower);

	/** 
	 * Generates a pseudo-random position within a unit disk,
	 * Whose PDF == 1 / PI, which is a uniform distribution over the area of the disk.
	 */
	FVector2f GetUniformUnitDiskPosition(FLMRandomStream& RandomStream);

	/** 
	 * Generates a pseudo-random direction within a cone,
	 * Whose PDF == 1 / (2 * PI * (1 - CosMaxConeTheta)), which is a uniform distribution over the directions in the cone. 
	 */
	FVector4f UniformSampleCone(FLMRandomStream& RandomStream, float CosMaxConeTheta, const FVector4f& XAxis, const FVector4f& YAxis, const FVector4f& ZAxis);

	FVector4f UniformSampleCone(float CosMaxConeTheta, const FVector4f& XAxis, const FVector4f& YAxis, const FVector4f& ZAxis, float Uniform1, float Uniform2);

	/** Calculates the PDF for a sample generated by UniformSampleCone */
	float UniformConePDF(float CosMaxConeTheta);

	FVector4f UniformSampleHemisphere(float Uniform1, float Uniform2);

	/** Generates unit length, stratified and uniformly distributed direction samples in a hemisphere. */
	void GenerateStratifiedUniformHemisphereSamples(int32 NumThetaSteps, int32 NumPhiSteps, FLMRandomStream& RandomStream, TArray<FVector4f>& Samples, TArray<FVector2f>& Uniforms);

	void GenerateStratifiedCosineHemisphereSamples(int32 NumThetaSteps, int32 NumPhiSteps, FLMRandomStream& RandomStream, TArray<FVector4f>& Samples);

	/** 
	 * Multiple importance sampling power heuristic of two functions with a power of two. 
	 * From Veach's PHD thesis titled "Robust Monte Carlo Methods for Light Transport Simulation", page 273.
	 */
	float PowerHeuristic(int32 NumF, float fPDF, int32 NumG, float gPDF);

	/** Calculates the step 1D cumulative distribution function for the given unnormalized probability distribution function. */
	void CalculateStep1dCDF(const TArray<float>& PDF, TArray<float>& CDF, float& UnnormalizedIntegral);

	/** Generates a Sample from the given step 1D probability distribution function. */
	void Sample1dCDF(const TArray<float>& PDFArray, const TArray<float>& CDFArray, float UnnormalizedIntegral, FLMRandomStream& RandomStream, float& PDF, float& Sample);

}
