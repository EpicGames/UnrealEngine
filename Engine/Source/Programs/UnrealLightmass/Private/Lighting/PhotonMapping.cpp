// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LightingSystem.h"
#include "HAL/RunnableThread.h"
#include "MonteCarlo.h"
#include "LightmassSwarm.h"
#include "HAL/ExceptionHandling.h"

namespace Lightmass
{

/** Average fraction of emitted direct photons that get deposited on surfaces, used to presize gathered photon arrays. */
static const float DirectPhotonEfficiency = .3f;

/** Average fraction of emitted indirect photons that get deposited on surfaces, used to presize gathered photon arrays. */
static const float IndirectPhotonEfficiency = .1f;

// Number of parts that photon operating passes will be split into.
// Random number generators and other state will be seeded at the beginning of each work range, 
// To ensure deterministic behavior regardless of how many threads are processing and what order operations happen within each thread.
static const int32 NumPhotonWorkRanges = 256;

/** Sets up photon mapping settings. */
void FStaticLightingSystem::InitializePhotonSettings()
{
	const FBoxSphereBounds3f SceneBounds = FBoxSphereBounds3f(AggregateMesh->GetBounds());
	const FBoxSphereBounds3f ImportanceBounds = GetImportanceBounds();

	// Get direct photon counts from each light
#if !LIGHTMASS_DO_PROCESSING
	const int32 MaxNumDirectPhotonsToEmit = 10;
#else
	// Maximum number of direct photons to emit, used to cap the memory and processing time for a given build
	//@todo - remove these clamps and come up with a robust solution for huge scenes
	const int32 MaxNumDirectPhotonsToEmit = 40000000;
#endif
	NumDirectPhotonsToEmit = 0;
	Stats.NumFirstPassPhotonsRequested = 0;
	// Only emit direct photons if they will be used for lighting
	if (GeneralSettings.NumIndirectLightingBounces > 0 || PhotonMappingSettings.bVisualizeCachedApproximateDirectLighting)
	{
		for (int32 LightIndex = 0; LightIndex < Lights.Num(); LightIndex++)
		{
			Stats.NumFirstPassPhotonsRequested += Lights[LightIndex]->GetNumDirectPhotons(PhotonMappingSettings.DirectPhotonDensity);
		}

		NumDirectPhotonsToEmit = FMath::Min<uint64>(Stats.NumFirstPassPhotonsRequested, (uint64)MaxNumDirectPhotonsToEmit);
		if (NumDirectPhotonsToEmit == MaxNumDirectPhotonsToEmit)
		{
			LogSolverMessage(FString::Printf(TEXT("Clamped the number of direct photons to emit to %.3f million, from %.3f million requested."), MaxNumDirectPhotonsToEmit / 1000000.0f, Stats.NumFirstPassPhotonsRequested / 1000000.0f));
		}
	}

	DirectIrradiancePhotonFraction = FMath::Clamp(Scene.PhotonMappingSettings.DirectIrradiancePhotonDensity / Scene.PhotonMappingSettings.DirectPhotonDensity, 0.0f, 1.0f);

	// Calculate numbers of photons to gather based on the scene using the given photon densities, the scene's surface area and the importance volume's surface area
	float SceneSurfaceAreaMillionUnits = FMath::Max(AggregateMesh->GetSurfaceArea() / 1000000.0f, DELTA);
	float SceneSurfaceAreaMillionUnitsEstimate = FMath::Max(4.0f * (float)PI * SceneBounds.SphereRadius * SceneBounds.SphereRadius / 1000000.0f, DELTA);
	float SceneSurfaceAreaMillionUnitsEstimateDiff = SceneSurfaceAreaMillionUnitsEstimate > DELTA ? ( SceneSurfaceAreaMillionUnits / SceneSurfaceAreaMillionUnitsEstimate * 100.0f ) : 0.0f;
	LogSolverMessage(FString::Printf(TEXT("Scene surface area calculated at %.3f million units (%.3f%% of the estimated %.3f million units)"), SceneSurfaceAreaMillionUnits, SceneSurfaceAreaMillionUnitsEstimateDiff, SceneSurfaceAreaMillionUnitsEstimate));

	float ImportanceSurfaceAreaMillionUnits = FMath::Max(AggregateMesh->GetSurfaceAreaWithinImportanceVolume() / 1000000.0f, DELTA);
	float ImportanceSurfaceAreaMillionUnitsEstimate = FMath::Max(4.0f * (float)PI * ImportanceBounds.SphereRadius * ImportanceBounds.SphereRadius / 1000000.0f, DELTA);
	float ImportanceSurfaceAreaMillionUnitsEstimateDiff = ImportanceSurfaceAreaMillionUnitsEstimate > DELTA ? ( ImportanceSurfaceAreaMillionUnits / ImportanceSurfaceAreaMillionUnitsEstimate * 100.0f ) : 0.0f;
	LogSolverMessage(FString::Printf(TEXT("Importance volume surface area calculated at %.3f million units (%.3f%% of the estimated %.3f million units)"), ImportanceSurfaceAreaMillionUnits, ImportanceSurfaceAreaMillionUnitsEstimateDiff, ImportanceSurfaceAreaMillionUnitsEstimate));

#if !LIGHTMASS_DO_PROCESSING
	const int32 MaxNumIndirectPhotonPaths = 10;
#else
	const int32 MaxNumIndirectPhotonPaths = 20000;
#endif
	// If the importance volume is valid, only gather enough indirect photon paths to meet IndirectPhotonPathDensity inside the importance volume
	if (!PhotonMappingSettings.bEmitPhotonsOutsideImportanceVolume && ImportanceBounds.SphereRadius > DELTA)
	{ 
		NumIndirectPhotonPaths = FMath::TruncToInt(Scene.PhotonMappingSettings.IndirectPhotonPathDensity * ImportanceSurfaceAreaMillionUnits);
	}
	else if (ImportanceBounds.SphereRadius > DELTA)
	{
		NumIndirectPhotonPaths = FMath::TruncToInt(Scene.PhotonMappingSettings.IndirectPhotonPathDensity * ImportanceSurfaceAreaMillionUnits
			+ Scene.PhotonMappingSettings.OutsideImportanceVolumeDensityScale * Scene.PhotonMappingSettings.IndirectPhotonPathDensity * SceneSurfaceAreaMillionUnits);
	}
	else
	{
		NumIndirectPhotonPaths = FMath::TruncToInt(Scene.PhotonMappingSettings.IndirectPhotonPathDensity * SceneSurfaceAreaMillionUnits);
	}
	NumIndirectPhotonPaths = NumIndirectPhotonPaths == appTruncErrorCode ? MaxNumIndirectPhotonPaths : NumIndirectPhotonPaths;
	NumIndirectPhotonPaths = FMath::Min(NumIndirectPhotonPaths, MaxNumIndirectPhotonPaths);
	if (NumIndirectPhotonPaths == MaxNumIndirectPhotonPaths)
	{
		LogSolverMessage(FString::Printf(TEXT("Clamped the number of indirect photon paths to %u."), MaxNumIndirectPhotonPaths));
	}

#if !LIGHTMASS_DO_PROCESSING
	const int32 MaxNumIndirectPhotons = 10;
#else
	const int32 MaxNumIndirectPhotons = 40000000;
#endif
	Stats.NumSecondPassPhotonsRequested = 0;
	// If the importance volume is valid, only emit enough indirect photons to meet IndirectPhotonDensity inside the importance volume
	if (!PhotonMappingSettings.bEmitPhotonsOutsideImportanceVolume && ImportanceBounds.SphereRadius > DELTA)
	{
		Stats.NumSecondPassPhotonsRequested = Scene.PhotonMappingSettings.IndirectPhotonDensity * ImportanceSurfaceAreaMillionUnits;
	}
	else if (ImportanceBounds.SphereRadius > DELTA)
	{
		Stats.NumSecondPassPhotonsRequested = Scene.PhotonMappingSettings.IndirectPhotonDensity * ImportanceSurfaceAreaMillionUnits
			+ Scene.PhotonMappingSettings.OutsideImportanceVolumeDensityScale * Scene.PhotonMappingSettings.IndirectPhotonDensity * SceneSurfaceAreaMillionUnits;
	}
	else
	{
		Stats.NumSecondPassPhotonsRequested = Scene.PhotonMappingSettings.IndirectPhotonDensity * SceneSurfaceAreaMillionUnits;
	}
	NumIndirectPhotonsToEmit = FMath::Min<uint64>(Stats.NumSecondPassPhotonsRequested, (uint64)MaxNumIndirectPhotons);
	if (NumIndirectPhotonsToEmit == MaxNumIndirectPhotons)
	{
		LogSolverMessage(FString::Printf(TEXT("Clamped the number of indirect photons to emit to %.3f million, from %.3f million requested."), MaxNumIndirectPhotons / 1000000.0f, Stats.NumSecondPassPhotonsRequested / 1000000.0f));
	}

	IndirectIrradiancePhotonFraction = FMath::Clamp(Scene.PhotonMappingSettings.IndirectIrradiancePhotonDensity / Scene.PhotonMappingSettings.IndirectPhotonDensity, 0.0f, 1.0f);
}

/** Emits photons, builds data structures to accelerate photon map lookups, and does any other photon preprocessing required. */
void FStaticLightingSystem::EmitPhotons()
{
	const FBoxSphereBounds3f SceneSphereBounds = FBoxSphereBounds3f(AggregateMesh->GetBounds());
	FBoxSphereBounds3f ImportanceVolumeBounds = GetImportanceBounds();
	if (ImportanceVolumeBounds.SphereRadius < DELTA)
	{
		ImportanceVolumeBounds = SceneSphereBounds;
	}
	
	// Presize for the results from two emitting passes (direct photon emitting, then indirect)
	mIrradiancePhotons.Empty(NumPhotonWorkRanges * 2);

	const double StartEmitDirectTime = FPlatformTime::Seconds();
	TArray<TArray<FIndirectPathRay> > IndirectPathRays;
	// Emit photons for the direct photon map, and gather rays which resulted in indirect photon paths.
	EmitDirectPhotons(ImportanceVolumeBounds, IndirectPathRays, mIrradiancePhotons);

	const double EndEmitDirectTime = FPlatformTime::Seconds();
	Stats.EmitDirectPhotonsTime = EndEmitDirectTime - StartEmitDirectTime;
	LogSolverMessage(FString::Printf(TEXT("EmitDirectPhotons complete, %.3f million photons emitted in %.1f seconds"), Stats.NumFirstPassPhotonsEmitted / 1000000.0f, Stats.EmitDirectPhotonsTime));

	// Let the scene's lights cache information about the indirect path rays, 
	// Which will be used to accelerate light sampling using those paths when emitting indirect photons.
	for (int32 LightIndex = 0; LightIndex < Lights.Num(); LightIndex++)
	{
		FLight* CurrentLight = Lights[LightIndex];
		CurrentLight->CachePathRays(IndirectPathRays[LightIndex]);
	}
	const double EndCachingIndirectPathsTime = FPlatformTime::Seconds();
	Stats.CachingIndirectPhotonPathsTime = EndCachingIndirectPathsTime - EndEmitDirectTime;

	// Emit photons for the indirect photon map, using the indirect photon paths to guide photon emission.
	EmitIndirectPhotons(ImportanceVolumeBounds, IndirectPathRays, mIrradiancePhotons);
	const double EndEmitIndirectTime = FPlatformTime::Seconds();
	Stats.EmitIndirectPhotonsTime = EndEmitIndirectTime - EndCachingIndirectPathsTime;
	LogSolverMessage(FString::Printf(TEXT("EmitIndirectPhotons complete, %.3f million photons emitted in %.1f seconds"), Stats.NumSecondPassPhotonsEmitted / 1000000.0f, Stats.EmitIndirectPhotonsTime));

	if (PhotonMappingSettings.bUseIrradiancePhotons)
	{
		// Process all irradiance photons and mark ones that have direct photons nearby,
		// So that we can search for those with a smaller radius when using them for rendering.
		// This allows more accurate direct shadow transitions with irradiance photons.
		MarkIrradiancePhotons(ImportanceVolumeBounds, mIrradiancePhotons);
		const double EndMarkIrradiancePhotonsTime = FPlatformTime::Seconds();
		Stats.IrradiancePhotonMarkingTime = EndMarkIrradiancePhotonsTime - EndEmitIndirectTime;
		LogSolverMessage(FString::Printf(TEXT("Marking Irradiance Photons complete, %.3f million photons marked in %.1f seconds"), Stats.NumIrradiancePhotons / 1000000.0f, Stats.IrradiancePhotonMarkingTime));

		if (PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces)
		{
			// Cache irradiance photons on surfaces, as an optimization for final gathering.
			// Final gather rays already know which surface they intersected, so we can do a constant time lookup to find
			// The nearest irradiance photon instead of doing a photon map gather at the end of each final gather ray.
			// As an additional benefit, only cached irradiance photons are actually used for rendering, so we only need to calculate irradiance for the used ones.
			CacheIrradiancePhotons();
			Stats.CacheIrradiancePhotonsTime = FPlatformTime::Seconds() - EndMarkIrradiancePhotonsTime;
			LogSolverMessage(FString::Printf(TEXT("Caching Irradiance Photons complete, %.3f million cache samples in %.1f seconds"), Stats.NumCachedIrradianceSamples / 1000000.0f, Stats.CacheIrradiancePhotonsTime));
		}
		// Calculate irradiance for photons found by the caching on surfaces pass.
		// This is done as an optimization to final gathering, as described in the paper titled "Faster Photon Map Global Illumination"
		// The optimization is that irradiance is pre-calculated at a subset of the photons so that final gather rays can just lookup the nearest irradiance photon,
		// Instead of doing a photon density estimation to calculate irradiance.
		const double StartCalculateIrradiancePhotonsTime = FPlatformTime::Seconds();
		CalculateIrradiancePhotons(ImportanceVolumeBounds, mIrradiancePhotons);
		Stats.IrradiancePhotonCalculatingTime = FPlatformTime::Seconds() - StartCalculateIrradiancePhotonsTime;
		LogSolverMessage(FString::Printf(TEXT("Calculate Irradiance Photons complete, %.3f million irradiance calculations in %.1f seconds"), Stats.NumFoundIrradiancePhotons / 1000000.0f, Stats.IrradiancePhotonCalculatingTime));
	}

	// Verify that temporary photon memory has been freed
	check(DirectPhotonEmittingWorkRanges.Num() == 0
		&& DirectPhotonEmittingOutputs.Num() == 0
		&& IndirectPhotonEmittingWorkRanges.Num() == 0
		&& IndirectPhotonEmittingOutputs.Num() == 0
		&& IrradianceMarkWorkRanges.Num() == 0
		&& IrradianceCalculationWorkRanges.Num() == 0
		&& IrradiancePhotonCachingThreads.Num() == 0);
}

/** Emits direct photons and generates indirect photon paths. */
void FStaticLightingSystem::EmitDirectPhotons(
	const FBoxSphereBounds3f& ImportanceBounds, 
	TArray<TArray<FIndirectPathRay> >& IndirectPathRays,
	TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons)
{
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing0, 0 ) );
	FSceneLightPowerDistribution LightDistribution;
	// Create a 1d Step Probability Density Function based on the number of direct photons each light wants to gather.
	LightDistribution.LightPDFs.Empty(Lights.Num());
	for (int32 LightIndex = 0; LightIndex < Lights.Num(); LightIndex++)
	{
		const FLight* CurrentLight = Lights[LightIndex];
		const int32 LightNumDirectPhotons = CurrentLight->GetNumDirectPhotons(PhotonMappingSettings.DirectPhotonDensity);
		LightDistribution.LightPDFs.Add(LightNumDirectPhotons);
	}

	if (Lights.Num() > 0)
	{
		// Compute the Cumulative Distribution Function for our step function of light powers
		CalculateStep1dCDF(LightDistribution.LightPDFs, LightDistribution.LightCDFs, LightDistribution.UnnormalizedIntegral);
	}

	IndirectPathRays.Empty(Lights.Num());
	IndirectPathRays.AddZeroed(Lights.Num());
	// Add irradiance photon array entries for all the work ranges that will be processed
	const int32 IrradianceArrayStart = IrradiancePhotons.AddZeroed(NumPhotonWorkRanges);

	const FDirectPhotonEmittingInput Input(ImportanceBounds, LightDistribution);

	int32 NumIndirectPhotonPathsRemaining = NumIndirectPhotonPaths;
	// Setup work ranges, which are sections of work that can be done in parallel.
	DirectPhotonEmittingWorkRanges.Empty(NumPhotonWorkRanges);
	for (int32 RangeIndex = 0; RangeIndex < NumPhotonWorkRanges - 1; RangeIndex++)
	{
		int32 NumIndirectPhotonPathsRange = FMath::Max(NumIndirectPhotonPaths / NumPhotonWorkRanges, 1);

		if (NumIndirectPhotonPathsRemaining == 0)
		{
			NumIndirectPhotonPathsRange = 0;
		}

		NumIndirectPhotonPathsRemaining = FMath::Max(NumIndirectPhotonPathsRemaining - NumIndirectPhotonPathsRange, 0);

		DirectPhotonEmittingWorkRanges.Add(FDirectPhotonEmittingWorkRange(RangeIndex, NumDirectPhotonsToEmit / NumPhotonWorkRanges, NumIndirectPhotonPathsRange));
	}
	// The last work range contains the remainders
	DirectPhotonEmittingWorkRanges.Add(FDirectPhotonEmittingWorkRange(
		NumPhotonWorkRanges - 1, 
		NumDirectPhotonsToEmit / NumPhotonWorkRanges + NumDirectPhotonsToEmit % NumPhotonWorkRanges, 
		NumIndirectPhotonPathsRemaining));

	DirectPhotonEmittingOutputs.Empty(NumPhotonWorkRanges);
	for (int32 RangeIndex = 0; RangeIndex < NumPhotonWorkRanges; RangeIndex++)
	{
		// Initialize outputs with the appropriate irradiance photon array
		DirectPhotonEmittingOutputs.Add(FDirectPhotonEmittingOutput(&IrradiancePhotons[IrradianceArrayStart + RangeIndex]));
	}

	// Spawn threads to emit direct photons
	TIndirectArray<FDirectPhotonEmittingThreadRunnable> DirectPhotonEmittingThreads;
	for (int32 ThreadIndex = 1; ThreadIndex < NumStaticLightingThreads; ThreadIndex++)
	{
		FDirectPhotonEmittingThreadRunnable* ThreadRunnable = new FDirectPhotonEmittingThreadRunnable(this, ThreadIndex, Input);
		DirectPhotonEmittingThreads.Add(ThreadRunnable);
		const FString ThreadName = FString::Printf(TEXT("DirectPhotonEmittingThread%u"), ThreadIndex);
		ThreadRunnable->Thread = FRunnableThread::Create(ThreadRunnable, *ThreadName);
	}

	const double StartEmittingDirectPhotonsMainThread = FPlatformTime::Seconds();

	// Add the photons into a spatial data structure to accelerate their searches later
	//@todo - should try a kd-tree instead as the distribution of photons is highly non-uniform
	DirectPhotonMap = FPhotonOctree(ImportanceBounds.Origin, ImportanceBounds.BoxExtent.GetMax());
	IrradiancePhotonMap = FIrradiancePhotonOctree(ImportanceBounds.Origin, ImportanceBounds.BoxExtent.GetMax());

	Stats.NumDirectPhotonsGathered = 0;
	Stats.NumDirectIrradiancePhotons = 0;
	int32 NumIndirectPhotonPathsGathered = 0;
	int32 NextOutputToProcess = 0;
	while (DirectPhotonEmittingWorkRangeIndex.GetValue() < DirectPhotonEmittingWorkRanges.Num()
		|| NextOutputToProcess < DirectPhotonEmittingOutputs.Num())
	{
		// Process one work range on the main thread
		EmitDirectPhotonsThreadLoop(Input, 0);

		LIGHTINGSTAT(FScopedRDTSCTimer MainThreadProcessTimer(Stats.ProcessDirectPhotonsThreadTime));
		// Process the outputs that have been completed by any thread
		// Outputs are collected from smallest to largest work range index so that the outputs will be deterministic.
		while (NextOutputToProcess < DirectPhotonEmittingOutputs.Num() 
			&& DirectPhotonEmittingOutputs[NextOutputToProcess].OutputComplete > 0)
		{
			const FDirectPhotonEmittingOutput& CurrentOutput = DirectPhotonEmittingOutputs[NextOutputToProcess];
			for (int32 PhotonIndex = 0; PhotonIndex < CurrentOutput.DirectPhotons.Num(); PhotonIndex++)
			{
				// Add direct photons to the direct photon map, which is an octree for now
				DirectPhotonMap.AddElement(FPhotonElement(CurrentOutput.DirectPhotons[PhotonIndex]));
			}

			for (int32 LightIndex = 0; LightIndex < CurrentOutput.IndirectPathRays.Num(); LightIndex++)
			{
				// Gather indirect path rays
				IndirectPathRays[LightIndex].Append(CurrentOutput.IndirectPathRays[LightIndex]);
				NumIndirectPhotonPathsGathered += CurrentOutput.IndirectPathRays[LightIndex].Num();
			}

			if (PhotonMappingSettings.bUseIrradiancePhotons && PhotonMappingSettings.bUsePhotonDirectLightingInFinalGather)
			{
				for (int32 PhotonIndex = 0; PhotonIndex < CurrentOutput.IrradiancePhotons->Num(); PhotonIndex++)
				{
					// Add the irradiance photons to an octree
					IrradiancePhotonMap.AddElement(FIrradiancePhotonElement(PhotonIndex, *CurrentOutput.IrradiancePhotons));
				}
				Stats.NumIrradiancePhotons += CurrentOutput.IrradiancePhotons->Num();
				Stats.NumDirectIrradiancePhotons += CurrentOutput.IrradiancePhotons->Num();
			}
			
			Stats.NumFirstPassPhotonsEmitted += CurrentOutput.NumPhotonsEmitted;
			NumPhotonsEmittedDirect += CurrentOutput.NumPhotonsEmittedDirect;
			Stats.NumDirectPhotonsGathered += CurrentOutput.DirectPhotons.Num();
			NextOutputToProcess++;
			Stats.DirectPhotonsTracingThreadTime += CurrentOutput.DirectPhotonsTracingThreadTime;
			Stats.DirectPhotonsLightSamplingThreadTime += CurrentOutput.DirectPhotonsLightSamplingThreadTime;
			Stats.DirectCustomAttenuationThreadTime += CurrentOutput.DirectCustomAttenuationThreadTime;
		}
	}

	Stats.EmitDirectPhotonsThreadTime = FPlatformTime::Seconds() - StartEmittingDirectPhotonsMainThread;

	// Wait until all worker threads have completed
	for (int32 ThreadIndex = 0; ThreadIndex < DirectPhotonEmittingThreads.Num(); ThreadIndex++)
	{
		DirectPhotonEmittingThreads[ThreadIndex].Thread->WaitForCompletion();
		DirectPhotonEmittingThreads[ThreadIndex].CheckHealth();
		delete DirectPhotonEmittingThreads[ThreadIndex].Thread;
		DirectPhotonEmittingThreads[ThreadIndex].Thread = NULL;
		Stats.EmitDirectPhotonsThreadTime += DirectPhotonEmittingThreads[ThreadIndex].ExecutionTime;
	}

	if (NumIndirectPhotonPathsGathered != NumIndirectPhotonPaths && GeneralSettings.NumIndirectLightingBounces > 0)
	{
		LogSolverMessage(FString::Printf(TEXT("Couldn't gather the requested number of indirect photon paths! %u gathered"), NumIndirectPhotonPathsGathered));
	}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (PhotonMappingSettings.bVisualizePhotonPaths)
	{
		if (GeneralSettings.ViewSingleBounceNumber < 0
			|| PhotonMappingSettings.bVisualizeCachedApproximateDirectLighting && GeneralSettings.ViewSingleBounceNumber == 0 
			|| PhotonMappingSettings.bUseFinalGathering && GeneralSettings.ViewSingleBounceNumber == 1)
		{
			if (PhotonMappingSettings.bUseIrradiancePhotons)
			{
				int32 NumDirectIrradiancePhotons = 0;
				for (int32 ArrayIndex = 0; ArrayIndex < IrradiancePhotons.Num(); ArrayIndex++)
				{
					NumDirectIrradiancePhotons += IrradiancePhotons[ArrayIndex].Num();
				}
				StartupDebugOutput.IrradiancePhotons.Empty(NumDirectIrradiancePhotons);
				for (int32 ArrayIndex = 0; ArrayIndex < IrradiancePhotons.Num(); ArrayIndex++)
				{
					for (int32 i = 0; i < IrradiancePhotons[ArrayIndex].Num(); i++)
					{
						StartupDebugOutput.IrradiancePhotons.Add(FDebugPhoton(0, IrradiancePhotons[ArrayIndex][i].GetPosition(), IrradiancePhotons[ArrayIndex][i].GetSurfaceNormal(), IrradiancePhotons[ArrayIndex][i].GetSurfaceNormal()));
					}
				}
			}
			else
			{
				StartupDebugOutput.DirectPhotons.Empty(Stats.NumDirectPhotonsGathered);
				for (int32 OutputIndex = 0; OutputIndex < DirectPhotonEmittingOutputs.Num(); OutputIndex++)
				{
					const FDirectPhotonEmittingOutput& CurrentOutput = DirectPhotonEmittingOutputs[OutputIndex];
					for (int32 i = 0; i < CurrentOutput.DirectPhotons.Num(); i++)
					{
						StartupDebugOutput.DirectPhotons.Add(FDebugPhoton(CurrentOutput.DirectPhotons[i].GetId(), CurrentOutput.DirectPhotons[i].GetPosition(), CurrentOutput.DirectPhotons[i].GetIncidentDirection(), CurrentOutput.DirectPhotons[i].GetSurfaceNormal()));
					}
				}
			}
		}
		if (GeneralSettings.ViewSingleBounceNumber != 0)
		{
			StartupDebugOutput.IndirectPhotonPaths.Empty(NumIndirectPhotonPathsGathered);
			for (int32 LightIndex = 0; LightIndex < IndirectPathRays.Num(); LightIndex++)
			{
				for (int32 RayIndex = 0; RayIndex < IndirectPathRays[LightIndex].Num(); RayIndex++)
				{
					StartupDebugOutput.IndirectPhotonPaths.Add(FDebugStaticLightingRay(
						IndirectPathRays[LightIndex][RayIndex].Start, 
						IndirectPathRays[LightIndex][RayIndex].Start + IndirectPathRays[LightIndex][RayIndex].UnitDirection * IndirectPathRays[LightIndex][RayIndex].Length,
						true));
				}
			}
		}
	}
#endif

	DirectPhotonEmittingWorkRanges.Empty();
	DirectPhotonEmittingOutputs.Empty();

	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing0, 0 ) );
}

uint32 FDirectPhotonEmittingThreadRunnable::Run()
{
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing0, ThreadIndex ) );

	const double StartThreadTime = FPlatformTime::Seconds();

	FixThreadGroupAffinity();

#if defined(_MSC_VER) && !defined(XBOX)
	if(!FPlatformMisc::IsDebuggerPresent())
	{
		__try
		{
			System->EmitDirectPhotonsThreadLoop(Input, ThreadIndex);
		}
		__except( ReportCrash( GetExceptionInformation() ) )
		{
			ErrorMessage = GErrorHist;
			bTerminatedByError = true;
		}
	}
	else
#endif
	{
		System->EmitDirectPhotonsThreadLoop(Input, ThreadIndex);
	}
	ExecutionTime = FPlatformTime::Seconds() - StartThreadTime;
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing0, ThreadIndex ) );
	return 0;
}

/** Entrypoint for all threads emitting direct photons. */
void FStaticLightingSystem::EmitDirectPhotonsThreadLoop(const FDirectPhotonEmittingInput& Input, int32 ThreadIndex)
{
	while (true)
	{
		// Atomically read and increment the next work range index to process.
		// In this way work ranges are processed on-demand, which ensures consistent end times between threads.
		// Processing from smallest to largest work range index since the main thread is processing outputs in the same order.
		const int32 RangeIndex = DirectPhotonEmittingWorkRangeIndex.Increment() - 1;
		if (RangeIndex < DirectPhotonEmittingWorkRanges.Num())
		{
			EmitDirectPhotonsWorkRange(Input, DirectPhotonEmittingWorkRanges[RangeIndex], DirectPhotonEmittingOutputs[RangeIndex]);
			if (ThreadIndex == 0)
			{
				// Break out of the loop on the main thread after one work range so that it can process any outputs that are ready
				break;
			}
		}
		else
		{
			// Processing has begun for all work ranges
			break;
		}
	}
}

/** Emits direct photons for a given work range. */
void FStaticLightingSystem::EmitDirectPhotonsWorkRange(
	const FDirectPhotonEmittingInput& Input, 
	FDirectPhotonEmittingWorkRange WorkRange, 
	FDirectPhotonEmittingOutput& Output)
{
	// No lights in the scene, so no photons to emit
	if (Lights.Num() == 0
		// No light power in the scene, so no photons to shoot
		|| Input.LightDistribution.UnnormalizedIntegral < DELTA)
	{
		// Indicate to the main thread that this output is ready for processing
		FPlatformAtomics::InterlockedIncrement(&DirectPhotonEmittingOutputs[WorkRange.RangeIndex].OutputComplete);
		return;
	}

	Output.IndirectPathRays.Empty(Lights.Num());
	Output.IndirectPathRays.AddZeroed(Lights.Num());
	for (int32 LightIndex = 0; LightIndex < Output.IndirectPathRays.Num(); LightIndex++)
	{
		Output.IndirectPathRays[LightIndex].Empty(WorkRange.TargetNumIndirectPhotonPaths);
	}
	if (PhotonMappingSettings.bUseIrradiancePhotons)
	{
		// Attempt to preallocate irradiance photons based on the percentage of photons that go into the irradiance photon map.
		// The actual number of irradiance photons is based on probability.
		Output.IrradiancePhotons->Empty(FMath::TruncToInt(DirectIrradiancePhotonFraction * DirectPhotonEfficiency * WorkRange.NumDirectPhotonsToEmit));
	}

	FCoherentRayCache CoherentRayCache;
	// Initialize the random stream using the work range's index,
	// So that different numbers will be generated for each work range, 
	// While maintaining determinism regardless of the order that work ranges are processed.
	FLMRandomStream RandomStream(WorkRange.RangeIndex);

	// Array of rays from each light which resulted in an indirect path.
	// These are used in the second emitting pass to guide light sampling for indirect photons.
	Output.DirectPhotons.Empty(FMath::TruncToInt(WorkRange.NumDirectPhotonsToEmit * DirectPhotonEfficiency));
	
	Output.NumPhotonsEmitted = 0;
	int32 NumIndirectPathRaysGathered = 0;

	// Emit photons until we reach the limit for this work range,
	while (Output.NumPhotonsEmitted < WorkRange.NumDirectPhotonsToEmit 
		// Or we haven't found enough indirect photon paths yet.
		|| NumIndirectPathRaysGathered < WorkRange.TargetNumIndirectPhotonPaths)
	{
		Output.NumPhotonsEmitted++;

		// Once we have emitted enough direct photons,
		// Stop emitting photons if we are getting below 0.2% efficiency for indirect photon paths
		// This can happen if the scene is close to convex
		if (Output.NumPhotonsEmitted >= WorkRange.NumDirectPhotonsToEmit 
			&& NumIndirectPathRaysGathered < WorkRange.TargetNumIndirectPhotonPaths
			&& Output.NumPhotonsEmitted > WorkRange.TargetNumIndirectPhotonPaths * 500.0f)
		{
			break;
		}

		int32 NumberOfPathVertices = 0;
		float LightPDF;
		float LightIndex;
		// Pick a light with probability proportional to the light's fraction of the direct photons being gathered for the whole scene
		Sample1dCDF(Input.LightDistribution.LightPDFs, Input.LightDistribution.LightCDFs, Input.LightDistribution.UnnormalizedIntegral, RandomStream, LightPDF, LightIndex);
		const int32 QuantizedLightIndex = FMath::TruncToInt(LightIndex * Input.LightDistribution.LightPDFs.Num());
		check(QuantizedLightIndex >= 0 && QuantizedLightIndex < Lights.Num());
		const FLight* Light = Lights[QuantizedLightIndex];

		FLightRay SampleRay;
		FVector4f LightSourceNormal;
		FVector2f LightSurfacePosition;
		float RayDirectionPDF;
		FLinearColor PathAlpha;
		{
			LIGHTINGSTAT(FScopedRDTSCTimer LightSampleTimer(Output.DirectPhotonsLightSamplingThreadTime));
			// Generate the first ray for a new path from the light's distribution of emitted light
			Light->SampleDirection(RandomStream, SampleRay, LightSourceNormal, LightSurfacePosition, RayDirectionPDF, PathAlpha);
		}
		// Update the path's throughput based on the probability of picking this light and this direction
		PathAlpha = PathAlpha / (LightPDF * RayDirectionPDF);
		if (PathAlpha.R <= 0.0f && PathAlpha.G <= 0.0f && PathAlpha.B <= 0.0f)
		{
			// Skip to next photon since the light doesn't emit any energy in this direction
			continue;
		}

		const float BeforeDirectTraceTime = CoherentRayCache.FirstHitRayTraceTime;
		// Find the first vertex of the photon path
		FLightRayIntersection PathIntersection;
		SampleRay.TraceFlags |= LIGHTRAY_FLIP_SIDEDNESS;
		AggregateMesh->IntersectLightRay(SampleRay, true, true, true, CoherentRayCache, PathIntersection);
		Output.DirectPhotonsTracingThreadTime += CoherentRayCache.FirstHitRayTraceTime - BeforeDirectTraceTime;

		const FVector4f WorldPathDirection = SampleRay.Direction.GetUnsafeNormal3();

		// Register this photon path as long as it hit a frontface of something in the scene
		if (PathIntersection.bIntersects 
			&& Dot3(WorldPathDirection, PathIntersection.IntersectionVertex.WorldTangentZ) < 0.0f)
		{
			{
				LIGHTINGSTAT(FScopedRDTSCTimer CustomAttenuationTimer(Output.DirectCustomAttenuationThreadTime));
				// Allow the light to attenuate in a non-physically correct way
				PathAlpha *= Light->CustomAttenuation(PathIntersection.IntersectionVertex.WorldPosition, RandomStream, true);
			}
			
			// Apply transmission
			PathAlpha *= PathIntersection.Transmission;

			if ((PathAlpha.R < DELTA && PathAlpha.G < DELTA && PathAlpha.B < DELTA)
				// Ray can hit translucent meshes if they have bCastShadowAsMasked, but we don't have diffuse for translucency, so just terminate
				|| PathIntersection.Mesh->IsTranslucent(PathIntersection.ElementIndex))
			{
				// Skip to the next photon since the path contribution was completely filtered out by transmission or attenuation
				continue;
			}

			NumberOfPathVertices++;
			// Note: SampleRay.Start is offset from the actual start position, but not enough to matter for the algorithms which make use of the photon's traveled distance.
			const float RayLength = (SampleRay.Start - PathIntersection.IntersectionVertex.WorldPosition).Size3();
			// Create a photon from this path vertex's information
			const FPhoton NewPhoton(Output.NumPhotonsEmitted, PathIntersection.IntersectionVertex.WorldPosition, RayLength, -WorldPathDirection, PathIntersection.IntersectionVertex.WorldTangentZ, PathAlpha);
			checkSlow(FLinearColorUtils::AreFloatsValid(PathAlpha));
			if (Output.NumPhotonsEmitted < WorkRange.NumDirectPhotonsToEmit
				// Only deposit photons inside the importance bounds
				&& Input.ImportanceBounds.GetBox().IsInside(PathIntersection.IntersectionVertex.WorldPosition))
			{
				Output.DirectPhotons.Add(NewPhoton);
				Output.NumPhotonsEmittedDirect = Output.NumPhotonsEmitted;
				if (PhotonMappingSettings.bUseIrradiancePhotons 
					// Create an irradiance photon for a fraction of the direct photons
					&& RandomStream.GetFraction() < DirectIrradiancePhotonFraction)
				{
					const FIrradiancePhoton NewIrradiancePhoton(PathIntersection.IntersectionVertex.WorldPosition, PathIntersection.IntersectionVertex.WorldTangentZ, true);
					Output.IrradiancePhotons->Add(NewIrradiancePhoton);
				}
			}

			// Continue tracing this path if we don't have enough indirect photon paths yet
			if (NumIndirectPathRaysGathered < WorkRange.TargetNumIndirectPhotonPaths)
			{
				FStaticLightingVertex IntersectionVertexWithTangents(PathIntersection.IntersectionVertex);
				FVector4f NewWorldPathDirection;
				float BRDFDirectionPDF;

				// Generate a new path direction from the BRDF
				const FLinearColor BRDF = PathIntersection.Mesh->SampleBRDF(
					IntersectionVertexWithTangents, 
					PathIntersection.ElementIndex,
					-WorldPathDirection, 
					NewWorldPathDirection, 
					BRDFDirectionPDF, 
					RandomStream);

				// Terminate if the path has lost all of its energy due to the surface's BRDF
				if (BRDF.Equals(FLinearColor::Black))
				{
					continue;
				}

				const float CosFactor = -Dot3(WorldPathDirection, IntersectionVertexWithTangents.WorldTangentZ);
				checkSlow(CosFactor >= 0.0f && CosFactor <= 1.0f);

				const FVector4f RayStart = IntersectionVertexWithTangents.WorldPosition 
					+ NewWorldPathDirection * SceneConstants.VisibilityRayOffsetDistance 
					+ IntersectionVertexWithTangents.WorldTangentZ * SceneConstants.VisibilityNormalOffsetDistance;
				const FVector4f RayEnd = IntersectionVertexWithTangents.WorldPosition + NewWorldPathDirection * MaxRayDistance;

				FLightRay IndirectSampleRay(
					RayStart,
					RayEnd,
					NULL,
					NULL
					); 

				const float BeforeIndirectTraceTime = CoherentRayCache.FirstHitRayTraceTime;
				FLightRayIntersection NewPathIntersection;
				IndirectSampleRay.TraceFlags |= LIGHTRAY_FLIP_SIDEDNESS;
				AggregateMesh->IntersectLightRay(IndirectSampleRay, true, false, false, CoherentRayCache, NewPathIntersection);
				Output.DirectPhotonsTracingThreadTime += CoherentRayCache.FirstHitRayTraceTime - BeforeIndirectTraceTime;

				if (NewPathIntersection.bIntersects && Dot3(NewWorldPathDirection, NewPathIntersection.IntersectionVertex.WorldTangentZ) < 0.0f)
				{
					// Store the original photon path which led to an indirect photon path,
					// This will be used in a second pass to guide indirect photon emission.
					Output.IndirectPathRays[QuantizedLightIndex].Add(FIndirectPathRay(SampleRay.Start, WorldPathDirection, LightSourceNormal, LightSurfacePosition, RayLength));
					NumIndirectPathRaysGathered++;
				}
			}
		}
	}
	// Indicate to the main thread that this output is ready for processing
	FPlatformAtomics::InterlockedIncrement(&DirectPhotonEmittingOutputs[WorkRange.RangeIndex].OutputComplete);
}

void FStaticLightingSystem::BuildPhotonSegmentMap(const FPhotonOctree& SourcePhotonMap, FPhotonSegmentOctree& OutPhotonSegementMap, float AddToSegmentMapChance)
{
	FLMRandomStream RandomStream(12345);

	for(FPhotonOctree::TConstIterator<> NodeIt(SourcePhotonMap); NodeIt.HasPendingNodes(); NodeIt.Advance())
	{
		const FPhotonOctree::FNode& CurrentNode = NodeIt.GetCurrentNode();

		FOREACH_OCTREE_CHILD_NODE(ChildRef)
		{
			if (CurrentNode.HasChild(ChildRef))
			{
				NodeIt.PushChild(ChildRef);
			}
		}

		for (FPhotonOctree::ElementConstIt ElementIt(CurrentNode.GetConstElementIt()); ElementIt; ++ElementIt)
		{
			const FPhotonElement& PhotonElement = *ElementIt;
			
			if (AddToSegmentMapChance >= 1 || RandomStream.GetFraction() < AddToSegmentMapChance)
			{
				const int32 NumSegments = FMath::DivideAndRoundUp(PhotonElement.Photon.GetDistance(), PhotonMappingSettings.PhotonSegmentMaxLength);
				const float InvNumSegments = 1.0f / NumSegments;

				for (int32 SegmentIndex = 0; SegmentIndex < NumSegments; SegmentIndex++)
				{
					FPhotonSegmentElement NewElement(&PhotonElement.Photon, SegmentIndex * InvNumSegments, InvNumSegments);
					OutPhotonSegementMap.AddElement(NewElement);
				}
			}
		}
	}
}

/** Gathers indirect photons based on the indirect photon paths. */
void FStaticLightingSystem::EmitIndirectPhotons(
	const FBoxSphereBounds3f& ImportanceBounds,
	const TArray<TArray<FIndirectPathRay> >& IndirectPathRays, 
	TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons)
{
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing1, 0 ) );
	FSceneLightPowerDistribution LightDistribution;
	// Create a 1d Step Probability Density Function based on light powers,
	// So that lights are chosen with a probability proportional to their fraction of the total light power in the scene.
	LightDistribution.LightPDFs.Empty(Lights.Num());
	for (int32 LightIndex = 0; LightIndex < Lights.Num(); LightIndex++)
	{
		const FLight* CurrentLight = Lights[LightIndex];
		LightDistribution.LightPDFs.Add(CurrentLight->Power());
	}

	if (Lights.Num() > 0) 
	{
		// Compute the Cumulative Distribution Function for our step function of light powers
		CalculateStep1dCDF(LightDistribution.LightPDFs, LightDistribution.LightCDFs, LightDistribution.UnnormalizedIntegral);
	}
	// Add irradiance photon array entries for all the work ranges that will be processed
	const int32 IndirectIrradianceArrayStart = IrradiancePhotons.AddZeroed(NumPhotonWorkRanges);
	const FIndirectPhotonEmittingInput Input(ImportanceBounds, LightDistribution, IndirectPathRays);

	// Setup work ranges, which are sections of work that can be done in parallel.
	IndirectPhotonEmittingWorkRanges.Empty(NumPhotonWorkRanges);
	for (int32 RangeIndex = 0; RangeIndex < NumPhotonWorkRanges - 1; RangeIndex++)
	{
		IndirectPhotonEmittingWorkRanges.Add(FIndirectPhotonEmittingWorkRange(RangeIndex, NumIndirectPhotonsToEmit / NumPhotonWorkRanges));
	}
	// The last work range will contain the remainder of photons
	IndirectPhotonEmittingWorkRanges.Add(FIndirectPhotonEmittingWorkRange(
		NumPhotonWorkRanges - 1, 
		NumIndirectPhotonsToEmit / NumPhotonWorkRanges + NumIndirectPhotonsToEmit % NumPhotonWorkRanges));

	IndirectPhotonEmittingOutputs.Empty(NumPhotonWorkRanges);
	for (int32 RangeIndex = 0; RangeIndex < NumPhotonWorkRanges; RangeIndex++)
	{
		// Initialize outputs with the appropriate irradiance photon array
		IndirectPhotonEmittingOutputs.Add(FIndirectPhotonEmittingOutput(&IrradiancePhotons[IndirectIrradianceArrayStart + RangeIndex]));
	}

	// Spawn threads to emit indirect photons
	TIndirectArray<FIndirectPhotonEmittingThreadRunnable> IndirectPhotonEmittingThreads;
	for (int32 ThreadIndex = 1; ThreadIndex < NumStaticLightingThreads; ThreadIndex++)
	{
		FIndirectPhotonEmittingThreadRunnable* ThreadRunnable = new FIndirectPhotonEmittingThreadRunnable(this, ThreadIndex, Input);
		IndirectPhotonEmittingThreads.Add(ThreadRunnable);
		const FString ThreadName = FString::Printf(TEXT("IndirectPhotonEmittingThread%u"), ThreadIndex);
		ThreadRunnable->Thread = FRunnableThread::Create(ThreadRunnable, *ThreadName);
	}

	const double StartEmittingIndirectPhotonsMainThread = FPlatformTime::Seconds();

	// Add the photons into spatial data structures to accelerate their searches later
	FirstBouncePhotonMap = FPhotonOctree(ImportanceBounds.Origin, ImportanceBounds.BoxExtent.GetMax());
	FirstBounceEscapedPhotonMap = FPhotonOctree(ImportanceBounds.Origin, ImportanceBounds.BoxExtent.GetMax());
	SecondBouncePhotonMap = FPhotonOctree(ImportanceBounds.Origin, ImportanceBounds.BoxExtent.GetMax());

	Stats.NumIndirectPhotonsGathered = 0;
	int32 NextOutputToProcess = 0;
	while (IndirectPhotonEmittingWorkRangeIndex.GetValue() < IndirectPhotonEmittingWorkRanges.Num()
		|| NextOutputToProcess < IndirectPhotonEmittingOutputs.Num())
	{
		// Process one work range on the main thread
		EmitIndirectPhotonsThreadLoop(Input, 0);

		LIGHTINGSTAT(FScopedRDTSCTimer MainThreadProcessTimer(Stats.ProcessIndirectPhotonsThreadTime));
		// Process the thread's outputs
		// Outputs are collected from smallest to largest work range index so that the outputs will be deterministic.
		while (NextOutputToProcess < IndirectPhotonEmittingOutputs.Num() 
			&& IndirectPhotonEmittingOutputs[NextOutputToProcess].OutputComplete > 0)
		{
			FIndirectPhotonEmittingOutput& CurrentOutput = IndirectPhotonEmittingOutputs[NextOutputToProcess];
			for (int32 PhotonIndex = 0; PhotonIndex < CurrentOutput.FirstBouncePhotons.Num(); PhotonIndex++)
			{
				FirstBouncePhotonMap.AddElement(FPhotonElement(CurrentOutput.FirstBouncePhotons[PhotonIndex]));
			}

			if (PhotonMappingSettings.bUsePhotonSegmentsForVolumeLighting)
			{
				for (int32 PhotonIndex = 0; PhotonIndex < CurrentOutput.FirstBounceEscapedPhotons.Num(); PhotonIndex++)
				{
					FirstBounceEscapedPhotonMap.AddElement(FPhotonElement(CurrentOutput.FirstBounceEscapedPhotons[PhotonIndex]));
				}
			}

			for (int32 PhotonIndex = 0; PhotonIndex < CurrentOutput.SecondBouncePhotons.Num(); PhotonIndex++)
			{
				SecondBouncePhotonMap.AddElement(FPhotonElement(CurrentOutput.SecondBouncePhotons[PhotonIndex]));
			}

			if (PhotonMappingSettings.bUseIrradiancePhotons)
			{
				for (int32 PhotonIndex = 0; PhotonIndex < CurrentOutput.IrradiancePhotons->Num(); PhotonIndex++)
				{
					// Add the irradiance photons to an octree
					IrradiancePhotonMap.AddElement(FIrradiancePhotonElement(PhotonIndex, *CurrentOutput.IrradiancePhotons));
				}
				Stats.NumIrradiancePhotons += CurrentOutput.IrradiancePhotons->Num();
			}

			Stats.NumSecondPassPhotonsEmitted += CurrentOutput.NumPhotonsEmitted;
			Stats.LightSamplingThreadTime += CurrentOutput.LightSamplingThreadTime;
			Stats.IndirectCustomAttenuationThreadTime += CurrentOutput.IndirectCustomAttenuationThreadTime;
			Stats.IntersectLightRayThreadTime += CurrentOutput.IntersectLightRayThreadTime;
			Stats.PhotonBounceTracingThreadTime += CurrentOutput.PhotonBounceTracingThreadTime;
			NumPhotonsEmittedFirstBounce += CurrentOutput.NumPhotonsEmittedFirstBounce;
			NumPhotonsEmittedSecondBounce += CurrentOutput.NumPhotonsEmittedSecondBounce;

			Stats.NumIndirectPhotonsGathered += CurrentOutput.FirstBouncePhotons.Num() + CurrentOutput.SecondBouncePhotons.Num();
			NextOutputToProcess++;
			CurrentOutput.FirstBouncePhotons.Empty();
			CurrentOutput.SecondBouncePhotons.Empty();
		}
	}

	FirstBouncePhotonSegmentMap = FPhotonSegmentOctree(ImportanceBounds.Origin, ImportanceBounds.BoxExtent.GetMax());

	if (PhotonMappingSettings.bUsePhotonSegmentsForVolumeLighting)
	{
		const double SegmentStartTime = FPlatformTime::Seconds();
		BuildPhotonSegmentMap(FirstBouncePhotonMap, FirstBouncePhotonSegmentMap, PhotonMappingSettings.GeneratePhotonSegmentChance);
		BuildPhotonSegmentMap(FirstBounceEscapedPhotonMap, FirstBouncePhotonSegmentMap, 1.0f);
		const float BuildSegmentMapTime = (float)(FPlatformTime::Seconds() - SegmentStartTime);
		LogSolverMessage(FString::Printf(TEXT("Built photon segment map in %.1f seconds"), BuildSegmentMapTime));
	}

	Stats.EmitIndirectPhotonsThreadTime = FPlatformTime::Seconds() - StartEmittingIndirectPhotonsMainThread;

	// Wait until all worker threads have completed
	for (int32 ThreadIndex = 0; ThreadIndex < IndirectPhotonEmittingThreads.Num(); ThreadIndex++)
	{
		IndirectPhotonEmittingThreads[ThreadIndex].Thread->WaitForCompletion();
		IndirectPhotonEmittingThreads[ThreadIndex].CheckHealth();
		delete IndirectPhotonEmittingThreads[ThreadIndex].Thread;
		IndirectPhotonEmittingThreads[ThreadIndex].Thread = NULL;
		Stats.EmitIndirectPhotonsThreadTime += IndirectPhotonEmittingThreads[ThreadIndex].ExecutionTime;
	}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (PhotonMappingSettings.bVisualizePhotonPaths
		&& PhotonMappingSettings.bUseIrradiancePhotons 
		&& GeneralSettings.ViewSingleBounceNumber != 0)
	{
		int32 NumIndirectIrradiancePhotons = 0;
		for (int32 RangeIndex = NumPhotonWorkRanges; RangeIndex < IrradiancePhotons.Num(); RangeIndex++)
		{
			NumIndirectIrradiancePhotons += IrradiancePhotons[RangeIndex].Num();
		}
		StartupDebugOutput.IrradiancePhotons.Empty(NumIndirectIrradiancePhotons);
		for (int32 RangeIndex = NumPhotonWorkRanges; RangeIndex < IrradiancePhotons.Num(); RangeIndex++)
		{
			for (int32 i = 0; i < IrradiancePhotons[RangeIndex].Num(); i++)
			{
				StartupDebugOutput.IrradiancePhotons.Add(FDebugPhoton(0, IrradiancePhotons[RangeIndex][i].GetPosition(), IrradiancePhotons[RangeIndex][i].GetSurfaceNormal(), IrradiancePhotons[RangeIndex][i].GetSurfaceNormal()));
			}
		}
	}
#endif
	
	IndirectPhotonEmittingWorkRanges.Empty();
	IndirectPhotonEmittingOutputs.Empty();
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing1, 0 ) );
}

uint32 FIndirectPhotonEmittingThreadRunnable::Run()
{
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing1, ThreadIndex ) );
	const double StartThreadTime = FPlatformTime::Seconds();
	FixThreadGroupAffinity();
#if defined(_MSC_VER) && !defined(XBOX)
	if(!FPlatformMisc::IsDebuggerPresent())
	{
		__try
		{
			System->EmitIndirectPhotonsThreadLoop(Input, ThreadIndex);
		}
		__except( ReportCrash( GetExceptionInformation() ) )
		{
			ErrorMessage = GErrorHist;
			bTerminatedByError = true;
		}
	}
	else
#endif
	{
		System->EmitIndirectPhotonsThreadLoop(Input, ThreadIndex);
	}
	const double EndThreadTime = FPlatformTime::Seconds();
	EndTime = EndThreadTime - GStartupTime;
	ExecutionTime = EndThreadTime - StartThreadTime;

	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing1, ThreadIndex ) );
	return 0;
}

/** Entrypoint for all threads emitting indirect photons. */
void FStaticLightingSystem::EmitIndirectPhotonsThreadLoop(const FIndirectPhotonEmittingInput& Input, int32 ThreadIndex)
{
	while (true)
	{
		// Atomically read and increment the next work range index to process.
		// In this way work ranges are processed on-demand, which ensures consistent end times between threads.
		// Processing from smallest to largest work range index since the main thread is processing outputs in the same order.
		const int32 RangeIndex = IndirectPhotonEmittingWorkRangeIndex.Increment() - 1;
		if (RangeIndex < IndirectPhotonEmittingWorkRanges.Num())
		{
			EmitIndirectPhotonsWorkRange(Input, IndirectPhotonEmittingWorkRanges[RangeIndex], IndirectPhotonEmittingOutputs[RangeIndex]);
			if (ThreadIndex == 0)
			{
				// Break out of the loop on the main thread after one work range so that it can process any outputs that are ready
				break;
			}
		}
		else
		{
			// Processing has begun for all work ranges
			break;
		}
	}
}

/** Emits indirect photons for a given work range. */
void FStaticLightingSystem::EmitIndirectPhotonsWorkRange(
	const FIndirectPhotonEmittingInput& Input, 
	FIndirectPhotonEmittingWorkRange WorkRange, 
	FIndirectPhotonEmittingOutput& Output)
{
	if (Input.IndirectPathRays.Num() == 0 || Input.LightDistribution.UnnormalizedIntegral < DELTA)
	{
		// No lights in the scene, so no photons to emit
		// Indicate to the main thread that this output is ready for processing
		FPlatformAtomics::InterlockedIncrement(&IndirectPhotonEmittingOutputs[WorkRange.RangeIndex].OutputComplete);
		return;
	}

	//@todo - re-evaluate these sizes
	Output.FirstBouncePhotons.Empty(FMath::TruncToInt(WorkRange.NumIndirectPhotonsToEmit * .6f * IndirectPhotonEfficiency));
	Output.FirstBounceEscapedPhotons.Empty(FMath::TruncToInt(WorkRange.NumIndirectPhotonsToEmit * .6f * IndirectPhotonEfficiency * PhotonMappingSettings.GeneratePhotonSegmentChance));
	Output.SecondBouncePhotons.Empty(FMath::TruncToInt(WorkRange.NumIndirectPhotonsToEmit * .4f * IndirectPhotonEfficiency));
	if (PhotonMappingSettings.bUseIrradiancePhotons)
	{
		// Attempt to preallocate irradiance photons based on the percentage of photons that go into the irradiance photon map.
		// The actual number of irradiance photons is based on probability.
		Output.IrradiancePhotons->Empty(FMath::TruncToInt(IndirectIrradiancePhotonFraction * IndirectPhotonEfficiency * WorkRange.NumIndirectPhotonsToEmit));
	}

	FCoherentRayCache CoherentRayCache;
	// Seed the random number generator at the beginning of each work range, so we get deterministic results regardless of the number of threads being used.
	FLMRandomStream RandomStream(WorkRange.RangeIndex);

	const bool bIndirectPhotonsNeeded = WorkRange.NumIndirectPhotonsToEmit > 0 && GeneralSettings.NumIndirectLightingBounces > 0;

	Output.NumPhotonsEmitted = 0;

	// Emit photons until we reach the limit for this work range,
	while (bIndirectPhotonsNeeded && Output.NumPhotonsEmitted < WorkRange.NumIndirectPhotonsToEmit)
	{
		Output.NumPhotonsEmitted++;

		int32 NumberOfPathVertices = 0;

		FLightRay SampleRay;
		FLinearColor PathAlpha;
		const FLight* Light = NULL;
		{
			LIGHTINGSTAT(FScopedRDTSCTimer SampleLightTimer(Output.LightSamplingThreadTime));
			float LightPDF;
			float LightIndex;
			// Pick a light with probability proportional to the light's fraction of the scene's light power
			Sample1dCDF(Input.LightDistribution.LightPDFs, Input.LightDistribution.LightCDFs, Input.LightDistribution.UnnormalizedIntegral, RandomStream, LightPDF, LightIndex);
			const int32 QuantizedLightIndex = FMath::TruncToInt(LightIndex * Input.LightDistribution.LightPDFs.Num());
			check(QuantizedLightIndex >= 0 && QuantizedLightIndex < Lights.Num());
			Light = Lights[QuantizedLightIndex];

			float RayDirectionPDF;
			if (Input.IndirectPathRays[QuantizedLightIndex].Num() > 0)
			{
				// Use the indirect path rays to sample the light
				Light->SampleDirection(
					Input.IndirectPathRays[QuantizedLightIndex], 
					RandomStream, 
					SampleRay, 
					RayDirectionPDF, 
					PathAlpha);
			}
			else
			{
				FVector4f LightSourceNormal;
				FVector2f LightSurfacePosition;
				// No indirect path rays from this light, sample it uniformly
				Light->SampleDirection(RandomStream, SampleRay, LightSourceNormal, LightSurfacePosition, RayDirectionPDF, PathAlpha);
			}
			// Update the path's throughput based on the probability of picking this light and this direction
			PathAlpha = PathAlpha / (LightPDF * RayDirectionPDF);
			checkSlow(FLinearColorUtils::AreFloatsValid(PathAlpha));
			if (PathAlpha.R < DELTA && PathAlpha.G < DELTA && PathAlpha.B < DELTA)
			{
				// Skip to the next photon since the light doesn't emit any energy in this direction
				continue;
			}

			// Clip the end of the photon path to the importance volume, or skip if the photon path does not intersect the importance volume at all.
			FVector4f ClippedStart, ClippedEnd;
			if (!ClipLineWithBox(Input.ImportanceBounds.GetBox(), SampleRay.Start, SampleRay.End, ClippedStart, ClippedEnd))
			{
				continue;
			}
			SampleRay.End = ClippedEnd;
		}

		// Find the first vertex of the photon path
		FLightRayIntersection PathIntersection;
		const float BeforeLightRayTime = CoherentRayCache.FirstHitRayTraceTime;
		SampleRay.TraceFlags |= LIGHTRAY_FLIP_SIDEDNESS;
		AggregateMesh->IntersectLightRay(SampleRay, true, true, true, CoherentRayCache, PathIntersection);
		Output.IntersectLightRayThreadTime += CoherentRayCache.FirstHitRayTraceTime - BeforeLightRayTime;

		LIGHTINGSTAT(FScopedRDTSCTimer PhotonTracingTimer(Output.PhotonBounceTracingThreadTime));
		FVector4f WorldPathDirection = SampleRay.Direction.GetUnsafeNormal3();
		// Continue tracing this photon path as long as it hit a frontface of something in the scene
		while (PathIntersection.bIntersects && Dot3(WorldPathDirection, PathIntersection.IntersectionVertex.WorldTangentZ) < 0.0f)
		{
			if (NumberOfPathVertices == 0)
			{
				LIGHTINGSTAT(FScopedRDTSCTimer CustomAttenuationTimer(Output.IndirectCustomAttenuationThreadTime));
				// Allow the light to attenuate in a non-physically correct way
				PathAlpha *= Light->CustomAttenuation(PathIntersection.IntersectionVertex.WorldPosition, RandomStream, false);
			}

			// Apply transmission
			PathAlpha *= PathIntersection.Transmission;

			if (PathAlpha.R < DELTA && PathAlpha.G < DELTA && PathAlpha.B < DELTA)
			{
				// Skip to the next photon since the light was completely filtered out by transmission or attenuation
				break;
			}

			checkSlow(FLinearColorUtils::AreFloatsValid(PathAlpha));

			NumberOfPathVertices++;

			// Only deposit photons inside the importance bounds
			if (Input.ImportanceBounds.GetBox().IsInside(PathIntersection.IntersectionVertex.WorldPosition))
			{
				// Only deposit a photon if it is not a direct lighting path, and we still need to gather more indirect photons
				if (NumberOfPathVertices > 1 
					&& Output.NumPhotonsEmitted < WorkRange.NumIndirectPhotonsToEmit)
				{
					// Note: SampleRay.Start is offset from the actual start position, but not enough to matter for the algorithms which make use of the photon's traveled distance.
					const float RayLength = (SampleRay.Start - PathIntersection.IntersectionVertex.WorldPosition).Size3();
					// Create a photon from this path vertex's information
					const FPhoton NewPhoton(Output.NumPhotonsEmitted, PathIntersection.IntersectionVertex.WorldPosition, RayLength, -WorldPathDirection, PathIntersection.IntersectionVertex.WorldTangentZ, PathAlpha);

					bool bShouldCreateIrradiancePhoton = false;
					if (NumberOfPathVertices == 2)
					{ 
						// This is a first bounce photon
						Output.FirstBouncePhotons.Add(NewPhoton);
						Output.NumPhotonsEmittedFirstBounce = Output.NumPhotonsEmitted;
						// Only allow creating an irradiance photon if one or more indirect bounces are required
						// The final gather is the first bounce when enabled
						bShouldCreateIrradiancePhoton = 
							(PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 1)
							|| (!PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 0);
					}
					else
					{
						Output.SecondBouncePhotons.Add(NewPhoton);
						Output.NumPhotonsEmittedSecondBounce = Output.NumPhotonsEmitted;
						// Only allow creating an irradiance photon if two or more indirect bounces are required
						// The final gather is the first bounce when enabled
						bShouldCreateIrradiancePhoton = 
							(PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 2)
							|| (!PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 1);
					}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
					if (PhotonMappingSettings.bVisualizePhotonPaths
						&& !PhotonMappingSettings.bUseIrradiancePhotons
						&& (GeneralSettings.ViewSingleBounceNumber < 0
						|| PhotonMappingSettings.bUseFinalGathering && GeneralSettings.ViewSingleBounceNumber > 1 
						|| !PhotonMappingSettings.bUseFinalGathering && GeneralSettings.ViewSingleBounceNumber > 0))
					{
						FScopeLock DebugOutputLock(&DebugOutputSync);
						if (StartupDebugOutput.IndirectPhotons.Num() == 0)
						{
							StartupDebugOutput.IndirectPhotons.Empty(FMath::TruncToInt(NumIndirectPhotonsToEmit * IndirectPhotonEfficiency));
						}
						StartupDebugOutput.IndirectPhotons.Add(FDebugPhoton(NewPhoton.GetId(), NewPhoton.GetPosition(), SampleRay.Start - NewPhoton.GetPosition(), NewPhoton.GetSurfaceNormal()));
					}
#endif
					// Create an irradiance photon for a fraction of the deposited photons
					if (PhotonMappingSettings.bUseIrradiancePhotons 
						&& bShouldCreateIrradiancePhoton
						&& RandomStream.GetFraction() < IndirectIrradiancePhotonFraction)
					{
						const FIrradiancePhoton NewIrradiancePhoton(NewPhoton.GetPosition(), PathIntersection.IntersectionVertex.WorldTangentZ, false);
						Output.IrradiancePhotons->Add(NewIrradiancePhoton);
					}
				}
			}

			// Photon bounce number = NumberOfPathVertices - 1
			// But with final gathering, a first bounce photon is second bounce lighting
			const int32 EffectiveBounceNumber = NumberOfPathVertices - 1 + 1;

			// Stop tracing this photon due to bounce number
			if (EffectiveBounceNumber >= GeneralSettings.NumIndirectLightingBounces
				// Ray can hit translucent meshes if they have bCastShadowAsMasked, but we don't have diffuse for translucency, so just terminate
				|| PathIntersection.Mesh->IsTranslucent(PathIntersection.ElementIndex))
			{
				break;
			}

			FStaticLightingVertex IntersectionVertexWithTangents(PathIntersection.IntersectionVertex);

			FVector4f NewWorldPathDirection;
			float BRDFDirectionPDF;

			// Generate a new path direction from the BRDF
			const FLinearColor BRDF = PathIntersection.Mesh->SampleBRDF(
				IntersectionVertexWithTangents, 
				PathIntersection.ElementIndex,
				-WorldPathDirection, 
				NewWorldPathDirection, 
				BRDFDirectionPDF, 
				RandomStream);

			// Terminate if the path has lost all of its energy due to the surface's BRDF
			if (BRDF.Equals(FLinearColor::Black)
				// Terminate if indirect photons are completed
				|| Output.NumPhotonsEmitted >= WorkRange.NumIndirectPhotonsToEmit)
			{
				break;
			}

			const float CosFactor = -Dot3(WorldPathDirection, IntersectionVertexWithTangents.WorldTangentZ);
			checkSlow(CosFactor >= 0.0f && CosFactor <= 1.0f);
			if (NumberOfPathVertices == 1)
			{
				// On the first bounce, re-weight the photon's throughput instead of using Russian Roulette to maintain equal weights,
				// Because NumEmitted/NumDeposited efficiency is more important to first bounce photons than having equal weight,
				// Since they are used for importance sampling the final gather.
				// Re-weight the throughput based on the probability of surviving.
				PathAlpha = PathAlpha * BRDF * CosFactor / BRDFDirectionPDF;
			}
			else
			{
				const FLinearColor NewPathAlpha = PathAlpha * BRDF * CosFactor / BRDFDirectionPDF;
				// On second and up bounces, terminate the path with probability proportional to the ratio between the new throughput and the old
				// This results in a smaller number of photons after surface reflections, but they have the same weight as before the reflection,
				// Which is desirable to reduce noise from the photon maps, at the cost of lower NumEmitted/NumDeposited efficiency.
				// See the "Extended Photon Map Implementation" paper for details.

				// Note: to be physically correct this probability should be clamped to [0,1], however this produces photons with extremely large weights,
				// So instead we maintain a constant photon weight after the surface interaction,
				// At the cost of introducing bias and leaking energy for bounces where BRDF * CosFactor / BRDFDirectionPDF > 1.
				const float ContinueProbability = FLinearColorUtils::LinearRGBToXYZ(NewPathAlpha).G / FLinearColorUtils::LinearRGBToXYZ(PathAlpha).G;
				const float RandomFloat = RandomStream.GetFraction();
				if (RandomFloat > ContinueProbability)
				{
					// Terminate due to Russian Roulette
					break;
				}
				PathAlpha = NewPathAlpha / ContinueProbability;
			}

			checkSlow(FLinearColorUtils::AreFloatsValid(PathAlpha));

			const FVector4f RayStart = IntersectionVertexWithTangents.WorldPosition 
				+ NewWorldPathDirection * SceneConstants.VisibilityRayOffsetDistance 
				+ IntersectionVertexWithTangents.WorldTangentZ * SceneConstants.VisibilityNormalOffsetDistance;
			FVector4f RayEnd = IntersectionVertexWithTangents.WorldPosition + NewWorldPathDirection * MaxRayDistance;

			// Clip photon path end points to the importance volume, so we do not bother tracing rays outside the area that photons can be deposited.
			// If the photon path does not intersect the importance volume at all, it did not originate from inside the volume, so skip to the next photon.
			FVector4f ClippedStart, ClippedEnd;
			if (!ClipLineWithBox(Input.ImportanceBounds.GetBox(), RayStart, RayEnd, ClippedStart, ClippedEnd))
			{
				break;
			}
			RayEnd = ClippedEnd;

			SampleRay = FLightRay(
				RayStart,
				RayEnd,
				NULL,
				NULL
				); 

			// Trace a ray to determine the next vertex of the photon's path.
			SampleRay.TraceFlags |= LIGHTRAY_FLIP_SIDEDNESS;
			AggregateMesh->IntersectLightRay(SampleRay, true, true, false, CoherentRayCache, PathIntersection);
			WorldPathDirection = NewWorldPathDirection;
		}

		// First bounce escaped photon
		if (!PathIntersection.bIntersects 
			&& NumberOfPathVertices == 1 
			&& PhotonMappingSettings.bUsePhotonSegmentsForVolumeLighting)
		{
			if (RandomStream.GetFraction() < PhotonMappingSettings.GeneratePhotonSegmentChance)
			{
				// Apply transmission
				PathAlpha *= PathIntersection.Transmission;

				checkSlow(FLinearColorUtils::AreFloatsValid(PathAlpha));

				FVector4f RayEnd = SampleRay.Start + WorldPathDirection * Input.ImportanceBounds.SphereRadius * 2;
				FVector4f ClippedStart, ClippedEnd;
				if (ClipLineWithBox(Input.ImportanceBounds.GetBox(), SampleRay.Start, RayEnd, ClippedStart, ClippedEnd))
				{
					// Create the description for an escaped photon
					const FPhoton NewPhoton(Output.NumPhotonsEmitted, ClippedEnd, (ClippedEnd - SampleRay.Start).Size3(), -WorldPathDirection, FVector4f(0, 0, 1), PathAlpha);
					Output.FirstBounceEscapedPhotons.Add(NewPhoton);
				}
			}
		}
	}
	// Indicate to the main thread that this output is ready for processing
	FPlatformAtomics::InterlockedIncrement(&IndirectPhotonEmittingOutputs[WorkRange.RangeIndex].OutputComplete);
}

/** Iterates through all irradiance photons, searches for nearby direct photons, and marks the irradiance photon has having direct photon influence if necessary. */
void FStaticLightingSystem::MarkIrradiancePhotons(const FBoxSphereBounds3f& ImportanceBounds, TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons)
{
	check(PhotonMappingSettings.bUseIrradiancePhotons);
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing2, 0 ) );

	// Setup work ranges for processing the irradiance photons
	IrradianceMarkWorkRanges.Empty(IrradiancePhotons.Num());
	for (int32 WorkRange = 0; WorkRange < IrradiancePhotons.Num(); WorkRange++)
	{
		IrradianceMarkWorkRanges.Add(FIrradianceMarkingWorkRange(WorkRange, WorkRange));
	}

	TIndirectArray<FIrradiancePhotonMarkingThreadRunnable> IrradiancePhotonMarkingThreads;
	IrradiancePhotonMarkingThreads.Empty(NumStaticLightingThreads);
	for(int32 ThreadIndex = 1; ThreadIndex < NumStaticLightingThreads; ThreadIndex++)
	{
		FIrradiancePhotonMarkingThreadRunnable* ThreadRunnable = new FIrradiancePhotonMarkingThreadRunnable(this, ThreadIndex, IrradiancePhotons);
		IrradiancePhotonMarkingThreads.Add(ThreadRunnable);
		const FString ThreadName = FString::Printf(TEXT("IrradiancePhotonMarkingThread%u"), ThreadIndex);
		ThreadRunnable->Thread = FRunnableThread::Create(ThreadRunnable, *ThreadName);
	}

	const double MainThreadStartTime = FPlatformTime::Seconds();

	MarkIrradiancePhotonsThreadLoop(0, IrradiancePhotons);
	
	Stats.IrradiancePhotonMarkingThreadTime = FPlatformTime::Seconds() - MainThreadStartTime;

	// Stop the static lighting threads.
	for(int32 ThreadIndex = 0;ThreadIndex < IrradiancePhotonMarkingThreads.Num();ThreadIndex++)
	{
		// Wait for the thread to exit.
		IrradiancePhotonMarkingThreads[ThreadIndex].Thread->WaitForCompletion();

		// Check that it didn't terminate with an error.
		IrradiancePhotonMarkingThreads[ThreadIndex].CheckHealth();

		// Destroy the thread.
		delete IrradiancePhotonMarkingThreads[ThreadIndex].Thread;

		// Accumulate each thread's execution time and stats
		Stats.IrradiancePhotonMarkingThreadTime += IrradiancePhotonMarkingThreads[ThreadIndex].ExecutionTime;
	}

	IrradianceMarkWorkRanges.Empty();

	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing2, 0 ) );
}

uint32 FIrradiancePhotonMarkingThreadRunnable::Run()
{
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing2, ThreadIndex ) );
	const double StartThreadTime = FPlatformTime::Seconds();
	FixThreadGroupAffinity();
#if defined(_MSC_VER) && !defined(XBOX)
	if(!FPlatformMisc::IsDebuggerPresent())
	{
		__try
		{
			System->MarkIrradiancePhotonsThreadLoop(ThreadIndex, IrradiancePhotons);
		}
		__except( ReportCrash( GetExceptionInformation() ) )
		{
			ErrorMessage = GErrorHist;
			bTerminatedByError = true;
		}
	}
	else
#endif
	{
		System->MarkIrradiancePhotonsThreadLoop(ThreadIndex, IrradiancePhotons);
	}
	const double EndThreadTime = FPlatformTime::Seconds();
	EndTime = EndThreadTime - GStartupTime;
	ExecutionTime = EndThreadTime - StartThreadTime;
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing2, ThreadIndex ) );
	return 0;
}

/** Entry point for all threads marking irradiance photons. */
void FStaticLightingSystem::MarkIrradiancePhotonsThreadLoop(
	int32 ThreadIndex, 
	TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons)
{
	while (true)
	{
		// Atomically read and increment the next work range index to process.
		// In this way work ranges are processed on-demand, which ensures consistent end times between threads.
		const int32 RangeIndex = IrradianceMarkWorkRangeIndex.Increment() - 1;
		if (RangeIndex < IrradianceMarkWorkRanges.Num())
		{
			MarkIrradiancePhotonsWorkRange(IrradiancePhotons, IrradianceMarkWorkRanges[RangeIndex]);
		}
		else
		{
			// Processing has begun for all work ranges
			break;
		}
	}
}

/** Marks irradiance photons specified by a single work range. */
void FStaticLightingSystem::MarkIrradiancePhotonsWorkRange(
	TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons, 
	FIrradianceMarkingWorkRange WorkRange)
{
	// Temporary array that is reused for all photon searches by this thread, to reduce allocations
	TArray<FPhoton> TempFoundPhotons;
	TArray<FIrradiancePhoton>& CurrentArray = IrradiancePhotons[WorkRange.IrradiancePhotonArrayIndex];
	for (int32 PhotonIndex = 0; PhotonIndex < CurrentArray.Num(); PhotonIndex++)
	{
		FIrradiancePhoton& CurrentIrradiancePhoton = CurrentArray[PhotonIndex];

		// Only add direct contribution if we are final gathering and at least one bounce is required,
		if ((PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 0)
			// Or if photon mapping is being used for direct lighting.
			|| PhotonMappingSettings.bVisualizeCachedApproximateDirectLighting)
		{
			// Find a nearby direct photon
			const bool bHasDirectContribution = FindAnyNearbyPhoton(DirectPhotonMap, CurrentIrradiancePhoton.GetPosition(), PhotonMappingSettings.DirectPhotonSearchDistance, false);
			if (bHasDirectContribution)
			{
				// Mark the irradiance photon has having direct contribution, which will be used to reduce the search radius for this irradiance photon,
				// In order to get more accurate direct shadow transitions using the photon map.
				CurrentIrradiancePhoton.SetHasDirectContribution();
			}
		}
	}
}

/** Calculates irradiance for photons randomly chosen to precalculate irradiance. */
void FStaticLightingSystem::CalculateIrradiancePhotons(const FBoxSphereBounds3f& ImportanceBounds, TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons)
{
	check(PhotonMappingSettings.bUseIrradiancePhotons);
	//@todo - add a preparing stage for the swarm visualizer
	//GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing2, 0 ) );

	if (!PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces)
	{
		// Without bCacheIrradiancePhotonsOnSurfaces, treat all irradiance photons as found since we'll have to calculate irradiance for all of them.
		Stats.NumFoundIrradiancePhotons = Stats.NumIrradiancePhotons;
	}

	if (PhotonMappingSettings.bVisualizeIrradiancePhotonCalculation && Scene.DebugMapping)
	{
		float ClosestIrradiancePhotonDistSq = FLT_MAX;
		// Skip direct irradiance photons if viewing indirect bounces
		const int32 ArrayStart = GeneralSettings.ViewSingleBounceNumber > 0 ? NumPhotonWorkRanges : 0;
		// Skip indirect irradiance photons if viewing direct only
		const int32 ArrayEnd = GeneralSettings.ViewSingleBounceNumber == 0 ? NumPhotonWorkRanges : IrradiancePhotons.Num();
		for (int32 ArrayIndex = ArrayStart; ArrayIndex < ArrayEnd; ArrayIndex++)
		{
			for (int32 PhotonIndex = 0; PhotonIndex < IrradiancePhotons[ArrayIndex].Num(); PhotonIndex++)
			{
				const FIrradiancePhoton& CurrentPhoton = IrradiancePhotons[ArrayIndex][PhotonIndex];
				const float CurrentDistSquared = (CurrentPhoton.GetPosition() - Scene.DebugInput.Position).SizeSquared3();
				if ((!PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces || CurrentPhoton.IsUsed())
					&& CurrentDistSquared < ClosestIrradiancePhotonDistSq)
				{
					// Debug the closest irradiance photon to the selected position.
					// NOTE: This is not necessarily the photon that will get cached for the selected texel!
					// It's not easy to figure out which photon will get cached at this point in the lighting process, so we use the closest instead.
					//@todo - if bCacheIrradiancePhotonsOnSurfaces is enabled, we can figure out exactly which photon will be used by the selected texel or vertex.
					ClosestIrradiancePhotonDistSq = CurrentDistSquared;
					DebugIrradiancePhotonCalculationArrayIndex = ArrayIndex;
					DebugIrradiancePhotonCalculationPhotonIndex = PhotonIndex;
				}
			}
		}
	}

	// Setup work ranges for processing the irradiance photons
	IrradianceCalculationWorkRanges.Empty(IrradiancePhotons.Num());
	for (int32 WorkRange = 0; WorkRange < IrradiancePhotons.Num(); WorkRange++)
	{
		IrradianceCalculationWorkRanges.Add(FIrradianceCalculatingWorkRange(WorkRange, WorkRange));
	}

	TIndirectArray<FIrradiancePhotonCalculatingThreadRunnable> IrradiancePhotonThreads;
	IrradiancePhotonThreads.Empty(NumStaticLightingThreads);
	for(int32 ThreadIndex = 1; ThreadIndex < NumStaticLightingThreads; ThreadIndex++)
	{
		FIrradiancePhotonCalculatingThreadRunnable* ThreadRunnable = new FIrradiancePhotonCalculatingThreadRunnable(this, ThreadIndex, IrradiancePhotons);
		IrradiancePhotonThreads.Add(ThreadRunnable);
		const FString ThreadName = FString::Printf(TEXT("IrradiancePhotonCalculatingThread%u"), ThreadIndex);
		ThreadRunnable->Thread = FRunnableThread::Create(ThreadRunnable, *ThreadName);
	}

	const double MainThreadStartTime = FPlatformTime::Seconds();

	FCalculateIrradiancePhotonStats MainThreadStats;
	CalculateIrradiancePhotonsThreadLoop(0, IrradiancePhotons, MainThreadStats);
	
	Stats.IrradiancePhotonCalculatingThreadTime = FPlatformTime::Seconds() - MainThreadStartTime;
	Stats.CalculateIrradiancePhotonStats = MainThreadStats;

	// Stop the static lighting threads.
	for(int32 ThreadIndex = 0;ThreadIndex < IrradiancePhotonThreads.Num();ThreadIndex++)
	{
		// Wait for the thread to exit.
		IrradiancePhotonThreads[ThreadIndex].Thread->WaitForCompletion();

		// Check that it didn't terminate with an error.
		IrradiancePhotonThreads[ThreadIndex].CheckHealth();

		// Destroy the thread.
		delete IrradiancePhotonThreads[ThreadIndex].Thread;

		// Accumulate each thread's execution time and stats
		Stats.IrradiancePhotonCalculatingThreadTime += IrradiancePhotonThreads[ThreadIndex].ExecutionTime;
		Stats.CalculateIrradiancePhotonStats += IrradiancePhotonThreads[ThreadIndex].Stats;
	}

	IrradianceCalculationWorkRanges.Empty();

	// Release all of the direct photon map memory since we are not going to need it later
	DirectPhotonMap.Destroy();
	// Release all of the second bounce photon map memory since it will not be used again
	SecondBouncePhotonMap.Destroy();
}

uint32 FIrradiancePhotonCalculatingThreadRunnable::Run()
{
	const double StartThreadTime = FPlatformTime::Seconds();
	FixThreadGroupAffinity();
#if defined(_MSC_VER) && !defined(XBOX)
	if(!FPlatformMisc::IsDebuggerPresent())
	{
		__try
		{
			System->CalculateIrradiancePhotonsThreadLoop(ThreadIndex, IrradiancePhotons, Stats);
		}
		__except( ReportCrash( GetExceptionInformation() ) )
		{
			ErrorMessage = GErrorHist;
			bTerminatedByError = true;
		}
	}
	else
#endif
	{
		System->CalculateIrradiancePhotonsThreadLoop(ThreadIndex, IrradiancePhotons, Stats);
	}
	const double EndThreadTime = FPlatformTime::Seconds();
	EndTime = EndThreadTime - GStartupTime;
	ExecutionTime = EndThreadTime - StartThreadTime;
	return 0;
}

/** Main loop that all threads access to calculate irradiance photons. */
void FStaticLightingSystem::CalculateIrradiancePhotonsThreadLoop(
	int32 ThreadIndex, 
	TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons, 
	FCalculateIrradiancePhotonStats& OutStats)
{
	while (true)
	{
		// Atomically read and increment the next work range index to process.
		// In this way work ranges are processed on-demand, which ensures consistent end times between threads.
		// Processing from smallest to largest work range index since the main thread is processing outputs in the same order.
		const int32 RangeIndex = IrradianceCalcWorkRangeIndex.Increment() - 1;
		if (RangeIndex < IrradianceCalculationWorkRanges.Num())
		{
			CalculateIrradiancePhotonsWorkRange(IrradiancePhotons, IrradianceCalculationWorkRanges[RangeIndex], OutStats);
		}
		else
		{
			// Processing has begun for all work ranges
			break;
		}
	}
}

/** Calculates irradiance for the photons specified by a single work range. */
void FStaticLightingSystem::CalculateIrradiancePhotonsWorkRange(
	TArray<TArray<FIrradiancePhoton>>& IrradiancePhotons, 
	FIrradianceCalculatingWorkRange WorkRange,
	FCalculateIrradiancePhotonStats& OutStats)
{
	// Temporary array that is reused for all photon searches by this thread, to reduce allocations
	TArray<FPhoton> TempFoundPhotons;
	TArray<FIrradiancePhoton>& CurrentArray = IrradiancePhotons[WorkRange.IrradiancePhotonArrayIndex];
	for (int32 PhotonIndex = 0; PhotonIndex < CurrentArray.Num(); PhotonIndex++)
	{
		FIrradiancePhoton& CurrentIrradiancePhoton = CurrentArray[PhotonIndex];
		// If we already cached irradiance photons on surfaces, only calculate irradiance for photons which actually got found.
		if (PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces && !CurrentIrradiancePhoton.IsUsed())
		{
			continue;
		}

		const bool bDebugThisPhoton = PhotonMappingSettings.bVisualizeIrradiancePhotonCalculation
			&& DebugIrradiancePhotonCalculationArrayIndex == WorkRange.IrradiancePhotonArrayIndex 
			&& DebugIrradiancePhotonCalculationPhotonIndex == PhotonIndex;

		FLinearColor AccumulatedIrradiance(FLinearColor::Black);
		// Only add direct contribution if we are final gathering and at least one bounce is required,
		if (((PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 0)
			// Or if photon mapping is being used for direct lighting.
			|| PhotonMappingSettings.bVisualizeCachedApproximateDirectLighting)
			&& PhotonMappingSettings.bUsePhotonDirectLightingInFinalGather)
		{
			const FLinearColor DirectPhotonIrradiance = CalculatePhotonIrradiance(
				DirectPhotonMap, 
				NumPhotonsEmittedDirect, 
				PhotonMappingSettings.NumIrradianceCalculationPhotons, 
				PhotonMappingSettings.DirectPhotonSearchDistance, 
				CurrentIrradiancePhoton,
				bDebugThisPhoton && GeneralSettings.ViewSingleBounceNumber == 0,
				TempFoundPhotons,
				OutStats);

			checkSlow(FLinearColorUtils::AreFloatsValid(DirectPhotonIrradiance));

			// Only add direct contribution if it should be viewed given ViewSingleBounceNumber
			if (GeneralSettings.ViewSingleBounceNumber < 0
				|| (PhotonMappingSettings.bUseFinalGathering && GeneralSettings.ViewSingleBounceNumber == 1)
				|| (!PhotonMappingSettings.bUseFinalGathering && GeneralSettings.ViewSingleBounceNumber == 0)
				|| (PhotonMappingSettings.bVisualizeCachedApproximateDirectLighting && GeneralSettings.ViewSingleBounceNumber == 0))
			{
				AccumulatedIrradiance = DirectPhotonIrradiance;
			}
		}

		// If we are final gathering, first bounce photons are actually the second lighting bounce since the final gather is the first bounce
		if ((PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 1)
			|| (!PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 0))
		{
			const FLinearColor FirstBouncePhotonIrradiance = CalculatePhotonIrradiance(
				FirstBouncePhotonMap, 
				NumPhotonsEmittedFirstBounce, 
				PhotonMappingSettings.NumIrradianceCalculationPhotons, 
				PhotonMappingSettings.IndirectPhotonSearchDistance, 
				CurrentIrradiancePhoton,
				bDebugThisPhoton && GeneralSettings.ViewSingleBounceNumber == 1,
				TempFoundPhotons,
				OutStats);

			checkSlow(FLinearColorUtils::AreFloatsValid(FirstBouncePhotonIrradiance));

			// Only add first bounce contribution if it should be viewed given ViewSingleBounceNumber
			if (GeneralSettings.ViewSingleBounceNumber < 0 
				|| (PhotonMappingSettings.bUseFinalGathering && GeneralSettings.ViewSingleBounceNumber == 2)
				|| (!PhotonMappingSettings.bUseFinalGathering && GeneralSettings.ViewSingleBounceNumber == 1))
			{
				AccumulatedIrradiance += FirstBouncePhotonIrradiance;
			}

			// If we are final gathering, second bounce photons are actually the third lighting bounce since the final gather is the first bounce
			if ((PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 2)
				|| (!PhotonMappingSettings.bUseFinalGathering && GeneralSettings.NumIndirectLightingBounces > 1))
			{
				const FLinearColor SecondBouncePhotonIrradiance = CalculatePhotonIrradiance(
					SecondBouncePhotonMap, 
					NumPhotonsEmittedSecondBounce, 
					PhotonMappingSettings.NumIrradianceCalculationPhotons, 
					PhotonMappingSettings.IndirectPhotonSearchDistance, 
					CurrentIrradiancePhoton,
					bDebugThisPhoton && GeneralSettings.ViewSingleBounceNumber > 1,
					TempFoundPhotons,
					OutStats);

				checkSlow(FLinearColorUtils::AreFloatsValid(SecondBouncePhotonIrradiance));

				// Only add second and up bounce contribution if it should be viewed given ViewSingleBounceNumber
				if (GeneralSettings.ViewSingleBounceNumber < 0 
					|| (PhotonMappingSettings.bUseFinalGathering && GeneralSettings.ViewSingleBounceNumber == 3)
					|| (!PhotonMappingSettings.bUseFinalGathering && GeneralSettings.ViewSingleBounceNumber == 2))
				{
					AccumulatedIrradiance += SecondBouncePhotonIrradiance;
				}
			}
		}
		CurrentIrradiancePhoton.SetIrradiance(AccumulatedIrradiance);
	}
}

/** Cache irradiance photons on surfaces. */
void FStaticLightingSystem::CacheIrradiancePhotons()
{
	check(PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces);
	for(int32 ThreadIndex = 1; ThreadIndex < NumStaticLightingThreads; ThreadIndex++)
	{
		FMappingProcessingThreadRunnable* ThreadRunnable = new FMappingProcessingThreadRunnable(this, ThreadIndex, StaticLightingTask_CacheIrradiancePhotons);
		IrradiancePhotonCachingThreads.Add(ThreadRunnable);
		const FString ThreadName = FString::Printf(TEXT("IrradiancePhotonCachingThread%u"), ThreadIndex);
		ThreadRunnable->Thread = FRunnableThread::Create(ThreadRunnable, *ThreadName);
	}

	// Start the static lighting thread loop on the main thread, too.
	// Once it returns, all static lighting mappings have begun processing.
	CacheIrradiancePhotonsThreadLoop(0, true);

	// Stop the static lighting threads.
	for(int32 ThreadIndex = 0;ThreadIndex < IrradiancePhotonCachingThreads.Num();ThreadIndex++)
	{
		// Wait for the thread to exit.
		IrradiancePhotonCachingThreads[ThreadIndex].Thread->WaitForCompletion();
		// Check that it didn't terminate with an error.
		IrradiancePhotonCachingThreads[ThreadIndex].CheckHealth();

		// Destroy the thread.
		delete IrradiancePhotonCachingThreads[ThreadIndex].Thread;
	}
	IrradiancePhotonCachingThreads.Empty();
	IrradiancePhotonMap.Destroy();
}

/** Main loop that all threads access to cache irradiance photons. */
void FStaticLightingSystem::CacheIrradiancePhotonsThreadLoop(int32 ThreadIndex, bool bIsMainThread)
{
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing3, ThreadIndex ) );
	bool bIsDone = false;
	while (!bIsDone)
	{
		// Atomically read and increment the next mapping index to process.
		const int32 MappingIndex = NextMappingToCacheIrradiancePhotonsOn.Increment() - 1;

		if (MappingIndex < AllMappings.Num())
		{
			// If this is the main thread, update progress and apply completed static lighting.
			if (bIsMainThread)
			{
				// Check the health of all static lighting threads.
				for (int32 ThreadIndexIter = 0; ThreadIndexIter < IrradiancePhotonCachingThreads.Num(); ThreadIndexIter++)
				{
					IrradiancePhotonCachingThreads[ThreadIndexIter].CheckHealth();
				}
			}

			FStaticLightingTextureMapping* TextureMapping = AllMappings[MappingIndex]->GetTextureMapping();

			if (TextureMapping)
			{
				CacheIrradiancePhotonsTextureMapping(TextureMapping);
			}
		}
		else
		{
			// Processing has begun for all mappings.
			bIsDone = true;
		}
	}
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing3, ThreadIndex ) );
}

/** Returns true if a photon was found within MaxPhotonSearchDistance. */
bool FStaticLightingSystem::FindAnyNearbyPhoton(
	const FPhotonOctree& PhotonMap, 
	const FVector4f& SearchPosition, 
	float MaxPhotonSearchDistance,
	bool bDebugThisLookup) const
{
	FPlatformAtomics::InterlockedIncrement(&Stats.NumPhotonGathers);

	const FBox3f SearchBox = FBox3f::BuildAABB(SearchPosition, FVector4f(MaxPhotonSearchDistance, MaxPhotonSearchDistance, MaxPhotonSearchDistance));
	for (FPhotonOctree::TConstIterator<> OctreeIt(PhotonMap); OctreeIt.HasPendingNodes(); OctreeIt.Advance())
	{
		const FPhotonOctree::FNode& CurrentNode = OctreeIt.GetCurrentNode();
		const FOctreeNodeContext& CurrentContext = OctreeIt.GetCurrentContext();

		// Push children onto the iterator stack if they intersect the query box
		if (!CurrentNode.IsLeaf())
		{
			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if (CurrentNode.HasChild(ChildRef))
				{
					const FOctreeNodeContext ChildContext = CurrentContext.GetChildContext(ChildRef);
					if (ChildContext.Bounds.GetBox().Intersect(SearchBox))
					{
						OctreeIt.PushChild(ChildRef);
					}
				}
			}
		}

		// Iterate over all photons in the nodes intersecting the query box
		for (FPhotonOctree::ElementConstIt MeshIt(CurrentNode.GetConstElementIt()); MeshIt; ++MeshIt)
		{
			const FPhotonElement& PhotonElement = *MeshIt;
			const float DistanceSquared = (PhotonElement.Photon.GetPosition() - SearchPosition).SizeSquared3();
			// Only searching for photons closer than the max distance
			if (DistanceSquared < MaxPhotonSearchDistance * MaxPhotonSearchDistance)
			{
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisLookup 
					&& PhotonMappingSettings.bVisualizePhotonGathers
					&& &PhotonMap == &DirectPhotonMap)
				{
					StartupDebugOutput.bDirectPhotonValid = true;
					StartupDebugOutput.GatheredDirectPhoton = FDebugPhoton(PhotonElement.Photon.GetId(), PhotonElement.Photon.GetPosition(), PhotonElement.Photon.GetIncidentDirection(), PhotonElement.Photon.GetSurfaceNormal());
				}
#endif
				return true;
			}
		}
	}
	return false;
}

/** 
 * Searches the given photon map for the nearest NumPhotonsToFind photons to SearchPosition using an iterative process, 
 * Unless the start and max search distances are the same, in which case all photons in that distance will be returned.
 * The iterative search starts at StartPhotonSearchDistance and doubles the search distance until enough photons are found or the distance is greater than MaxPhotonSearchDistance.
 * @return - the furthest found photon's distance squared from SearchPosition, unless the start and max search distances are the same,
 *		in which case FMath::Square(MaxPhotonSearchDistance) will be returned.
 */
float FStaticLightingSystem::FindNearbyPhotonsIterative(
	const FPhotonOctree& PhotonMap, 
	const FVector4f& SearchPosition, 
	const FVector4f& SearchNormal, 
	int32 NumPhotonsToFind,
	float StartPhotonSearchDistance, 
	float MaxPhotonSearchDistance,
	bool bDebugSearchResults,
	bool bDebugSearchProcess,
	TArray<FPhoton>& FoundPhotons,
	FFindNearbyPhotonStats& SearchStats) const
{
	FPlatformAtomics::InterlockedIncrement(&Stats.NumPhotonGathers);
	SearchStats.NumIterativePhotonMapSearches++;
	// Only enforce the search number if the start and max distances are not equal
	const bool bEnforceSearchNumber = !FMath::IsNearlyEqual(StartPhotonSearchDistance, MaxPhotonSearchDistance);
	float SearchDistance = StartPhotonSearchDistance;
	float FurthestPhotonDistanceSquared = 0.0f;

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugSearchProcess)
	{
		// Verify that only one search is debugged
		// This will not always catch multiple searches due to re-entrance by multiple threads
		checkSlow(StartupDebugOutput.GatheredPhotonNodes.Num() == 0);
	}
#endif

	// Continue searching until we have found enough photons or have exceeded the max search distance
	while (FoundPhotons.Num() < NumPhotonsToFind && SearchDistance <= MaxPhotonSearchDistance)
	{
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
		if (bDebugSearchProcess)
		{
			// Only capture the nodes visited on the last iteration
			StartupDebugOutput.GatheredPhotonNodes.Empty();
		}
#endif
		FurthestPhotonDistanceSquared = SearchDistance * SearchDistance;
		// Presize to avoid unnecessary allocations
		// Empty last search iteration's results so we don't have to use AddUniqueItem
		FoundPhotons.Empty(FMath::Max(NumPhotonsToFind, FoundPhotons.Num() + FoundPhotons.GetSlack()));
		const FBox3f SearchBox = FBox3f::BuildAABB(SearchPosition, FVector4f(SearchDistance, SearchDistance, SearchDistance));
		for (FPhotonOctree::TConstIterator<TInlineAllocator<600>> OctreeIt(PhotonMap); OctreeIt.HasPendingNodes(); OctreeIt.Advance())
		{
			const FPhotonOctree::FNode& CurrentNode = OctreeIt.GetCurrentNode();
			const FOctreeNodeContext& CurrentContext = OctreeIt.GetCurrentContext();
			{
				LIGHTINGSTAT(FScopedRDTSCTimer PushingChildrenTimer(SearchStats.PushingOctreeChildrenThreadTime));
				// Push children onto the iterator stack if they intersect the query box
				if (!CurrentNode.IsLeaf())
				{
					FOREACH_OCTREE_CHILD_NODE(ChildRef)
					{
						if (CurrentNode.HasChild(ChildRef))
						{
							const FOctreeNodeContext ChildContext = CurrentContext.GetChildContext(ChildRef);
							if (ChildContext.Bounds.GetBox().Intersect(SearchBox))
							{
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
								if (bDebugSearchProcess)
								{
									StartupDebugOutput.GatheredPhotonNodes.Add(FDebugOctreeNode(ChildContext.Bounds.Center, ChildContext.Bounds.Extent));
								}
#endif
								OctreeIt.PushChild(ChildRef);
							}
						}
					}
				}
			}

			LIGHTINGSTAT(FScopedRDTSCTimer ProcessingElementsTimer(SearchStats.ProcessingOctreeElementsThreadTime));
			// Iterate over all photons in the nodes intersecting the query box
			for (FPhotonOctree::ElementConstIt MeshIt(CurrentNode.GetConstElementIt()); MeshIt; ++MeshIt)
			{
				const FPhotonElement& PhotonElement = *MeshIt;
				const float DistanceSquared = (PhotonElement.Photon.GetPosition() - SearchPosition).SizeSquared3();
				const float CosNormalTheta = Dot3(SearchNormal, PhotonElement.Photon.GetSurfaceNormal());
				const float CosIncidentDirectionTheta = Dot3(SearchNormal, PhotonElement.Photon.GetIncidentDirection());
				// Only searching for photons closer than the max distance
				if (DistanceSquared < FurthestPhotonDistanceSquared
					// Whose normal is within the specified angle from the search normal
					&& CosNormalTheta > PhotonMappingSettings.PhotonSearchAngleThreshold
					// And whose incident direction is in the same hemisphere as the search normal.
					&& CosIncidentDirectionTheta > 0.0f)
				{
					if (bEnforceSearchNumber)
					{
						if (FoundPhotons.Num() < NumPhotonsToFind)
						{
							FoundPhotons.Add(PhotonElement.Photon);
						}
						else
						{
							checkSlow(FoundPhotons.Num() == NumPhotonsToFind);
							float FurthestFoundPhotonDistSq = 0;
							int32 FurthestFoundPhotonIndex = -1;

							// Find the furthest photon
							// This could be accelerated with a heap instead of doing an O(n) search
							LIGHTINGSTAT(FScopedRDTSCTimer FindingFurthestTimer(SearchStats.FindingFurthestPhotonThreadTime));
							for (int32 PhotonIndex = 0; PhotonIndex < FoundPhotons.Num(); PhotonIndex++)
							{
								const float CurrentDistanceSquared = (FoundPhotons[PhotonIndex].GetPosition() - SearchPosition).SizeSquared3();
								if (CurrentDistanceSquared > FurthestFoundPhotonDistSq)
								{
									FurthestFoundPhotonDistSq = CurrentDistanceSquared;
									FurthestFoundPhotonIndex = PhotonIndex;
								}
							}
							checkSlow(FurthestFoundPhotonIndex >= 0);
							FurthestPhotonDistanceSquared = FurthestFoundPhotonDistSq;
							if (DistanceSquared < FurthestFoundPhotonDistSq)
							{
								// Replace the furthest photon with the new photon since the new photon is closer
								FoundPhotons[FurthestFoundPhotonIndex] = PhotonElement.Photon;
							}
						}
					}
					else
					{
						FoundPhotons.Add(PhotonElement.Photon);
					}
				}
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugSearchProcess)
				{
					StartupDebugOutput.IrradiancePhotons.Add(FDebugPhoton(PhotonElement.Photon.GetId(), PhotonElement.Photon.GetPosition(), PhotonElement.Photon.GetIncidentDirection(), PhotonElement.Photon.GetSurfaceNormal()));
				}
#endif
			}
		}
		// Double the search radius for each iteration
		SearchDistance *= 2.0f;
		SearchStats.NumSearchIterations++;
	}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugSearchProcess || bDebugSearchResults && PhotonMappingSettings.bVisualizePhotonGathers)
	{
		// Assuming that only importance photons are debugged and enforce the search number
		if (bDebugSearchResults && bEnforceSearchNumber)
		{
			for (int32 i = 0; i < FoundPhotons.Num(); i++)
			{
				StartupDebugOutput.GatheredImportancePhotons.Add(FDebugPhoton(FoundPhotons[i].GetId(), FoundPhotons[i].GetPosition(), FoundPhotons[i].GetIncidentDirection(), FoundPhotons[i].GetSurfaceNormal()));
			}
		}
		else
		{
			for (int32 i = 0; i < FoundPhotons.Num(); i++)
			{
				StartupDebugOutput.GatheredPhotons.Add(FDebugPhoton(FoundPhotons[i].GetId(), FoundPhotons[i].GetPosition(), FoundPhotons[i].GetIncidentDirection(), FoundPhotons[i].GetSurfaceNormal()));
			}
		}
	}
#endif
	return FurthestPhotonDistanceSquared;
}

/** 
 * Searches a volume segment map for photons.  Can be used at any point in space, not just on surfaces. 
 */
float FStaticLightingSystem::FindNearbyPhotonsInVolumeIterative(
	const FPhotonSegmentOctree& PhotonSegmentMap, 
	const FVector4f& SearchPosition, 
	int32 NumPhotonsToFind,
	float StartPhotonSearchDistance, 
	float MaxPhotonSearchDistance,
	TArray<FPhotonSegmentElement>& FoundPhotonSegments,
	bool bDebugThisLookup) const
{
	FPlatformAtomics::InterlockedIncrement(&Stats.NumPhotonGathers);
	float SearchDistance = StartPhotonSearchDistance;
	float FurthestPhotonDistanceSquared = 0.0f;

	// Continue searching until we have found enough photons or have exceeded the max search distance
	while (FoundPhotonSegments.Num() < NumPhotonsToFind && SearchDistance <= MaxPhotonSearchDistance)
	{
		FurthestPhotonDistanceSquared = SearchDistance * SearchDistance;
		// Presize to avoid unnecessary allocations
		// Empty last search iteration's results so we don't have to use AddUniqueItem
		FoundPhotonSegments.Empty(FMath::Max(NumPhotonsToFind, FoundPhotonSegments.Num() + FoundPhotonSegments.GetSlack()));
		const FBox3f SearchBox = FBox3f::BuildAABB(SearchPosition, FVector4f(SearchDistance, SearchDistance, SearchDistance));
		for (FPhotonSegmentOctree::TConstIterator<TInlineAllocator<600>> OctreeIt(PhotonSegmentMap); OctreeIt.HasPendingNodes(); OctreeIt.Advance())
		{
			const FPhotonSegmentOctree::FNode& CurrentNode = OctreeIt.GetCurrentNode();
			const FOctreeNodeContext& CurrentContext = OctreeIt.GetCurrentContext();
			{
				// Push children onto the iterator stack if they intersect the query box
				if (!CurrentNode.IsLeaf())
				{
					FOREACH_OCTREE_CHILD_NODE(ChildRef)
					{
						if (CurrentNode.HasChild(ChildRef))
						{
							const FOctreeNodeContext ChildContext = CurrentContext.GetChildContext(ChildRef);
							if (ChildContext.Bounds.GetBox().Intersect(SearchBox))
							{
								OctreeIt.PushChild(ChildRef);
							}
						}
					}
				}
			}

			// Iterate over all photons in the nodes intersecting the query box
			for (FPhotonSegmentOctree::ElementConstIt MeshIt(CurrentNode.GetConstElementIt()); MeshIt; ++MeshIt)
			{
				const FPhotonSegmentElement& PhotonSegmentElement = *MeshIt;
				const float SegmentDistanceSquared = PhotonSegmentElement.ComputeSquaredDistanceToPoint(SearchPosition);

				// Only searching for photons closer than the max distance
				if (SegmentDistanceSquared < FurthestPhotonDistanceSquared)
				{
					bool bNewPhoton = true;

					for (int32 i = 0; i < FoundPhotonSegments.Num(); i++)
					{
						if (FoundPhotonSegments[i].Photon == PhotonSegmentElement.Photon)
						{
							bNewPhoton = false;
							break;
						}
					}

					if (bNewPhoton)
					{
						if (FoundPhotonSegments.Num() < NumPhotonsToFind)
						{
							FoundPhotonSegments.Add(PhotonSegmentElement);
						}
						else
						{
							checkSlow(FoundPhotonSegments.Num() == NumPhotonsToFind);
							float FurthestFoundPhotonDistSq = 0;
							int32 FurthestFoundPhotonIndex = -1;

							// Find the furthest photon
							for (int32 PhotonIndex = 0; PhotonIndex < FoundPhotonSegments.Num(); PhotonIndex++)
							{
								const float OtherSegmentDistanceSquared = FoundPhotonSegments[PhotonIndex].ComputeSquaredDistanceToPoint(SearchPosition);

								if (OtherSegmentDistanceSquared > FurthestFoundPhotonDistSq)
								{
									FurthestFoundPhotonDistSq = OtherSegmentDistanceSquared;
									FurthestFoundPhotonIndex = PhotonIndex;
								}
							}
							checkSlow(FurthestFoundPhotonIndex >= 0);
							FurthestPhotonDistanceSquared = FurthestFoundPhotonDistSq;
							if (SegmentDistanceSquared < FurthestFoundPhotonDistSq)
							{
								// Replace the furthest photon with the new photon since the new photon is closer
								FoundPhotonSegments[FurthestFoundPhotonIndex] = PhotonSegmentElement;
							}
						}
					}
				}
			}
		}
		// Double the search radius for each iteration
		SearchDistance *= 2.0f;
	}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisLookup && PhotonMappingSettings.bVisualizePhotonGathers)
	{
		for (int32 i = 0; i < FoundPhotonSegments.Num(); i++)
		{
			const FPhoton& Photon = *FoundPhotonSegments[i].Photon;
			StartupDebugOutput.GatheredImportancePhotons.Add(FDebugPhoton(Photon.GetId(), Photon.GetPosition(), Photon.GetIncidentDirection(), Photon.GetSurfaceNormal()));
		}
	}
#endif

	return FurthestPhotonDistanceSquared;
}

struct FOctreeNodeRefAndDistance
{
	FOctreeChildNodeRef NodeRef;
	float DistanceSquared;

	FORCEINLINE FOctreeNodeRefAndDistance(FOctreeChildNodeRef InNodeRef, float InDistanceSquared) :
		NodeRef(InNodeRef),
		DistanceSquared(InDistanceSquared)
	{}
};

/** 
 * Searches the given photon map for the nearest NumPhotonsToFind photons to SearchPosition by sorting octree nodes nearest to furthest.
 * @return - the furthest found photon's distance squared from SearchPosition.
 */
float FStaticLightingSystem::FindNearbyPhotonsSorted(
	const FPhotonOctree& PhotonMap, 
	const FVector4f& SearchPosition, 
	const FVector4f& SearchNormal, 
	int32 NumPhotonsToFind, 
	float MaxPhotonSearchDistance,
	bool bDebugSearchResults,
	bool bDebugSearchProcess,
	TArray<FPhoton>& FoundPhotons,
	FFindNearbyPhotonStats& SearchStats) const
{
	FPlatformAtomics::InterlockedIncrement(&Stats.NumPhotonGathers);
	float FurthestPhotonDistanceSquared = MaxPhotonSearchDistance * MaxPhotonSearchDistance;

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugSearchProcess)
	{
		// Verify that only one search is debugged
		// This will not always catch multiple searches due to re-entrance by multiple threads
		checkSlow(StartupDebugOutput.GatheredPhotonNodes.Num() == 0);
	}
#endif

	// Presize to avoid unnecessary allocations
	FoundPhotons.Empty(FMath::Max(NumPhotonsToFind, FoundPhotons.Num() + FoundPhotons.GetSlack()));
	for (FPhotonOctree::TConstIterator<TInlineAllocator<600>> OctreeIt(PhotonMap); OctreeIt.HasPendingNodes(); OctreeIt.Advance())
	{
		SearchStats.NumOctreeNodesVisited++;
		const FOctreeNodeContext& CurrentContext = OctreeIt.GetCurrentContext();
		const float ClosestNodePointDistanceSquared = (CurrentContext.Bounds.Center - SearchPosition).SizeSquared3() - CurrentContext.Bounds.Extent.SizeSquared3();
		if (ClosestNodePointDistanceSquared > FurthestPhotonDistanceSquared && !CurrentContext.Bounds.GetBox().IsInside(SearchPosition))
		{
			// Skip nodes that don't contain the search position and whose closest point is further than FurthestPhotonDistanceSquared
			// This check was already done before pushing the node, but FurthestPhotonDistanceSquared may have been reduced since then
			//@todo - can we skip all remaining nodes too?  Nodes are pushed from closest to furthest.
			continue;
		}
		
		const FPhotonOctree::FNode& CurrentNode = OctreeIt.GetCurrentNode();
		{
			LIGHTINGSTAT(FScopedRDTSCTimer ProcessingElementsTimer(SearchStats.ProcessingOctreeElementsThreadTime));
			// Iterate over all photons in the nodes intersecting the query box
			for (FPhotonOctree::ElementConstIt MeshIt(CurrentNode.GetConstElementIt()); MeshIt; ++MeshIt)
			{
				SearchStats.NumElementsTested++;
				const FPhotonElement& PhotonElement = *MeshIt;
				const float DistanceSquared = (PhotonElement.Photon.GetPosition() - SearchPosition).SizeSquared3();
				const float CosNormalTheta = Dot3(SearchNormal, PhotonElement.Photon.GetSurfaceNormal());
				const float CosIncidentDirectionTheta = Dot3(SearchNormal, PhotonElement.Photon.GetIncidentDirection());
				// Only searching for photons closer than the max distance
				if (DistanceSquared < FurthestPhotonDistanceSquared
					// Whose normal is within the specified angle from the search normal
					&& CosNormalTheta > PhotonMappingSettings.PhotonSearchAngleThreshold
					// And whose incident direction is in the same hemisphere as the search normal.
					&& CosIncidentDirectionTheta > 0.0f)
				{
					SearchStats.NumElementsAccepted++;
					if (FoundPhotons.Num() < NumPhotonsToFind)
					{
						FoundPhotons.Add(PhotonElement.Photon);
					}
					else
					{
						checkSlow(FoundPhotons.Num() == NumPhotonsToFind);
						float FurthestFoundPhotonDistSq = 0;
						int32 FurthestFoundPhotonIndex = -1;

						// Find the furthest photon
						// This could be accelerated with a heap instead of doing an O(n) search
						LIGHTINGSTAT(FScopedRDTSCTimer FindingFurthestTimer(SearchStats.FindingFurthestPhotonThreadTime));
						for (int32 PhotonIndex = FoundPhotons.Num() - 1; PhotonIndex >= 0; PhotonIndex--)
						{
							const float CurrentDistanceSquared = (FoundPhotons[PhotonIndex].GetPosition() - SearchPosition).SizeSquared3();
							if (CurrentDistanceSquared > FurthestFoundPhotonDistSq)
							{
								FurthestFoundPhotonDistSq = CurrentDistanceSquared;
								FurthestFoundPhotonIndex = PhotonIndex;
							}
						}
						
						checkSlow(FurthestFoundPhotonIndex >= 0);
						FurthestPhotonDistanceSquared = FurthestFoundPhotonDistSq;
						if (DistanceSquared < FurthestFoundPhotonDistSq)
						{
							// Replace the furthest photon with the new photon since the new photon is closer
							FoundPhotons[FurthestFoundPhotonIndex] = PhotonElement.Photon;
						}
					}
				}
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugSearchProcess)
				{
					StartupDebugOutput.IrradiancePhotons.Add(FDebugPhoton(PhotonElement.Photon.GetId(), PhotonElement.Photon.GetPosition(), PhotonElement.Photon.GetIncidentDirection(), PhotonElement.Photon.GetSurfaceNormal()));
				}
#endif
			}
		}
		
		LIGHTINGSTAT(FScopedRDTSCTimer PushingChildrenTimer(SearchStats.PushingOctreeChildrenThreadTime));
		// Push children onto the iterator stack if they intersect the query box
		if (!CurrentNode.IsLeaf())
		{
			TArray<FOctreeNodeRefAndDistance, TInlineAllocator<8> > ChildrenInRange;
			bool bAllNodesZeroDistance = true;
			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if (CurrentNode.HasChild(ChildRef))
				{
					SearchStats.NumOctreeNodesTested++;
					const FOctreeNodeContext ChildContext = CurrentContext.GetChildContext(ChildRef);
					const bool ChildContainsSearchPosition = ChildContext.Bounds.GetBox().IsInside(SearchPosition);
					const float ClosestChildPointDistanceSquared = ChildContainsSearchPosition ? 
						0 : 
						FMath::Max((ChildContext.Bounds.Center - SearchPosition).SizeSquared3() - ChildContext.Bounds.Extent.SizeSquared3(), 0.0f);

					// Only visit nodes that either contain the search position or whose closest point is closer than FurthestPhotonDistanceSquared
					if (ClosestChildPointDistanceSquared <= FurthestPhotonDistanceSquared)
					{
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
						if (bDebugSearchProcess)
						{
							StartupDebugOutput.GatheredPhotonNodes.Add(FDebugOctreeNode(ChildContext.Bounds.Center, ChildContext.Bounds.Extent));
						}
#endif
						bAllNodesZeroDistance = bAllNodesZeroDistance && ClosestChildPointDistanceSquared < DELTA;
						ChildrenInRange.Add(FOctreeNodeRefAndDistance(ChildRef, ClosestChildPointDistanceSquared));
					}
				}
			}

			if (!bAllNodesZeroDistance && ChildrenInRange.Num() > 1)
			{
				// Used to sort FOctreeNodeRefAndDistances from smallest DistanceSquared to largest.
				struct FCompareNodeDistance
				{
					FORCEINLINE bool operator()(const FOctreeNodeRefAndDistance& A, const FOctreeNodeRefAndDistance& B) const
					{
						return A.DistanceSquared < B.DistanceSquared;
					}
				};
				// Sort the nodes from closest to furthest
				ChildrenInRange.Sort(FCompareNodeDistance());
			}

			for (int32 NodeIndex = 0; NodeIndex < ChildrenInRange.Num(); NodeIndex++)
			{
				OctreeIt.PushChild(ChildrenInRange[NodeIndex].NodeRef);
			}
		}
	}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugSearchProcess || bDebugSearchResults && PhotonMappingSettings.bVisualizePhotonGathers)
	{
		// Assuming that only importance photons are debugged
		if (bDebugSearchResults)
		{
			for (int32 i = 0; i < FoundPhotons.Num(); i++)
			{
				StartupDebugOutput.GatheredImportancePhotons.Add(FDebugPhoton(FoundPhotons[i].GetId(), FoundPhotons[i].GetPosition(), FoundPhotons[i].GetIncidentDirection(), FoundPhotons[i].GetSurfaceNormal()));
			}
		}
		else
		{
			for (int32 i = 0; i < FoundPhotons.Num(); i++)
			{
				StartupDebugOutput.GatheredPhotons.Add(FDebugPhoton(FoundPhotons[i].GetId(), FoundPhotons[i].GetPosition(), FoundPhotons[i].GetIncidentDirection(), FoundPhotons[i].GetSurfaceNormal()));
			}
		}
	}
#endif
	return FurthestPhotonDistanceSquared;
}

/** Comparison class to sort an array of photons into ascending order of distance to ComparePosition. */
class FCompareNearestIrradiancePhotons
{
public:
	FCompareNearestIrradiancePhotons(const FVector4f& InPosition) :
		ComparePosition(InPosition)
	{}

	inline bool operator()(const FIrradiancePhoton& A, const FIrradiancePhoton& B) const
	{
		const float DistanceSquaredA = (A.GetPosition() - ComparePosition).SizeSquared3();
		const float DistanceSquaredB = (B.GetPosition() - ComparePosition).SizeSquared3();
		return DistanceSquaredA < DistanceSquaredB;
	}

private:
	const FVector4f ComparePosition;
};

/** Finds the nearest irradiance photon, if one exists. */
FIrradiancePhoton* FStaticLightingSystem::FindNearestIrradiancePhoton(
	const FMinimalStaticLightingVertex& Vertex, 
	FStaticLightingMappingContext& MappingContext, 
	TArray<FIrradiancePhoton*>& TempIrradiancePhotons,
	bool bVisibleOnly, 
	bool bDebugThisLookup) const
{
	MappingContext.Stats.NumIrradiancePhotonMapSearches++;

	FIrradiancePhoton* ClosestPhoton = NULL;
	// Traverse the octree with the maximum distance required
	const float SearchDistance = FMath::Max(PhotonMappingSettings.DirectPhotonSearchDistance, PhotonMappingSettings.IndirectPhotonSearchDistance);
	float ClosestDistanceSquared = FMath::Square(SearchDistance);

	// Empty the temporary array without reallocating
	TempIrradiancePhotons.Empty(TempIrradiancePhotons.Num() + TempIrradiancePhotons.GetSlack());
	const FBox3f SearchBox = FBox3f::BuildAABB(Vertex.WorldPosition, FVector4f(SearchDistance, SearchDistance, SearchDistance));
	{
		LIGHTINGSTAT(FScopedRDTSCTimer OctreeTraversal(MappingContext.Stats.IrradiancePhotonOctreeTraversalTime));
		for (FIrradiancePhotonOctree::TConstIterator<> OctreeIt(IrradiancePhotonMap); OctreeIt.HasPendingNodes(); OctreeIt.Advance())
		{
			const FIrradiancePhotonOctree::FNode& CurrentNode = OctreeIt.GetCurrentNode();
			const FOctreeNodeContext& CurrentContext = OctreeIt.GetCurrentContext();

			// Push children onto the iterator stack if they intersect the query box
			if (!CurrentNode.IsLeaf())
			{
				FOREACH_OCTREE_CHILD_NODE(ChildRef)
				{
					if (CurrentNode.HasChild(ChildRef))
					{
						const FOctreeNodeContext ChildContext = CurrentContext.GetChildContext(ChildRef);
						if (ChildContext.Bounds.GetBox().Intersect(SearchBox))
						{
							OctreeIt.PushChild(ChildRef);
						}
					}
				}
			}

			// Iterate over all photons in the nodes intersecting the query box
			for (FIrradiancePhotonOctree::ElementIt MeshIt(CurrentNode.GetElementIt()); MeshIt; ++MeshIt)
			{
				FIrradiancePhotonElement& PhotonElement = *MeshIt;
				FIrradiancePhoton& CurrentPhoton = PhotonElement.GetPhoton();
				const FVector4f PhotonToVertexVector = Vertex.WorldPosition - CurrentPhoton.GetPosition();
				const float DistanceSquared = PhotonToVertexVector.SizeSquared3();
				const float CosTheta = Dot3(Vertex.WorldTangentZ, CurrentPhoton.GetSurfaceNormal());

				// Only searching for irradiance photons with normals similar to the search normal
				if (CosTheta > PhotonMappingSettings.PhotonSearchAngleThreshold
					// And closer to the search position than the max search distance.
					&& ((CurrentPhoton.HasDirectContribution() && (DistanceSquared < FMath::Square(PhotonMappingSettings.DirectPhotonSearchDistance)))
					|| (!CurrentPhoton.HasDirectContribution() && (DistanceSquared < FMath::Square(PhotonMappingSettings.IndirectPhotonSearchDistance)))))
				{
					// Only accept irradiance photons within an angle of the plane defined by the vertex normal
					// This avoids expensive visibility traces to photons that are probably not on the same surface
					const float DirectionDotNormal = Dot3(CurrentPhoton.GetSurfaceNormal(), PhotonToVertexVector.GetSafeNormal());
					if (FMath::Abs(DirectionDotNormal) < PhotonMappingSettings.MinCosIrradiancePhotonSearchCone)
					{
						if (bVisibleOnly)
						{
							// Store the photon for later, which is faster than tracing a ray here since this may not be the closest photon
							TempIrradiancePhotons.Add(&CurrentPhoton);
						}
						else if (DistanceSquared < ClosestDistanceSquared)
						{
							// Only accept the closest photon if visibility is not required
							ClosestPhoton = &CurrentPhoton;
							ClosestDistanceSquared = DistanceSquared;
						}
					}
				}
			}
		}
	}

	if (bVisibleOnly)
	{
		// Sort the photons so the closest photon is in the beginning of the array
		FCompareNearestIrradiancePhotons CompareClassInstance(Vertex.WorldPosition);
		TempIrradiancePhotons.Sort(CompareClassInstance);

		// Trace a ray from the vertex to each irradiance photon until a visible one is found, starting with the closest
		for (int32 PhotonIndex = 0; PhotonIndex < TempIrradiancePhotons.Num(); PhotonIndex++)
		{
			FIrradiancePhoton* CurrentPhoton = TempIrradiancePhotons[PhotonIndex];
			const FVector4f VertexToPhoton = CurrentPhoton->GetPosition() - Vertex.WorldPosition;
			const FLightRay VertexToPhotonRay(
				Vertex.WorldPosition + VertexToPhoton.GetSafeNormal() * SceneConstants.VisibilityRayOffsetDistance + Vertex.WorldTangentZ * SceneConstants.VisibilityNormalOffsetDistance,
				CurrentPhoton->GetPosition() + CurrentPhoton->GetSurfaceNormal() * SceneConstants.VisibilityNormalOffsetDistance,
				NULL,
				NULL
				);

			MappingContext.Stats.NumIrradiancePhotonSearchRays++;
			const float PreviousShadowTraceTime = MappingContext.RayCache.BooleanRayTraceTime;
			// Check the line segment for intersection with the static lighting meshes.
			FLightRayIntersection Intersection;
			AggregateMesh->IntersectLightRay(VertexToPhotonRay, false, false, false, MappingContext.RayCache, Intersection);
			MappingContext.Stats.IrradiancePhotonSearchRayTime += MappingContext.RayCache.BooleanRayTraceTime - PreviousShadowTraceTime;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisLookup && PhotonMappingSettings.bVisualizePhotonGathers)
			{
				FDebugStaticLightingRay DebugRay(VertexToPhotonRay.Start, VertexToPhotonRay.End, Intersection.bIntersects);
				if (Intersection.bIntersects)
				{
					DebugRay.End = Intersection.IntersectionVertex.WorldPosition;
				}
				StartupDebugOutput.ShadowRays.Add(DebugRay);
			}
#endif
			if (!Intersection.bIntersects)
			{
				// Break on the first visible photon
				ClosestPhoton = CurrentPhoton;
				break;
			}
		}
	}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisLookup && ClosestPhoton != NULL && PhotonMappingSettings.bVisualizePhotonGathers)
	{
		StartupDebugOutput.GatheredPhotons.Add(FDebugPhoton(0, ClosestPhoton->GetPosition(), ClosestPhoton->GetSurfaceNormal(), ClosestPhoton->GetSurfaceNormal()));
	}
#endif

	return ClosestPhoton;
}
	
/** Calculates the irradiance for an irradiance photon */
FLinearColor FStaticLightingSystem::CalculatePhotonIrradiance(
	const FPhotonOctree& PhotonMap,
	int32 NumPhotonsEmitted, 
	int32 NumPhotonsToFind,
	float SearchDistance,
	const FIrradiancePhoton& IrradiancePhoton,
	bool bDebugThisCalculation,
	TArray<FPhoton>& TempFoundPhotons,
	FCalculateIrradiancePhotonStats& OutStats) const
{
	// Empty TempFoundPhotons without causing any allocations / frees
	TempFoundPhotons.Empty(TempFoundPhotons.Num() + TempFoundPhotons.GetSlack());

	const float MaxFoundDistanceSquared = FindNearbyPhotonsSorted(
		PhotonMap, 
		IrradiancePhoton.GetPosition(), 
		IrradiancePhoton.GetSurfaceNormal(), 
		NumPhotonsToFind,
		SearchDistance, 
		false,
		bDebugThisCalculation, 
		TempFoundPhotons, 
		OutStats);

	FLinearColor PhotonIrradiance(FLinearColor::Black);

	if (TempFoundPhotons.Num() > 0)
	{
		LIGHTINGSTAT(FScopedRDTSCTimer CalculateIrradianceTimer(OutStats.CalculateIrradianceThreadTime));
		const float MaxFoundDistance = FMath::Sqrt(MaxFoundDistanceSquared);
		// Estimate the photon density using a cone filter, from the paper "Global Illumination using Photon Maps"
		const float DiskArea = (float)PI * MaxFoundDistanceSquared;
		const float ConeFilterNormalizeConstant = 1.0f - 2.0f / (3.0f * PhotonMappingSettings.ConeFilterConstant);
		const float ConstantWeight = 1.0f / (ConeFilterNormalizeConstant * NumPhotonsEmitted * DiskArea);
		FCoherentRayCache UnusedRayCache;

		for (int32 PhotonIndex = 0; PhotonIndex < TempFoundPhotons.Num(); PhotonIndex++)
		{
			const FPhoton& CurrentPhoton = TempFoundPhotons[PhotonIndex];

			if (Dot3(IrradiancePhoton.GetSurfaceNormal(), CurrentPhoton.GetIncidentDirection()) > 0.0f)
			{
				float SearchNormalScales[2] = {.1f, .4f};

				bool bPhotonVisible = false;

				// Try to determine visibility to the photon before letting it contribute
				// This helps to prevent leaking through thin walls
				for (int32 SearchIndex = 0; SearchIndex < UE_ARRAY_COUNT(SearchNormalScales) && !bPhotonVisible; SearchIndex++)
				{
					const float NormalOffset = SearchDistance * SearchNormalScales[SearchIndex];

					const FLightRay Ray(
						IrradiancePhoton.GetPosition() + IrradiancePhoton.GetSurfaceNormal() * NormalOffset,
						CurrentPhoton.GetPosition() + CurrentPhoton.GetSurfaceNormal() * NormalOffset,
						NULL,
						NULL
						);

					FLightRayIntersection RayIntersection;
					AggregateMesh->IntersectLightRay(Ray, false, false, false, UnusedRayCache, RayIntersection);

					bPhotonVisible = !RayIntersection.bIntersects;
				}
				
				if (bPhotonVisible)
				{
					const float PhotonDistance = (CurrentPhoton.GetPosition() - IrradiancePhoton.GetPosition()).Size3();
					const float ConeWeight = FMath::Max(1.0f - PhotonDistance / (PhotonMappingSettings.ConeFilterConstant * MaxFoundDistance), 0.0f);
					PhotonIrradiance += CurrentPhoton.GetPower() * ConeWeight * ConstantWeight;
				}
			}
		}
	}
	return PhotonIrradiance;
}

/** Calculates incident radiance at a vertex from the given photon map. */
FGatheredLightSample FStaticLightingSystem::CalculatePhotonIncidentRadiance(
	const FPhotonOctree& PhotonMap,
	int32 NumPhotonsEmitted, 
	float SearchDistance,
	const FStaticLightingVertex& Vertex,
	bool bDebugThisDensityEstimation) const
{
	TArray<FPhoton> FoundPhotons;
	FFindNearbyPhotonStats DummyStats;
	FindNearbyPhotonsIterative(PhotonMap, Vertex.WorldPosition, Vertex.WorldTangentZ, 1, SearchDistance, SearchDistance, bDebugThisDensityEstimation, false, FoundPhotons, DummyStats);

	FGatheredLightSample PhotonIncidentRadiance;
	if (FoundPhotons.Num() > 0)
	{
		// Estimate the photon density using a cone filter, from the paper "Global Illumination using Photon Maps"
		const float DiskArea = (float)PI * SearchDistance * SearchDistance;
		const float ConeFilterNormalizeConstant = 1.0f - 2.0f / (3.0f * PhotonMappingSettings.ConeFilterConstant);
		const float ConstantWeight = 1.0f / (ConeFilterNormalizeConstant * NumPhotonsEmitted * DiskArea);
		for (int32 PhotonIndex = 0; PhotonIndex < FoundPhotons.Num(); PhotonIndex++)
		{
			const FPhoton& CurrentPhoton = FoundPhotons[PhotonIndex];
			const FVector4f TangentPathDirection = Vertex.TransformWorldVectorToTangent(CurrentPhoton.GetIncidentDirection());
			if (TangentPathDirection.Z > 0)
			{
				const float PhotonDistance = (CurrentPhoton.GetPosition() - Vertex.WorldPosition).Size3();
				const float ConeWeight = FMath::Max(1.0f - PhotonDistance / (PhotonMappingSettings.ConeFilterConstant * SearchDistance), 0.0f);
				PhotonIncidentRadiance.AddWeighted(FGatheredLightSampleUtil::PointLightWorldSpace<2>(CurrentPhoton.GetPower(), TangentPathDirection, CurrentPhoton.GetIncidentDirection()), ConeWeight * ConstantWeight);
			}
		}
	}
	
	return PhotonIncidentRadiance;
}

/** Calculates exitant radiance at a vertex from the given photon map. */
FLinearColor FStaticLightingSystem::CalculatePhotonExitantRadiance(
	const FPhotonOctree& PhotonMap,
	int32 NumPhotonsEmitted, 
	float SearchDistance,
	const FStaticLightingMesh* Mesh,
	const FMinimalStaticLightingVertex& Vertex,
	int32 ElementIndex,
	const FVector4f& OutgoingDirection,
	bool bDebugThisDensityEstimation) const
{
	TArray<FPhoton> FoundPhotons;
	FFindNearbyPhotonStats DummyStats;
	FindNearbyPhotonsIterative(PhotonMap, Vertex.WorldPosition, Vertex.WorldTangentZ, 1, SearchDistance, SearchDistance, bDebugThisDensityEstimation, false, FoundPhotons, DummyStats);

	FLinearColor AccumulatedRadiance(FLinearColor::Black);
	if (FoundPhotons.Num() > 0)
	{
		// Estimate the photon density using a cone filter, from the paper "Global Illumination using Photon Maps"
		const float DiskArea = (float)PI * SearchDistance * SearchDistance;
		const float ConeFilterNormalizeConstant = 1.0f - 2.0f / (3.0f * PhotonMappingSettings.ConeFilterConstant);
		const float ConstantWeight = 1.0f / (ConeFilterNormalizeConstant * NumPhotonsEmitted * DiskArea);
		for (int32 PhotonIndex = 0; PhotonIndex < FoundPhotons.Num(); PhotonIndex++)
		{
			const FPhoton& CurrentPhoton = FoundPhotons[PhotonIndex];
			if (Dot3(Vertex.WorldTangentZ, CurrentPhoton.GetIncidentDirection()) > 0.0f)
			{
				const float PhotonDistance = (CurrentPhoton.GetPosition() - Vertex.WorldPosition).Size3();
				const float ConeWeight = FMath::Max(1.0f - PhotonDistance / (PhotonMappingSettings.ConeFilterConstant * SearchDistance), 0.0f);
				const FLinearColor BRDF = Mesh->EvaluateBRDF(Vertex, ElementIndex, CurrentPhoton.GetIncidentDirection(), OutgoingDirection);
				AccumulatedRadiance += CurrentPhoton.GetPower() * ConeWeight * ConstantWeight * BRDF;
			}
		}
	}
	return AccumulatedRadiance;
}


}
