// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LightingSystem.h"
#include "MonteCarlo.h"
#include "Raster.h"
#include "HAL/PlatformTime.h"

namespace Lightmass
{

struct FVisibilitySamplePos
{
	/** 
	 * Min and max world space heights of a single sample on a triangle. 
	 * This is necessary because supersampling is used during rasterization.
	 */
	FVector2f HeightRange;
};

struct FCellHeights
{
	/** Last triangle index that rasterized to this cell. */
	uint64 TriangleIndex;
	/** World space X and Y position of this cell. */
	FVector2f Position;
	/** Array of triangle hits on this cell. */
	TArray<FVisibilitySamplePos> HitTriangles;
};

class FCellToHeightsMap
{
public:

	/** Initialization constructor. */
	FCellToHeightsMap(int32 InSizeX,int32 InSizeY):
		Data(InSizeX * InSizeY),
		SizeX(InSizeX),
		SizeY(InSizeY)
	{
		// Clear the map to zero.
		for(int32 Y = 0;Y < SizeY;Y++)
		{
			for(int32 X = 0;X < SizeX;X++)
			{
				FCellHeights& CurrentCell = (*this)(X,Y);
				FMemory::Memzero(&CurrentCell,sizeof(FCellHeights));
				CurrentCell.HitTriangles.Empty(50);
			}
		}
	}

	// Accessors.
	FCellHeights& operator()(int32 X,int32 Y)
	{
		const uint32 TexelIndex = Y * SizeX + X;
		return Data(TexelIndex);
	}
	const FCellHeights& operator()(int32 X,int32 Y) const
	{
		const int32 TexelIndex = Y * SizeX + X;
		return Data(TexelIndex);
	}

	int32 GetSizeX() const { return SizeX; }
	int32 GetSizeY() const { return SizeY; }
	SIZE_T GetAllocatedSize() const { return Data.GetAllocatedSize(); }

private:

	/** The mapping data. */
	TChunkedArray<FCellHeights> Data;

	/** The width of the mapping data. */
	int32 SizeX;

	/** The height of the mapping data. */
	int32 SizeY;
};
	
class FCellPlacementRasterPolicy
{
public:

	typedef FVector4f InterpolantType;

	/** Initialization constructor. */
	FCellPlacementRasterPolicy(
		FCellToHeightsMap& InHeightsMap,
		const FScene& InScene, 
		FStaticLightingSystem& InSystem,
		const FBoxSphereBounds3f& InPrecomputedVisibilityBounds,
		float InCellSize)
		:
		HeightsMap(InHeightsMap),
		Scene(InScene),
		System(InSystem),
		PrecomputedVisibilityBounds(InPrecomputedVisibilityBounds),
		CellSize(InCellSize)
	{
	}

	void SetTriangleIndex(uint64 InTriangleIndex)
	{
		TriangleIndex = InTriangleIndex;
	}

protected:

	// FTriangleRasterizer policy interface.

	int32 GetMinX() const { return 0; }
	int32 GetMaxX() const { return HeightsMap.GetSizeX(); }
	int32 GetMinY() const { return 0; }
	int32 GetMaxY() const { return HeightsMap.GetSizeY(); }

	void ProcessPixel(int32 X,int32 Y,const InterpolantType& Interpolant,bool BackFacing);

private:

	uint64 TriangleIndex;
	FCellToHeightsMap& HeightsMap;
	const FScene& Scene;
	FStaticLightingSystem& System;
	FBoxSphereBounds3f PrecomputedVisibilityBounds;
	float CellSize;
};

void FCellPlacementRasterPolicy::ProcessPixel(int32 X,int32 Y,const InterpolantType& WorldPosition,bool BackFacing)
{
	if (Scene.IsPointInVisibilityVolume(WorldPosition))
	{
		FCellHeights& Cell = HeightsMap(X, Y);

		if (Cell.TriangleIndex != TriangleIndex)
		{
			// If this is the first hit on this cell from the current triangle, add a new sample
			FVisibilitySamplePos Sample;
			const FVector3f GridPosition = PrecomputedVisibilityBounds.Origin - PrecomputedVisibilityBounds.BoxExtent + FVector3f(X, Y, 0) * CellSize;
			Sample.HeightRange = FVector2f(WorldPosition.Z, WorldPosition.Z);
			Cell.HitTriangles.Add(Sample);
			Cell.Position = FVector2f(GridPosition.X, GridPosition.Y);
			Cell.TriangleIndex = TriangleIndex;
		}
		else
		{
			// If this is not the first hit on this cell from the current triangle, expand the sample min and max
			FVisibilitySamplePos& TriangleEntry = Cell.HitTriangles.Last();
			TriangleEntry.HeightRange.X = FMath::Min(TriangleEntry.HeightRange.X, WorldPosition.Z);
			TriangleEntry.HeightRange.Y = FMath::Max(TriangleEntry.HeightRange.Y, WorldPosition.Z);
		}
	}
}

int32 FStaticLightingSystem::GetGroupCellIndex(FVector3f BoxCenter) const
{
	const FVector3f GridPosition = FVector3f(GroupVisibilityGridSizeXY, GroupVisibilityGridSizeXY, GroupVisibilityGridSizeZ) * (BoxCenter - VisibilityGridBounds.Min) / (VisibilityGridBounds.Max - VisibilityGridBounds.Min);
	const int32 GridX = FMath::TruncToInt(GridPosition.X);
	const int32 GridY = FMath::TruncToInt(GridPosition.Y);
	const int32 GridZ = FMath::TruncToInt(GridPosition.Z);
	const int32 CellIndex = GridZ * GroupVisibilityGridSizeXY * GroupVisibilityGridSizeZ + GridY * GroupVisibilityGridSizeXY + GridX;
	
	if (GridX > 0 && GridX < GroupVisibilityGridSizeXY 
		&& GridY > 0 && GridY < GroupVisibilityGridSizeXY 
		&& GridZ > 0 && GridZ < GroupVisibilityGridSizeZ)
	{
		return CellIndex;
	}
	else
	{
		return -1;
	}
}

class FVisibilityMeshSortInfo
{
public:
	float Distance;
	int32 Index;
	FBox3f Bounds;

	FVisibilityMeshSortInfo() :
		Bounds(FBox3f(ForceInit))
	{}
};

/** Determines visibility cell placement, called once at startup. */
void FStaticLightingSystem::SetupPrecomputedVisibility()
{
	const double StartTime = FPlatformTime::Seconds();
	FLMRandomStream RandomStream(0);

	const FBoxSphereBounds3f PrecomputedVisibilityBounds = Scene.GetVisibilityVolumeBounds();
	const FVector4f VolumeSizes = PrecomputedVisibilityBounds.BoxExtent * 2.0f / PrecomputedVisibilitySettings.CellSize;
	const int32 SizeX = FMath::TruncToInt(VolumeSizes.X + DELTA) + 1;
	const int32 SizeY = FMath::TruncToInt(VolumeSizes.Y + DELTA) + 1;

	if (!PrecomputedVisibilitySettings.bPlaceCellsOnlyAlongCameraTracks)
	{
		FCellToHeightsMap HeightsMap(SizeX, SizeY);
		FTriangleRasterizer<FCellPlacementRasterPolicy> Rasterizer(
			FCellPlacementRasterPolicy(
			HeightsMap,
			Scene, 
			*this,
			PrecomputedVisibilityBounds,
			PrecomputedVisibilitySettings.CellSize));

		check(Meshes.Num() == AllMappings.Num());

		uint64 NextTriangleIndex = 1;

		// Rasterize the scene to determine potential cell heights
		for (int32 MappingIndex = 0; MappingIndex < AllMappings.Num(); MappingIndex++)
		{
			const FStaticLightingMapping* CurrentMapping = AllMappings[MappingIndex];
			const FStaticLightingMesh* CurrentMesh = CurrentMapping->Mesh;

			const uint32 GeoMeshLODIndex = CurrentMesh->GetLODIndices() & 0xFFFF;
			const uint32 GeoHLODTreeIndex = (CurrentMesh->GetLODIndices() & 0xFFFF0000) >> 16;
			const uint32 GeoHLODRange = CurrentMesh->GetHLODRange();
			const uint32 GeoHLODRangeStart = GeoHLODRange & 0xFFFF;
			const uint32 GeoHLODRangeEnd = (GeoHLODRange & 0xFFFF0000) >> 16;

			bool bMeshBelongsToLOD0 = GeoMeshLODIndex == 0;

			if (GeoHLODTreeIndex > 0)
			{
				bMeshBelongsToLOD0 = GeoHLODRangeStart == GeoHLODRangeEnd;
			}

			// Only process meshes whose bounding box intersects a PVS volume
			if (Scene.DoesBoxIntersectVisibilityVolume(CurrentMesh->BoundingBox) && bMeshBelongsToLOD0)
			{
				// Whether mesh wants to be fully opaque for visibility step
				const bool bOpaqueMesh = CurrentMesh->IsAlwaysOpaqueForVisibility();

				// Rasterize all triangles in the mesh
				for (int32 TriangleIndex = 0; TriangleIndex < CurrentMesh->NumTriangles; TriangleIndex++)
				{
					FStaticLightingVertex Vertices[3];
					int32 ElementIndex;
					CurrentMesh->GetTriangle(TriangleIndex, Vertices[0], Vertices[1], Vertices[2], ElementIndex);

					// Only place cells on opaque surfaces if requested, which can save some memory for foliage maps
					if (!PrecomputedVisibilitySettings.bPlaceCellsOnOpaqueOnly || bOpaqueMesh
						|| (!CurrentMesh->IsMasked(ElementIndex) && !CurrentMesh->IsTranslucent(ElementIndex)))
					{
						FVector2f XYPositions[3];
						for (int32 VertIndex = 0; VertIndex < 3; VertIndex++)
						{
							// Transform world space positions from [PrecomputedVisibilityBounds.Origin - PrecomputedVisibilityBounds.BoxExtent, PrecomputedVisibilityBounds.Origin + PrecomputedVisibilityBounds.BoxExtent] into [0,1]
							const FVector4f TransformedPosition = (Vertices[VertIndex].WorldPosition - PrecomputedVisibilityBounds.Origin + PrecomputedVisibilityBounds.BoxExtent) / (2.0f * PrecomputedVisibilityBounds.BoxExtent);
							// Project positions onto the XY plane
							XYPositions[VertIndex] = FVector2f(TransformedPosition.X * SizeX, TransformedPosition.Y * SizeY);
						}

						const FVector4f TriangleNormal = (Vertices[2].WorldPosition - Vertices[0].WorldPosition) ^ (Vertices[1].WorldPosition - Vertices[0].WorldPosition);

						// Only rasterize upward facing triangles
						if (TriangleNormal.Z > 0.0f)
						{
							Rasterizer.SetTriangleIndex(NextTriangleIndex);

							FVector2f SubsamplePositions[9];
							SubsamplePositions[0] = FVector2f(.5f, .5f);
							SubsamplePositions[1] = FVector2f(0, .5f);
							SubsamplePositions[2] = FVector2f(.5f, 0);
							SubsamplePositions[3] = FVector2f(1, .5f);
							SubsamplePositions[4] = FVector2f(.5f, 1);
							SubsamplePositions[5] = FVector2f(1, 1);
							SubsamplePositions[6] = FVector2f(0, 1);
							SubsamplePositions[7] = FVector2f(1, 0);
							SubsamplePositions[8] = FVector2f(0, 0);

							const float EdgePullback = .1f;

							for (int32 SampleIndex = 0; SampleIndex < UE_ARRAY_COUNT(SubsamplePositions); SampleIndex++)
							{
								const FVector2f SamplePosition = SubsamplePositions[SampleIndex] * (1 - 2 * EdgePullback) + FVector2f(EdgePullback, EdgePullback);

								Rasterizer.DrawTriangle(
									Vertices[0].WorldPosition,
									Vertices[1].WorldPosition,
									Vertices[2].WorldPosition,
									XYPositions[0] - SamplePosition,
									XYPositions[1] - SamplePosition,
									XYPositions[2] - SamplePosition,
									false
									);
							}

							NextTriangleIndex++;
						}
					}
				}
			}
		}

		AllPrecomputedVisibilityCells.Empty(SizeX * SizeY * 2);

		TArray<FVector2f> PlacedHeightRanges;
		for (int32 Y = 0; Y < SizeY; Y++)
		{
			for (int32 X = 0; X < SizeX; X++)
			{
				FCellHeights& Cell = HeightsMap(X, Y);
				const FVector2f CurrentPosition = Cell.Position;

				// Sort the heights from smallest to largest
				struct FCompareSampleZ
				{
					FORCEINLINE bool operator()(const FVisibilitySamplePos& A, const FVisibilitySamplePos& B) const
					{
						return A.HeightRange.Y < B.HeightRange.Y;
					}
				};
				Cell.HitTriangles.Sort(FCompareSampleZ());
				float LastSampleHeight = -FLT_MAX;

				PlacedHeightRanges.Reset();

				// Pass 1 - only place cells in the largest holes which are most likely to be where the play area is
				// Place the bottom slightly above the surface, since cells that clip through the floor often have poor occlusion culling
				for (int32 HeightIndex = 0; HeightIndex < Cell.HitTriangles.Num(); HeightIndex++)
				{
					const float CurrentMaxHeight = Cell.HitTriangles[HeightIndex].HeightRange.Y;

					// Place a new cell if this is the highest height
					if (HeightIndex + 1 == Cell.HitTriangles.Num()
						// Or if there's a gap above this height of size PlayAreaHeight
						|| ((Cell.HitTriangles[HeightIndex + 1].HeightRange.Y - CurrentMaxHeight) > PrecomputedVisibilitySettings.PlayAreaHeight
						// And this height is not within a cell that was just placed
						&& CurrentMaxHeight - LastSampleHeight > PrecomputedVisibilitySettings.PlayAreaHeight))
					{
						FPrecomputedVisibilityCell NewCell;
						NewCell.Bounds = FBox3f(
							FVector4f(
								CurrentPosition.X - PrecomputedVisibilitySettings.CellSize / 2,
								CurrentPosition.Y - PrecomputedVisibilitySettings.CellSize / 2,
								CurrentMaxHeight),
							FVector4f(
								CurrentPosition.X + PrecomputedVisibilitySettings.CellSize / 2,
								CurrentPosition.Y + PrecomputedVisibilitySettings.CellSize / 2,
								CurrentMaxHeight + PrecomputedVisibilitySettings.PlayAreaHeight));

						AllPrecomputedVisibilityCells.Add(NewCell);
						LastSampleHeight = CurrentMaxHeight;
						PlacedHeightRanges.Add(FVector2f(NewCell.Bounds.Min.Z, NewCell.Bounds.Max.Z));
					}
				}

				// Fractions of PrecomputedVisibilitySettings.PlayAreaHeight to guarantee have cell coverage
				float TestHeights[3] = {.4f, .6f, .8f};
				
				// Pass 2 - make sure the space above every triangle is covered by precomputed visibility cells, even if the cells are placed poorly (intersecting the floor)
				for (int32 HeightIndex = 0; HeightIndex < Cell.HitTriangles.Num() - 1; HeightIndex++)
				{
					for (int32 ExtremaIndex = 0; ExtremaIndex < 2; ExtremaIndex++)
					{
						const float CurrentMaxHeight = ExtremaIndex == 0 ? Cell.HitTriangles[HeightIndex].HeightRange.X : Cell.HitTriangles[HeightIndex].HeightRange.Y;
						const float CompareHeight = CurrentMaxHeight + .5f * PrecomputedVisibilitySettings.PlayAreaHeight;

						for (int32 TestIndex = 0; TestIndex < UE_ARRAY_COUNT(TestHeights); TestIndex++)
						{
							const float TestHeight = CurrentMaxHeight + TestHeights[TestIndex] * PrecomputedVisibilitySettings.PlayAreaHeight;

							int32 ClosestCellInZIndex = -1;
							float ClosestCellInZDistance = FLT_MAX;
							bool bInsideCell = false;

							for (int32 PlacedHeightIndex = 0; PlacedHeightIndex < PlacedHeightRanges.Num(); PlacedHeightIndex++)
							{
								FVector2f CellHeightRange = PlacedHeightRanges[PlacedHeightIndex];

								if (TestHeight > CellHeightRange.X && TestHeight < CellHeightRange.Y)
								{
									bInsideCell = true;
									break;
								}

								float AbsDistance = FMath::Min(FMath::Abs(CompareHeight - CellHeightRange.X), FMath::Abs(CompareHeight - CellHeightRange.Y));

								if (AbsDistance < ClosestCellInZDistance)
								{
									ClosestCellInZDistance = AbsDistance;
									ClosestCellInZIndex = PlacedHeightIndex;
								}
							}

							// Place a cell if TestHeight was not inside any existing cells
							if (!bInsideCell)
							{
								FPrecomputedVisibilityCell NewCell;
								float DesiredCellBottom = CurrentMaxHeight;

								if (ClosestCellInZIndex >= 0)
								{
									const FVector2f NearestCellHeightRange = PlacedHeightRanges[ClosestCellInZIndex];
									const float NearestCellCompareHeight = (NearestCellHeightRange.X + NearestCellHeightRange.Y) / 2;

									// Move the bottom of the cell to be placed such that it doesn't overlap the nearest cell
									// This makes use of the cell's full height to cover space
									if (CompareHeight < NearestCellCompareHeight)
									{
										DesiredCellBottom = FMath::Min(DesiredCellBottom, NearestCellHeightRange.X - PrecomputedVisibilitySettings.PlayAreaHeight);
									}
									else if (CompareHeight > NearestCellCompareHeight)
									{
										DesiredCellBottom = FMath::Max(DesiredCellBottom, NearestCellHeightRange.Y);
									}
								}

								NewCell.Bounds = FBox3f(
									FVector4f(
									CurrentPosition.X - PrecomputedVisibilitySettings.CellSize / 2,
									CurrentPosition.Y - PrecomputedVisibilitySettings.CellSize / 2,
									DesiredCellBottom),
									FVector4f(
									CurrentPosition.X + PrecomputedVisibilitySettings.CellSize / 2,
									CurrentPosition.Y + PrecomputedVisibilitySettings.CellSize / 2,
									DesiredCellBottom + PrecomputedVisibilitySettings.PlayAreaHeight));

								AllPrecomputedVisibilityCells.Add(NewCell);
								PlacedHeightRanges.Add(FVector2f(NewCell.Bounds.Min.Z, NewCell.Bounds.Max.Z));
							}
						}
					}
				}
			}
		}
	}

	// Place cells along camera tracks
	const int32 NumCellsPlacedOnSurfaces = AllPrecomputedVisibilityCells.Num();
	for (int32 CameraPositionIndex = 0; CameraPositionIndex < Scene.CameraTrackPositions.Num(); CameraPositionIndex++)
	{
		const FVector4f& CurrentPosition = Scene.CameraTrackPositions[CameraPositionIndex];
		bool bInsideCell = false;
		for (int32 CellIndex = 0; CellIndex < AllPrecomputedVisibilityCells.Num(); CellIndex++)
		{
			if (AllPrecomputedVisibilityCells[CellIndex].Bounds.IsInside(CurrentPosition))
			{
				bInsideCell = true;
				break;
			}
		}

		if (!bInsideCell)
		{
			FPrecomputedVisibilityCell NewCell;

			// Snap the cell min to the nearest factor of CellSize from the visibility bounds min + CellSize / 2
			// The CellSize / 2 offset is necessary to match up with cells produced by the rasterizer, since pixels are rasterized at cell centers
			const FVector4f PreSnapTranslation = FVector3f(PrecomputedVisibilitySettings.CellSize / 2) + PrecomputedVisibilityBounds.Origin - PrecomputedVisibilityBounds.BoxExtent;
			const FVector4f TranslatedPosition = CurrentPosition - PreSnapTranslation;
			// FMath::Fmod gives the offset to round up for negative numbers, when we always want the offset to round down
			const float XOffset = TranslatedPosition.X > 0.0f ? 
				FMath::Fmod(TranslatedPosition.X, PrecomputedVisibilitySettings.CellSize) : 
				PrecomputedVisibilitySettings.CellSize - FMath::Fmod(-TranslatedPosition.X, PrecomputedVisibilitySettings.CellSize);
			const float YOffset = TranslatedPosition.Y > 0.0f ? 
				FMath::Fmod(TranslatedPosition.Y, PrecomputedVisibilitySettings.CellSize) : 
				PrecomputedVisibilitySettings.CellSize - FMath::Fmod(-TranslatedPosition.Y, PrecomputedVisibilitySettings.CellSize);
			const FVector4f SnappedPosition(CurrentPosition.X - XOffset, CurrentPosition.Y - YOffset, CurrentPosition.Z);

			NewCell.Bounds = FBox3f(
				FVector4f(
					SnappedPosition.X,
					SnappedPosition.Y,
					SnappedPosition.Z - .5f * PrecomputedVisibilitySettings.PlayAreaHeight),
				FVector4f(
					SnappedPosition.X + PrecomputedVisibilitySettings.CellSize,
					SnappedPosition.Y + PrecomputedVisibilitySettings.CellSize,
					SnappedPosition.Z + .5f * PrecomputedVisibilitySettings.PlayAreaHeight));

			AllPrecomputedVisibilityCells.Add(NewCell);

			// Verify that the camera track position is inside the placed cell
			checkSlow(NewCell.Bounds.IsInside(CurrentPosition));
		}
	}

	{
		TArray<FVisibilityMeshSortInfo> SortMeshes;
		SortMeshes.Empty(VisibilityMeshes.Num());

		FVector3f CenterPosition(0, 0, 0);

		if (VisibilityMeshes.Num() > 0)
		{
			// Initialize to first mesh position so we can handle lighting stuff away from the origin
			CenterPosition = VisibilityMeshes[0].Meshes[0]->BoundingBox.GetCenter() / VisibilityMeshes.Num();
		}

		for (int32 VisibilityMeshIndex = 0; VisibilityMeshIndex < VisibilityMeshes.Num(); VisibilityMeshIndex++)
		{
			FVisibilityMeshSortInfo NewInfo;

			for (int32 OriginalMeshIndex = 0; OriginalMeshIndex < VisibilityMeshes[VisibilityMeshIndex].Meshes.Num(); OriginalMeshIndex++)
			{
				NewInfo.Bounds += VisibilityMeshes[VisibilityMeshIndex].Meshes[OriginalMeshIndex]->BoundingBox;
			}

			// First mesh already contributed
			if (VisibilityMeshIndex > 0)
			{
				CenterPosition += NewInfo.Bounds.GetCenter() / VisibilityMeshes.Num();
			}
			
			NewInfo.Index = VisibilityMeshIndex;
			SortMeshes.Add(NewInfo);
		}

		FVector3f CubeCorners[8];
		CubeCorners[0] = FVector3f(1, 1, 1);
		CubeCorners[1] = FVector3f(-1, 1, 1);
		CubeCorners[2] = FVector3f(1, -1, 1);
		CubeCorners[3] = FVector3f(-1, -1, 1);
		CubeCorners[4] = FVector3f(1, 1, -1);
		CubeCorners[5] = FVector3f(-1, 1, -1);
		CubeCorners[6] = FVector3f(1, -1, -1);
		CubeCorners[7] = FVector3f(-1, -1, -1);

		for (int32 MeshIndex = 0; MeshIndex < SortMeshes.Num(); MeshIndex++)
		{
			const FVector3f BoxCenter = SortMeshes[MeshIndex].Bounds.GetCenter();
			const FVector3f BoxExtent = SortMeshes[MeshIndex].Bounds.GetExtent();
			float LocalDistance = 0;

			for (int32 CornerIndex = 0; CornerIndex < UE_ARRAY_COUNT(CubeCorners); CornerIndex++)
			{
				// Calculate max distance to a corner of the bounds as a measure of how much this mesh will expand the grid bounds
				LocalDistance = FMath::Max(LocalDistance, (BoxCenter + BoxExtent * CubeCorners[CornerIndex]).SizeSquared());
			}
			
			SortMeshes[MeshIndex].Distance = LocalDistance;
		}

		struct FCompareMeshesByDistance
		{
			FORCEINLINE bool operator()( const FVisibilityMeshSortInfo& A, const FVisibilityMeshSortInfo& B ) const
			{
				return A.Distance < B.Distance;
			}
		};
		SortMeshes.Sort(FCompareMeshesByDistance());

		VisibilityGridBounds = FBox3f(ForceInit);
		// Drop last 10% of meshes which will expand the grid bounds 
		// This is to handle distant skybox type meshes
		const int32 MaxMeshIndex = FMath::Min(FMath::Max(FMath::TruncToInt(.9f * SortMeshes.Num()), 1), SortMeshes.Num() - 1);

		for (int32 MeshIndex = 0; MeshIndex < MaxMeshIndex; MeshIndex++)
		{
			VisibilityGridBounds += SortMeshes[MeshIndex].Bounds;
		}

		//@todo - expose
		const float TargetNumGroupsAsFractionOfMeshes = .3f;
		const int32 ZDimensionDivisor = 4;
		// Determine grid X and Y size using SizeX * SizeY * SizeZ = NumMeshes * TargetNumGroupsAsFractionOfMeshes
		GroupVisibilityGridSizeXY = FMath::Max(FMath::TruncToInt(FMath::Pow((ZDimensionDivisor * TargetNumGroupsAsFractionOfMeshes * VisibilityMeshes.Num()), 1.0f / 3.0f)), 1);
		GroupVisibilityGridSizeZ = FMath::Max(GroupVisibilityGridSizeXY / ZDimensionDivisor, 1);

		const int32 GridSizeXY = GroupVisibilityGridSizeXY;
		const int32 GridSizeZ = GroupVisibilityGridSizeZ;

		const FVector3f CellSize = VisibilityGridBounds.GetExtent() / FVector3f(GroupVisibilityGridSizeXY, GroupVisibilityGridSizeXY, GroupVisibilityGridSizeZ);
		const float GridCellBoundingRadius = CellSize.Size();
		const float MeshGroupingCellRadiusThreshold = .5f;

		GroupGrid.Empty(GridSizeXY * GridSizeXY * GridSizeZ);
		
		// Initialize grid group indices to invalid
		for (int32 GridEntry = 0; GridEntry < GridSizeXY * GridSizeXY * GridSizeZ; GridEntry++)
		{
			GroupGrid.Add(-1);
		}

		for (int32 VisibilityMeshIndex = 0; VisibilityMeshIndex < VisibilityMeshes.Num(); VisibilityMeshIndex++)
		{
			FBox3f MeshBounds(ForceInit);

			for (int32 OriginalMeshIndex = 0; OriginalMeshIndex < VisibilityMeshes[VisibilityMeshIndex].Meshes.Num(); OriginalMeshIndex++)
			{
				MeshBounds += VisibilityMeshes[VisibilityMeshIndex].Meshes[OriginalMeshIndex]->BoundingBox;
			}

			const float MeshBoundingRadiusSqr = MeshBounds.GetExtent().SizeSquared();

			// Only put the mesh in a group if its radius is small enough to keep the group effective
			bool bPutInGroup = MeshBoundingRadiusSqr < FMath::Square(GridCellBoundingRadius * MeshGroupingCellRadiusThreshold);

			if (bPutInGroup)
			{
				const int32 CellIndex = GetGroupCellIndex(MeshBounds.GetCenter());

				if (CellIndex >= 0)
				{
					int32 GroupIndex = GroupGrid[CellIndex];

					if (GroupIndex == -1)
					{
						// Add a new group if needed
						VisibilityGroups.Add(FVisibilityMeshGroup());
						GroupIndex = VisibilityGroups.Num() - 1;
						GroupGrid[CellIndex] = GroupIndex;
					}

					// Add to list of meshes in the group
					VisibilityGroups[GroupIndex].VisibilityIds.Add(VisibilityMeshIndex);
				}
				else
				{
					// Mesh was not inside grid
					bPutInGroup = false;
				}
			}

			// Mark whether the mesh was put into a group so we can look it up during visibility tracing
			VisibilityMeshes[VisibilityMeshIndex].bInGroup = bPutInGroup;

			if (!bPutInGroup)
			{
				Stats.NumPrecomputedVisibilityMeshesExcludedFromGroups++;
			}
		}

		// Build group bounds
		for (int32 GroupIndex = 0; GroupIndex < VisibilityGroups.Num(); GroupIndex++)
		{
			FVisibilityMeshGroup& Group = VisibilityGroups[GroupIndex];
			Group.GroupBounds = FBox3f(ForceInit);

			for (int32 EntryIndex = 0; EntryIndex < Group.VisibilityIds.Num(); EntryIndex++)
			{
				const int32 VisibilityId = Group.VisibilityIds[EntryIndex];

				for (int32 MeshIndex = 0; MeshIndex < VisibilityMeshes[VisibilityId].Meshes.Num(); MeshIndex++)
				{
					Group.GroupBounds += VisibilityMeshes[VisibilityId].Meshes[MeshIndex]->BoundingBox;
				}
			}
		}
	}
	
	const SIZE_T NumVisDataBytes = static_cast<SIZE_T>(AllPrecomputedVisibilityCells.Num()) * VisibilityMeshes.Num() / 8;
	Stats.NumPrecomputedVisibilityCellsTotal = AllPrecomputedVisibilityCells.Num();
	Stats.NumPrecomputedVisibilityCellsCamaraTracks = AllPrecomputedVisibilityCells.Num() - NumCellsPlacedOnSurfaces;
	Stats.NumPrecomputedVisibilityMeshes = VisibilityMeshes.Num();
	Stats.PrecomputedVisibilityDataBytes = NumVisDataBytes;
	Stats.PrecomputedVisibilitySetupTime = FPlatformTime::Seconds() - StartTime;

	if (AllPrecomputedVisibilityCells.Num() > 0)
	{
		LogSolverMessage(FString::Printf(TEXT("Setup precomputed visibility %.1fs, %u meshes, %u Cells"), FPlatformTime::Seconds() - StartTime, Stats.NumPrecomputedVisibilityMeshes, Stats.NumPrecomputedVisibilityCellsTotal));
	}
}

static bool IsMeshVisible(const TArray<uint8>& VisibilityData, int32 MeshId)
{
	return (VisibilityData[MeshId / 8] & 1 << (MeshId % 8)) != 0;
}

static void SetMeshVisible(TArray<uint8>& VisibilityData, int32 MeshId)
{
	VisibilityData[MeshId / 8] |= 1 << (MeshId % 8);
}

class FAxisAlignedCellFace
{
public:

	FAxisAlignedCellFace() {}

	FAxisAlignedCellFace(const FVector4f& InFaceDirection, const FVector4f& InFaceMin, const FVector4f& InFaceExtent) :
		FaceDirection(InFaceDirection),
		FaceMin(InFaceMin),
		FaceExtent(InFaceExtent)
	{}

	FVector4f FaceDirection;
	FVector4f FaceMin;
	FVector4f FaceExtent;
};

/** Stores information about a single query sample between a visibility cell and a mesh. */
struct FVisibilityQuerySample
{
	/** Sample position generated from the mesh. */
	FVector4f MeshPosition;

	/** Sample position generated from the cell. */
	FVector4f CellPosition;

	/** Position of the intersection with the scene between the sample positions. */
	FVector4f IntersectionPosition;

	/** Distance along the vector perpendicular to the mesh->cell vector. */
	float PerpendicularDistance;
};

/** 
 * Computes visibility from a cell to a box, returns true if the box is visible from the cell.
 * The TArrays are passed in to avoid allocations between cells, the same arrays are reused.
 */
bool ComputeBoxVisibility(
	FStaticLightingAggregateMeshType& AggregateMesh,
	FPrecomputedVisibilitySettings& PrecomputedVisibilitySettings,
	FPrecomputedVisibilityCell& CurrentCell, 
	FAxisAlignedCellFace* CellFaces,
	const FBox3f& MeshBox,
	FStaticLightingMappingContext& MappingContext,
	FLMRandomStream& RandomStream,
	TArray<int32>& VisibleCellFaces,
	TArray<float>& VisibleCellFacePDFs,
	TArray<float>& VisibleCellFaceCDFs,
	TArray<int32>& VisibleMeshFaces,
	TArray<FVisibilityQuerySample>& SamplePositions,
	TArray<const FVisibilityQuerySample*>& FurthestSamples,
	TArray<FDebugStaticLightingRay>& DebugVisibilityRays,
	bool bDebugThisCell,
	bool bDebugThisMesh,
	bool bGroupQuery)
{
	const double SampleGenerationStartTime = FPlatformTime::Seconds();

	const FVector4f CenterCellPosition = .5f * (CurrentCell.Bounds.Min + CurrentCell.Bounds.Max);
	const FVector4f MeshToCellCenter = CenterCellPosition - MeshBox.GetCenter();
	const float Distance = MeshToCellCenter.Size3();
	const FVector4f MeshBoxExtent = MeshBox.GetExtent() * 2;

	FAxisAlignedCellFace MeshBoxFaces[6];
	MeshBoxFaces[0] = FAxisAlignedCellFace(FVector4f(-1, 0, 0), FVector4f(MeshBox.Min.X, MeshBox.Min.Y, MeshBox.Min.Z), FVector4f(0, MeshBoxExtent.Y, MeshBoxExtent.Z));
	MeshBoxFaces[1] = FAxisAlignedCellFace(FVector4f(1, 0, 0), FVector4f(MeshBox.Min.X + MeshBoxExtent.X, MeshBox.Min.Y, MeshBox.Min.Z), FVector4f(0, MeshBoxExtent.Y, MeshBoxExtent.Z));
	MeshBoxFaces[2] = FAxisAlignedCellFace(FVector4f(0, -1, 0), FVector4f(MeshBox.Min.X, MeshBox.Min.Y, MeshBox.Min.Z), FVector4f(MeshBoxExtent.X, 0, MeshBoxExtent.Z));
	MeshBoxFaces[3] = FAxisAlignedCellFace(FVector4f(0, 1, 0), FVector4f(MeshBox.Min.X, MeshBox.Min.Y + MeshBoxExtent.Y, MeshBox.Min.Z), FVector4f(MeshBoxExtent.X, 0, MeshBoxExtent.Z));
	MeshBoxFaces[4] = FAxisAlignedCellFace(FVector4f(0, 0, -1), FVector4f(MeshBox.Min.X, MeshBox.Min.Y, MeshBox.Min.Z), FVector4f(MeshBoxExtent.X, MeshBoxExtent.Y, 0));
	MeshBoxFaces[5] = FAxisAlignedCellFace(FVector4f(0, 0, 1), FVector4f(MeshBox.Min.X, MeshBox.Min.Y, MeshBox.Min.Z + MeshBoxExtent.Z), FVector4f(MeshBoxExtent.X, MeshBoxExtent.Y, 0));

	VisibleCellFaces.Reset();
	VisibleCellFacePDFs.Reset();
	for (int32 i = 0; i < 6; i++)
	{
		const float DotProduct = -Dot3((MeshToCellCenter / Distance), CellFaces[i].FaceDirection);
		if (DotProduct > 0.0f)
		{
			VisibleCellFaces.Add(i);
			VisibleCellFacePDFs.Add(DotProduct);
		}
	}

	// Ensure that some of the faces will be sampled
	if (VisibleCellFacePDFs.Num() == 0)
	{
		for (int32 i = 0; i < 6; i++)
		{
			VisibleCellFaces.Add(i);
			VisibleCellFacePDFs.Add(i);
		}
	}

	float UnnormalizedIntegral;
	CalculateStep1dCDF(VisibleCellFacePDFs, VisibleCellFaceCDFs, UnnormalizedIntegral);

	VisibleMeshFaces.Reset();
	for (int32 i = 0; i < 6; i++)
	{
		if (Dot3(MeshToCellCenter, MeshBoxFaces[i].FaceDirection) > 0.0f)
		{
			VisibleMeshFaces.Add(i);
		}
	}

	if (bGroupQuery)
	{
		MappingContext.Stats.NumPrecomputedVisibilityGroupQueries++;
	}
	else
	{
		MappingContext.Stats.NumPrecomputedVisibilityQueries++;
	}

	const float MeshSize = MeshBox.GetExtent().Size();
	const float SizeRatio = MeshSize / Distance;
	// Use MaxMeshSamples for meshes with a large projected angle, and MinMeshSamples for meshes with a small projected angle
	// Meshes with a large projected angle require more samples to determine visibility accurately
	const int32 NumMeshSamples = FMath::Clamp(FMath::TruncToInt(SizeRatio * PrecomputedVisibilitySettings.MaxMeshSamples), PrecomputedVisibilitySettings.MinMeshSamples, PrecomputedVisibilitySettings.MaxMeshSamples);

	// Treat meshes whose projected angle is greater than 90 degrees as visible, since it becomes overly costly to determine if these are visible
	// (consider a large close mesh that only has a tiny part visible)
	bool bVisible = SizeRatio > 1.0f;

	if (bVisible)
	{
		MappingContext.Stats.NumQueriesVisibleByDistanceRatio++;
	}

	if (!bVisible)
	{
		const FVector4f PerpendicularVector = MeshToCellCenter ^ FVector4f(0, 0, 1);
		SamplePositions.Reset();

		// Generate samples for explicit visibility sampling of the mesh
		for (int32 CellSampleIndex = 0; CellSampleIndex < PrecomputedVisibilitySettings.NumCellSamples; CellSampleIndex++)
		{
			for (int32 MeshSampleIndex = 0; MeshSampleIndex < NumMeshSamples; MeshSampleIndex++)
			{
				FVisibilityQuerySample NewSample;
				{
					float PDF;
					float Sample;
					// Generate a sample on the visible faces of the cell, picking a face with probability proportional to the projected angle of the cell face onto the mesh's origin
					//@todo - weight by face area, since cells have a different height from their x and y sizes
					Sample1dCDF(VisibleCellFacePDFs, VisibleCellFaceCDFs, UnnormalizedIntegral, RandomStream, PDF, Sample);
					const int32 ChosenCellFaceIndex = FMath::TruncToInt(Sample * VisibleCellFaces.Num());
					const FAxisAlignedCellFace& ChosenFace = CellFaces[VisibleCellFaces[ChosenCellFaceIndex]];
					NewSample.CellPosition = ChosenFace.FaceMin + ChosenFace.FaceExtent * FVector4f(RandomStream.GetFraction(), RandomStream.GetFraction(), RandomStream.GetFraction());
				}
				{
					// Generate a sample on the visible faces of the mesh
					const int32 ChosenFaceIndex = FMath::TruncToInt(RandomStream.GetFraction() * VisibleMeshFaces.Num());
					const FAxisAlignedCellFace& ChosenFace = MeshBoxFaces[VisibleMeshFaces[ChosenFaceIndex]];
					NewSample.MeshPosition = ChosenFace.FaceMin + ChosenFace.FaceExtent * FVector4f(RandomStream.GetFraction(), RandomStream.GetFraction(), RandomStream.GetFraction());
				}
				const FVector4f HalfPosition = .5f * (NewSample.CellPosition + NewSample.MeshPosition);
				NewSample.PerpendicularDistance = Dot3(HalfPosition, PerpendicularVector);
				SamplePositions.Add(NewSample);
			}
		}

		// Sorts two samples based on perpendicular distance.
		struct FCompareSampleDistance
		{
			FORCEINLINE bool operator()(const FVisibilityQuerySample& A, const FVisibilityQuerySample& B) const
			{
				return A.PerpendicularDistance < B.PerpendicularDistance;
			}
		};
		// Sort the samples to make them more coherent in kDOP tree traversals, which provides a small speedup
		SamplePositions.Sort(FCompareSampleDistance());

		const double SampleGenerationEndTime = FPlatformTime::Seconds();
		MappingContext.Stats.PrecomputedVisibilitySampleSetupThreadTime += SampleGenerationEndTime - SampleGenerationStartTime;

		float FurthestDistanceSquared = 0;
		// Early out if any sample finds the mesh visible, unless we are debugging this cell and want to see all the samples
		for (int32 CellSampleIndex = 0; (CellSampleIndex < PrecomputedVisibilitySettings.NumCellSamples) && (!bVisible || bDebugThisCell); CellSampleIndex++)
		{
			for (int32 MeshSampleIndex = 0; MeshSampleIndex < NumMeshSamples; MeshSampleIndex++)
			{
				FVisibilityQuerySample& CurrentSample = SamplePositions[CellSampleIndex * NumMeshSamples + MeshSampleIndex];
				const FVector4f CellSamplePosition = CurrentSample.CellPosition;
				const FVector4f MeshSamplePosition = CurrentSample.MeshPosition;

				FLightRay Ray(
					CellSamplePosition,
					MeshSamplePosition,
					NULL,
					NULL,
					// Masked materials often have small holes which increase visibility errors
					// This also allows us to use boolean visibility traces which are much faster than first hit traces
					// Only intersect with static objects since they will not move in game
					LIGHTRAY_STATIC_AND_OPAQUEONLY
					);

				FLightRayIntersection Intersection;
				// Use boolean visibility traces
				AggregateMesh.IntersectLightRay(Ray, false, false, false, MappingContext.RayCache, Intersection);

				MappingContext.Stats.NumPrecomputedVisibilityRayTraces++;

				//Note: using intersection position even though we used a boolean ray trace, so the position may not be the closest
				CurrentSample.IntersectionPosition = Intersection.IntersectionVertex.WorldPosition;

				const float DistanceSquared = (CellSamplePosition - Intersection.IntersectionVertex.WorldPosition).SizeSquared3();
				FurthestDistanceSquared = FMath::Max(FurthestDistanceSquared, DistanceSquared);

				if (bDebugThisMesh)
				{
					// Draw all the rays from the debug cell to the debug mesh
					FDebugStaticLightingRay DebugRay(CellSamplePosition, MeshSamplePosition, Intersection.bIntersects, false);
					if (Intersection.bIntersects)
					{
						DebugRay.End = Intersection.IntersectionVertex.WorldPosition;
					}
					DebugVisibilityRays.Add(DebugRay);
				}

				if (!Intersection.bIntersects)
				{
					MappingContext.Stats.NumQueriesVisibleExplicitSampling++;
					bVisible = true;
					if (!bDebugThisCell)
					{
						// The mesh is visible from the current cell, move on to the next mesh
						break;
					}
				}
			}
		}

		const double RayTraceEndTime = FPlatformTime::Seconds();
		MappingContext.Stats.PrecomputedVisibilityRayTraceThreadTime += RayTraceEndTime - SampleGenerationEndTime;

		// If the meshes has not been determined to be visible by explicit sampling, 
		// Do importance sampling to try and find visible meshes through cracks that have a low probability of being detected by explicit sampling.
		if (!bVisible)
		{
			FurthestSamples.Reset();

			// Create an array of all the longest rays toward the mesh
			const float DistanceThreshold = FMath::Sqrt(FurthestDistanceSquared) * 7.0f / 8.0f;
			const float DistanceThresholdSq = DistanceThreshold * DistanceThreshold;
			for (int32 SampleIndex = 0; SampleIndex < SamplePositions.Num(); SampleIndex++)
			{
				const FVisibilityQuerySample& CurrentSample = SamplePositions[SampleIndex];
				const float DistanceSquared = (CurrentSample.CellPosition - CurrentSample.IntersectionPosition).SizeSquared3();
				if (DistanceSquared > DistanceThresholdSq)
				{
					FurthestSamples.Add(&CurrentSample);
				}
			}

			// Trace importance sampled rays to try and find visible meshes through small cracks
			// This is only slightly effective, but doesn't cost much compared to explicit sampling due to the small number of rays
			for (int32 ImportanceSampleIndex = 0; ImportanceSampleIndex < PrecomputedVisibilitySettings.NumImportanceSamples && !bVisible && FurthestSamples.Num() > 0; ImportanceSampleIndex++)
			{
				// Pick one of the furthest samples with uniform probability
				const int32 SampleIndex = FMath::TruncToInt(RandomStream.GetFraction() * FurthestSamples.Num());
				const FVisibilityQuerySample& CurrentSample = *FurthestSamples[SampleIndex];
				const float VectorLength = (CurrentSample.CellPosition - CurrentSample.MeshPosition).Size3();
				const FVector4f CurrentDirection = (CurrentSample.MeshPosition - CurrentSample.CellPosition).GetSafeNormal();

				FVector4f XAxis;
				FVector4f YAxis;
				GenerateCoordinateSystem(CurrentDirection, XAxis, YAxis);

				// Generate a new direction in a cone 2 degrees from the original direction, to find cracks nearby
				const FVector4f SampleDirection = UniformSampleCone(
					RandomStream, 
					FMath::Cos(2.0f * PI / 180.0f), 
					XAxis, 
					YAxis, 
					CurrentDirection);

				const FVector4f EndPoint = CurrentSample.CellPosition + SampleDirection * VectorLength;

				FLightRay Ray(
					CurrentSample.CellPosition,
					EndPoint,
					NULL,
					NULL,
					LIGHTRAY_STATIC_AND_OPAQUEONLY
					);

				FLightRayIntersection Intersection;
				AggregateMesh.IntersectLightRay(Ray, false, false, false, MappingContext.RayCache, Intersection);

				MappingContext.Stats.NumPrecomputedVisibilityRayTraces++;

				if (bDebugThisMesh)
				{
					// Draw all the rays from the debug cell to the debug mesh
					FDebugStaticLightingRay DebugRay(CurrentSample.CellPosition, EndPoint, Intersection.bIntersects, true);
					if (Intersection.bIntersects)
					{
						DebugRay.End = Intersection.IntersectionVertex.WorldPosition;
					}
					DebugVisibilityRays.Add(DebugRay);
				}

				if (!Intersection.bIntersects)
				{
					MappingContext.Stats.NumQueriesVisibleImportanceSampling++;
					bVisible = true;
					if (!bDebugThisCell)
					{
						// The mesh is visible from the current cell, move on to the next mesh
						break;
					}
				}
			}
			MappingContext.Stats.PrecomputedVisibilityImportanceSampleThreadTime += FPlatformTime::Seconds() - RayTraceEndTime;
		}
	}

	return bVisible;
}

void AddMeshDebugLines(TArray<FDebugStaticLightingRay>& DebugVisibilityRays, const TArray<FStaticLightingMesh*>& Meshes, const FBox3f& MeshBox)
{
	// Draw the bounding boxes of each mesh and the combined bounds
	if (Meshes.Num() > 1)
	{
		for (int32 OriginalMeshIndex = 0; OriginalMeshIndex < Meshes.Num(); OriginalMeshIndex++)
		{
			const FVector4f Min = Meshes[OriginalMeshIndex]->BoundingBox.Min;
			const FVector4f Max = Meshes[OriginalMeshIndex]->BoundingBox.Max;
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Min.Z), FVector4f(Min.X, Max.Y, Min.Z), false));
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Max.Y, Min.Z), FVector4f(Min.X, Max.Y, Max.Z), false));
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Min.Z), FVector4f(Min.X, Min.Y, Max.Z), false));
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Max.Z), FVector4f(Min.X, Max.Y, Max.Z), false));

			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Min.Y, Min.Z), FVector4f(Max.X, Max.Y, Min.Z), false));
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Max.Y, Min.Z), FVector4f(Max.X, Max.Y, Max.Z), false));
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Min.Y, Min.Z), FVector4f(Max.X, Min.Y, Max.Z), false));
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Min.Y, Max.Z), FVector4f(Max.X, Max.Y, Max.Z), false));

			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Min.Z), FVector4f(Max.X, Min.Y, Min.Z), false));
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Max.Z), FVector4f(Max.X, Min.Y, Max.Z), false));
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Max.Y, Min.Z), FVector4f(Max.X, Max.Y, Min.Z), false));
			DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Max.Y, Max.Z), FVector4f(Max.X, Max.Y, Max.Z), false));
		}
	}

	const FVector4f Min = MeshBox.Min;
	const FVector4f Max = MeshBox.Max;
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Min.Z), FVector4f(Min.X, Max.Y, Min.Z), true));
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Max.Y, Min.Z), FVector4f(Min.X, Max.Y, Max.Z), true));
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Min.Z), FVector4f(Min.X, Min.Y, Max.Z), true));
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Max.Z), FVector4f(Min.X, Max.Y, Max.Z), true));

	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Min.Y, Min.Z), FVector4f(Max.X, Max.Y, Min.Z), true));
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Max.Y, Min.Z), FVector4f(Max.X, Max.Y, Max.Z), true));
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Min.Y, Min.Z), FVector4f(Max.X, Min.Y, Max.Z), true));
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Min.Y, Max.Z), FVector4f(Max.X, Max.Y, Max.Z), true));

	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Min.Z), FVector4f(Max.X, Min.Y, Min.Z), true));
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Max.Z), FVector4f(Max.X, Min.Y, Max.Z), true));
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Max.Y, Min.Z), FVector4f(Max.X, Max.Y, Min.Z), true));
	DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Max.Y, Max.Z), FVector4f(Max.X, Max.Y, Max.Z), true));
}

/** Calculates visibility for a given group of cells, called from all threads. */
void FStaticLightingSystem::CalculatePrecomputedVisibility(int32 BucketIndex)
{
	const double StartTime = FPlatformTime::Seconds();
	check(BucketIndex >= 0 && BucketIndex < PrecomputedVisibilitySettings.NumCellDistributionBuckets);
	// Create a new link for the output of this task
	TList<FPrecomputedVisibilityData>* DataLink = new TList<FPrecomputedVisibilityData>(FPrecomputedVisibilityData(),NULL);
	DataLink->Element.Guid = Scene.VisibilityBucketGuids[BucketIndex];
	
	// Determine the range of cells to process from the bucket index
	const int32 StartCellIndex = BucketIndex * AllPrecomputedVisibilityCells.Num() / PrecomputedVisibilitySettings.NumCellDistributionBuckets;
	const int32 MaxCellIndex = BucketIndex + 1 == PrecomputedVisibilitySettings.NumCellDistributionBuckets ? 
		// Last bucket processes up to the end of the array
		AllPrecomputedVisibilityCells.Num() :
		(BucketIndex + 1) * AllPrecomputedVisibilityCells.Num() / PrecomputedVisibilitySettings.NumCellDistributionBuckets;

	DataLink->Element.PrecomputedVisibilityCells.Empty(MaxCellIndex - StartCellIndex);
	
	FStaticLightingMappingContext MappingContext(NULL, *this);

	// These are re-used across operations on the same thread to reduce reallocations
	TArray<int32> VisibleCellFaces;
	TArray<float> VisibleCellFacePDFs;
	TArray<float> VisibleCellFaceCDFs;
	TArray<int32> VisibleMeshFaces;
	TArray<FVisibilityQuerySample> SamplePositions;
	TArray<const FVisibilityQuerySample*> FurthestSamples;
	TArray<bool> GroupVisibility;

	TArray<const FPrecomputedVisibilityOverrideVolume*> AffectingOverrideVolumes;
	for (int32 CellIndex = StartCellIndex; CellIndex < MaxCellIndex; CellIndex++)
	{
		const double StartSampleTime = FPlatformTime::Seconds();

		// Seed by absolute cell index for deterministic results regardless of how cell tasks are distributed
		FLMRandomStream RandomStream(CellIndex);

		// Reset cached information for this cell so that traces won't be affected by what previous cells did
		MappingContext.RayCache.Clear();

		DataLink->Element.PrecomputedVisibilityCells.Add(AllPrecomputedVisibilityCells[CellIndex]);
		FPrecomputedVisibilityCell& CurrentCell = DataLink->Element.PrecomputedVisibilityCells.Last();

		const bool bDebugThisCell = CurrentCell.Bounds.IsInside(Scene.DebugInput.CameraPosition) && PrecomputedVisibilitySettings.bVisualizePrecomputedVisibility;

		AffectingOverrideVolumes.Reset();
		for (int32 VolumeIndex = 0; VolumeIndex < Scene.PrecomputedVisibilityOverrideVolumes.Num(); VolumeIndex++)
		{
			if (Scene.PrecomputedVisibilityOverrideVolumes[VolumeIndex].Bounds.Intersect(CurrentCell.Bounds))
			{
				AffectingOverrideVolumes.Add(&Scene.PrecomputedVisibilityOverrideVolumes[VolumeIndex]);
			}
		}

		CurrentCell.VisibilityData.Empty(VisibilityMeshes.Num() / 8 + 1);
		CurrentCell.VisibilityData.AddZeroed(VisibilityMeshes.Num() / 8 + 1);

		FAxisAlignedCellFace CellFaces[6];
		const FVector3f CellBoundsSize = CurrentCell.Bounds.GetSize();
		CellFaces[0] = FAxisAlignedCellFace(FVector4f(-1, 0, 0), FVector4f(CurrentCell.Bounds.Min.X, CurrentCell.Bounds.Min.Y, CurrentCell.Bounds.Min.Z), FVector4f(0, CellBoundsSize.Y, CellBoundsSize.Z));
		CellFaces[1] = FAxisAlignedCellFace(FVector4f(1, 0, 0), FVector4f(CurrentCell.Bounds.Max.X, CurrentCell.Bounds.Min.Y, CurrentCell.Bounds.Min.Z), FVector4f(0, CellBoundsSize.Y, CellBoundsSize.Z));
		CellFaces[2] = FAxisAlignedCellFace(FVector4f(0, -1, 0), FVector4f(CurrentCell.Bounds.Min.X, CurrentCell.Bounds.Min.Y, CurrentCell.Bounds.Min.Z), FVector4f(CellBoundsSize.X, 0, CellBoundsSize.Z));
		CellFaces[3] = FAxisAlignedCellFace(FVector4f(0, 1, 0), FVector4f(CurrentCell.Bounds.Min.X, CurrentCell.Bounds.Max.Y, CurrentCell.Bounds.Min.Z), FVector4f(CellBoundsSize.X, 0, CellBoundsSize.Z));
		CellFaces[4] = FAxisAlignedCellFace(FVector4f(0, 0, -1), FVector4f(CurrentCell.Bounds.Min.X, CurrentCell.Bounds.Min.Y, CurrentCell.Bounds.Min.Z), FVector4f(CellBoundsSize.X, CellBoundsSize.Y, 0));
		CellFaces[5] = FAxisAlignedCellFace(FVector4f(0, 0, 1), FVector4f(CurrentCell.Bounds.Min.X, CurrentCell.Bounds.Min.Y, CurrentCell.Bounds.Max.Z), FVector4f(CellBoundsSize.X, CellBoundsSize.Y, 0));

		GroupVisibility.Reset();
		
		// First determine group visibility using the combined bounds, so we can skip lots of mesh queries later (if they are all invisible)
		for (int32 GroupIndex = 0; GroupIndex < VisibilityGroups.Num(); GroupIndex++)
		{
			const FVisibilityMeshGroup& Group = VisibilityGroups[GroupIndex];
			bool bDebugThisMesh = false;

			bool bVisible = ComputeBoxVisibility(
				*AggregateMesh, 
				PrecomputedVisibilitySettings, 
				CurrentCell, 
				CellFaces, 
				Group.GroupBounds, 
				MappingContext, 
				RandomStream, 
				VisibleCellFaces, 
				VisibleCellFacePDFs, 
				VisibleCellFaceCDFs, 
				VisibleMeshFaces, 
				SamplePositions, 
				FurthestSamples, 
				DataLink->Element.DebugVisibilityRays, 
				bDebugThisCell, 
				bDebugThisMesh,
				true);

			GroupVisibility.Add(bVisible);
		}

		for (int32 VisibilityMeshIndex = 0; VisibilityMeshIndex < VisibilityMeshes.Num(); VisibilityMeshIndex++)
		{
			const FVisibilityMesh& VisibilityMesh = VisibilityMeshes[VisibilityMeshIndex];

			FBox3f OriginalMeshBounds(ForceInit);
			// Combine mesh bounds, usually only BSP has multiple meshes per VisibilityId
			//@todo - could explicitly sample each bounds separately, but they tend to be pretty close together in world space
			for (int32 OriginalMeshIndex = 0; OriginalMeshIndex < VisibilityMesh.Meshes.Num(); OriginalMeshIndex++)
			{
				OriginalMeshBounds += VisibilityMesh.Meshes[OriginalMeshIndex]->BoundingBox;
			}

			const FBox3f MeshBox(
				OriginalMeshBounds.GetCenter() - OriginalMeshBounds.GetExtent() * PrecomputedVisibilitySettings.MeshBoundsScale,
				OriginalMeshBounds.GetCenter() + OriginalMeshBounds.GetExtent() * PrecomputedVisibilitySettings.MeshBoundsScale);

			const bool bDebugThisMesh = VisibilityMeshIndex == Scene.DebugInput.DebugVisibilityId && bDebugThisCell;

			if (bDebugThisMesh)
			{
				AddMeshDebugLines(DataLink->Element.DebugVisibilityRays, VisibilityMesh.Meshes, MeshBox);
			}

			bool bVisible = false;
			bool bForceInvisible = false;

			// Apply override volumes first in case they can save us some work
			for (int32 VolumeIndex = 0; VolumeIndex < AffectingOverrideVolumes.Num(); VolumeIndex++)
			{
				if (AffectingOverrideVolumes[VolumeIndex]->OverrideVisibilityIds.Contains(VisibilityMeshIndex))
				{
					bVisible = true;
					break;
				}
				// This means forced visibility will override forced invisibility
				// Something to keep in mind when an LD complains that an actor 
				// they put into the OverrideInvisibility list is still showing up!
				if (AffectingOverrideVolumes[VolumeIndex]->OverrideInvisibilityIds.Contains(VisibilityMeshIndex))
				{
					bVisible = false;
					bForceInvisible = true;
					break;
				}
			}

			if (!bVisible && !bForceInvisible)
			{
				const int32 GroupCellIndex = GetGroupCellIndex(OriginalMeshBounds.GetCenter());

				bool bGroupVisible = true;
				
				// Lookup group visibility, if this mesh was put into a group
				if (GroupCellIndex >= 0 && VisibilityMesh.bInGroup)
				{
					const int32 GroupId = GroupGrid[GroupCellIndex];
					bGroupVisible = GroupVisibility[GroupGrid[GroupCellIndex]];
				}
				
				// Only determine mesh visibility if the group containing the mesh was also visible
				if (bGroupVisible)
				{
					bVisible = ComputeBoxVisibility(
						*AggregateMesh, 
						PrecomputedVisibilitySettings, 
						CurrentCell, 
						CellFaces, 
						MeshBox, 
						MappingContext, 
						RandomStream, 
						VisibleCellFaces, 
						VisibleCellFacePDFs, 
						VisibleCellFaceCDFs, 
						VisibleMeshFaces, 
						SamplePositions, 
						FurthestSamples, 
						DataLink->Element.DebugVisibilityRays, 
						bDebugThisCell, 
						bDebugThisMesh,
						false);
				}
				else
				{
					MappingContext.Stats.NumPrecomputedVisibilityMeshQueriesSkipped++;
				}
			}

			if (bVisible)
			{
				SetMeshVisible(CurrentCell.VisibilityData, VisibilityMeshIndex);
			}
		}

		if (bDebugThisCell)
		{
			// Draw the bounds of each cell processed
			const FVector4f Min = CurrentCell.Bounds.Min;
			const FVector4f Max = CurrentCell.Bounds.Max;
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Min.Z), FVector4f(Min.X, Max.Y, Min.Z), false));
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Max.Y, Min.Z), FVector4f(Min.X, Max.Y, Max.Z), false));
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Min.Z), FVector4f(Min.X, Min.Y, Max.Z), false));
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Max.Z), FVector4f(Min.X, Max.Y, Max.Z), false));

			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Min.Y, Min.Z), FVector4f(Max.X, Max.Y, Min.Z), false));
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Max.Y, Min.Z), FVector4f(Max.X, Max.Y, Max.Z), false));
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Min.Y, Min.Z), FVector4f(Max.X, Min.Y, Max.Z), false));
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Max.X, Min.Y, Max.Z), FVector4f(Max.X, Max.Y, Max.Z), false));

			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Min.Z), FVector4f(Max.X, Min.Y, Min.Z), false));
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Min.Y, Max.Z), FVector4f(Max.X, Min.Y, Max.Z), false));
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Max.Y, Min.Z), FVector4f(Max.X, Max.Y, Min.Z), false));
			DataLink->Element.DebugVisibilityRays.Add(FDebugStaticLightingRay(FVector4f(Min.X, Max.Y, Max.Z), FVector4f(Max.X, Max.Y, Max.Z), false));
		}
	}
	
	MappingContext.Stats.PrecomputedVisibilityThreadTime = FPlatformTime::Seconds() - StartTime;
	MappingContext.Stats.NumPrecomputedVisibilityCellsProcessed = MaxCellIndex - StartCellIndex;
	CompleteVisibilityTaskList.AddElement(DataLink);
}

}
