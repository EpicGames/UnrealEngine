// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include "Raster.h"
#include "LightingSystem.h"
#include "LightmassSwarm.h"
#include "Misc/Compression.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "TextureMappingSetup.h"
#include "Misc/Compression.h"

namespace Lightmass
{

bool bCompressRadiosityCachedData = false;

void FCompressedGatherHitPoints::Compress(const FGatherHitPoints& Source)
{
	{
		const int32 UncompressedSize = Source.GatherHitPointRanges.Num() * Source.GatherHitPointRanges.GetTypeSize();

		TArray<uint8> TempCompressedMemory;
		// Compressed can be slightly larger than uncompressed
		TempCompressedMemory.Empty(UncompressedSize * 4 / 3);
		TempCompressedMemory.AddUninitialized(UncompressedSize * 4 / 3);
		int32 CompressedSize = TempCompressedMemory.Num() * TempCompressedMemory.GetTypeSize();

		verify(FCompression::CompressMemory(
			NAME_Zlib, 
			TempCompressedMemory.GetData(), 
			CompressedSize, 
			Source.GatherHitPointRanges.GetData(), 
			UncompressedSize,
			COMPRESS_BiasSpeed));

		GatherHitPointRanges.Empty(CompressedSize);
		GatherHitPointRanges.AddUninitialized(CompressedSize);

		FPlatformMemory::Memcpy(GatherHitPointRanges.GetData(), TempCompressedMemory.GetData(), CompressedSize);
		GatherHitPointRangesUncompressedSize = UncompressedSize;
	}
	
	{
		const int32 UncompressedSize = Source.GatherHitPointData.Num() * Source.GatherHitPointData.GetTypeSize();

		TArray<uint8> TempCompressedMemory;
		// Compressed can be slightly larger than uncompressed
		TempCompressedMemory.Empty(UncompressedSize * 4 / 3);
		TempCompressedMemory.AddUninitialized(UncompressedSize * 4 / 3);
		int32 CompressedSize = TempCompressedMemory.Num() * TempCompressedMemory.GetTypeSize();

		verify(FCompression::CompressMemory(
			NAME_Zlib,
			TempCompressedMemory.GetData(), 
			CompressedSize, 
			Source.GatherHitPointData.GetData(), 
			UncompressedSize));

		GatherHitPointData.Empty(CompressedSize);
		GatherHitPointData.AddUninitialized(CompressedSize);

		FPlatformMemory::Memcpy(GatherHitPointData.GetData(), TempCompressedMemory.GetData(), CompressedSize);
		GatherHitPointDataUncompressedSize = UncompressedSize;
	}
}

void FCompressedGatherHitPoints::Decompress(FGatherHitPoints& Dest) const
{
	Dest.GatherHitPointRanges.Reset(GatherHitPointRangesUncompressedSize);
	Dest.GatherHitPointRanges.AddUninitialized(GatherHitPointRangesUncompressedSize);

	verify(FCompression::UncompressMemory(
		NAME_Zlib,
		Dest.GatherHitPointRanges.GetData(), 
		GatherHitPointRangesUncompressedSize, 
		GatherHitPointRanges.GetData(), 
		GatherHitPointRanges.Num()));

	Dest.GatherHitPointData.Reset(GatherHitPointDataUncompressedSize);
	Dest.GatherHitPointData.AddUninitialized(GatherHitPointDataUncompressedSize);

	verify(FCompression::UncompressMemory(
		NAME_Zlib,
		Dest.GatherHitPointData.GetData(), 
		GatherHitPointDataUncompressedSize, 
		GatherHitPointData.GetData(), 
		GatherHitPointData.Num()));
}


void FCompressedInfluencingRecords::Compress(const FInfluencingRecords& Source)
{
	{
		const int32 UncompressedSize = Source.Ranges.Num() * Source.Ranges.GetTypeSize();

		TArray<uint8> TempCompressedMemory;
		// Compressed can be slightly larger than uncompressed
		TempCompressedMemory.Empty(UncompressedSize * 4 / 3);
		TempCompressedMemory.AddUninitialized(UncompressedSize * 4 / 3);
		int32 CompressedSize = TempCompressedMemory.Num() * TempCompressedMemory.GetTypeSize();

		verify(FCompression::CompressMemory(
			NAME_Zlib,
			TempCompressedMemory.GetData(), 
			CompressedSize, 
			Source.Ranges.GetData(), 
			UncompressedSize,
			COMPRESS_BiasSpeed));

		Ranges.Empty(CompressedSize);
		Ranges.AddUninitialized(CompressedSize);

		FPlatformMemory::Memcpy(Ranges.GetData(), TempCompressedMemory.GetData(), CompressedSize);
		RangesUncompressedSize = UncompressedSize;
	}
	
	{
		const int32 UncompressedSize = Source.Data.Num() * Source.Data.GetTypeSize();

		TArray<uint8> TempCompressedMemory;
		// Compressed can be slightly larger than uncompressed
		TempCompressedMemory.Empty(UncompressedSize * 4 / 3);
		TempCompressedMemory.AddUninitialized(UncompressedSize * 4 / 3);
		int32 CompressedSize = TempCompressedMemory.Num() * TempCompressedMemory.GetTypeSize();

		verify(FCompression::CompressMemory(
			NAME_Zlib,
			TempCompressedMemory.GetData(), 
			CompressedSize, 
			Source.Data.GetData(), 
			UncompressedSize));

		Data.Empty(CompressedSize);
		Data.AddUninitialized(CompressedSize);

		FPlatformMemory::Memcpy(Data.GetData(), TempCompressedMemory.GetData(), CompressedSize);
		DataUncompressedSize = UncompressedSize;
	}
}

void FCompressedInfluencingRecords::Decompress(FInfluencingRecords& Dest) const
{
	Dest.Ranges.Reset(RangesUncompressedSize);
	Dest.Ranges.AddUninitialized(RangesUncompressedSize);

	verify(FCompression::UncompressMemory(
		NAME_Zlib,
		Dest.Ranges.GetData(), 
		RangesUncompressedSize, 
		Ranges.GetData(), 
		Ranges.Num()));

	Dest.Data.Reset(DataUncompressedSize);
	Dest.Data.AddUninitialized(DataUncompressedSize);

	verify(FCompression::UncompressMemory(
		NAME_Zlib,
		Dest.Data.GetData(), 
		DataUncompressedSize, 
		Data.GetData(), 
		Data.Num()));
}

void FStaticLightingSystem::SetupRadiosity()
{
	const double RadiosityStartTime = FPlatformTime::Seconds();

	for(int32 ThreadIndex = 1; ThreadIndex < NumStaticLightingThreads; ThreadIndex++)
	{
		FMappingProcessingThreadRunnable* ThreadRunnable = new FMappingProcessingThreadRunnable(this, ThreadIndex, StaticLightingTask_RadiositySetup);
		RadiositySetupThreads.Add(ThreadRunnable);
		const FString ThreadName = FString::Printf(TEXT("RadiositySetupThread%u"), ThreadIndex);
		ThreadRunnable->Thread = FRunnableThread::Create(ThreadRunnable, *ThreadName);
	}

	// Start the static lighting thread loop on the main thread, too.
	// Once it returns, all static lighting mappings have begun processing.
	RadiositySetupThreadLoop(0, true);

	// Stop the static lighting threads.
	for(int32 ThreadIndex = 0;ThreadIndex < RadiositySetupThreads.Num();ThreadIndex++)
	{
		// Wait for the thread to exit.
		RadiositySetupThreads[ThreadIndex].Thread->WaitForCompletion();
		// Check that it didn't terminate with an error.
		RadiositySetupThreads[ThreadIndex].CheckHealth();

		// Destroy the thread.
		delete RadiositySetupThreads[ThreadIndex].Thread;
	}
	RadiositySetupThreads.Empty();

	const float RadiosityDuration = FPlatformTime::Seconds() - RadiosityStartTime;
	LogSolverMessage(FString::Printf(TEXT("Radiosity Setup %.1fs"), RadiosityDuration));
}

void FStaticLightingSystem::RadiositySetupThreadLoop(int32 ThreadIndex, bool bIsMainThread)
{
	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing4, ThreadIndex ) );
	bool bIsDone = false;
	while (!bIsDone)
	{
		// Atomically read and increment the next mapping index to process.
		const int32 MappingIndex = NextMappingToProcessRadiositySetup.Increment() - 1;

		if (MappingIndex < AllMappings.Num())
		{
			// If this is the main thread, update progress and apply completed static lighting.
			if (bIsMainThread)
			{
				// Check the health of all static lighting threads.
				for (int32 ThreadIndexIter = 0; ThreadIndexIter < RadiositySetupThreads.Num(); ThreadIndexIter++)
				{
					RadiositySetupThreads[ThreadIndexIter].CheckHealth();
				}
			}

			FStaticLightingTextureMapping* TextureMapping = AllMappings[MappingIndex]->GetTextureMapping();

			if (TextureMapping)
			{
				RadiositySetupTextureMapping(TextureMapping);
			}
		}
		else
		{
			// Processing has begun for all mappings.
			bIsDone = true;
		}
	}
}

/** Caches irradiance photons on a single texture mapping. */
void FStaticLightingSystem::RadiositySetupTextureMapping(FStaticLightingTextureMapping* TextureMapping)
{
	checkSlow(TextureMapping);
	FStaticLightingMappingContext MappingContext(TextureMapping->Mesh,*this,&StartupDebugOutput);
	LIGHTINGSTAT(FScopedRDTSCTimer CachingTime(MappingContext.Stats.RadiositySetupThreadTime));
	const FBoxSphereBounds3f ImportanceBounds = Scene.GetImportanceBounds();

	FTexelToVertexMap TexelToVertexMap(TextureMapping->SurfaceCacheSizeX, TextureMapping->SurfaceCacheSizeY);

	bool bDebugThisMapping = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	bDebugThisMapping = TextureMapping == Scene.DebugMapping;
	int32 SurfaceCacheDebugX = -1;
	int32 SurfaceCacheDebugY = -1;
	if (bDebugThisMapping)
	{
		SurfaceCacheDebugX = FMath::TruncToInt(Scene.DebugInput.LocalX / (float)TextureMapping->CachedSizeX * TextureMapping->SurfaceCacheSizeX);
		SurfaceCacheDebugY = FMath::TruncToInt(Scene.DebugInput.LocalY / (float)TextureMapping->CachedSizeY * TextureMapping->SurfaceCacheSizeY);
	}
#endif

	RasterizeToSurfaceCacheTextureMapping(TextureMapping, bDebugThisMapping, TexelToVertexMap);

	TextureMapping->SurfaceCacheLighting.Empty(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);
	TextureMapping->SurfaceCacheLighting.AddZeroed(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);

	TextureMapping->RadiositySurfaceCache[0].Empty(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);
	TextureMapping->RadiositySurfaceCache[0].AddZeroed(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);
	TextureMapping->RadiositySurfaceCache[1].Empty(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);
	TextureMapping->RadiositySurfaceCache[1].AddZeroed(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);

	const bool bCacheFinalGatherHitPoints = ImportanceTracingSettings.bCacheFinalGatherHitPointsForRadiosity && GeneralSettings.NumSkyLightingBounces > 0;

	FGatherHitPoints& GatherHitPoints = TextureMapping->UncompressedGatherHitPoints;

	if (bCacheFinalGatherHitPoints)
	{
		GatherHitPoints.GatherHitPointRanges.Empty(TextureMapping->SurfaceCacheSizeY * TextureMapping->SurfaceCacheSizeX / 4);
		GatherHitPoints.GatherHitPointData.Empty(TextureMapping->SurfaceCacheSizeY * TextureMapping->SurfaceCacheSizeX / 4);
	}

	TLightingCache<FFinalGatherSample> RadiosityCache(TextureMapping->Mesh->BoundingBox, *this, 1);
	
	FLMRandomStream RandomStream(0);

#if LIGHTMASS_DO_PROCESSING

	if (GeneralSettings.NumSkyLightingBounces > 0)
	{
		for (int32 Y = 0; Y < TextureMapping->SurfaceCacheSizeY; Y++)
		{
			for (int32 X = 0; X < TextureMapping->SurfaceCacheSizeX; X++)
			{
				bool bDebugThisTexel = false;
	#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& Y == SurfaceCacheDebugY
					&& X == SurfaceCacheDebugX)
				{
					bDebugThisTexel = true;
				}
	#endif
				const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
				if (TexelToVertex.TotalSampleWeight > 0.0f)
				{
					FFullStaticLightingVertex Vertex = TexelToVertex.GetFullVertex();
					Vertex.ApplyVertexModifications(TexelToVertex.ElementIndex, MaterialSettings.bUseNormalMapsForLighting, TextureMapping->Mesh);
					FFinalGatherSample SkyLighting;
					FFinalGatherSample UnusedSecondLighting;
					float UnusedBackfacingHitsFraction;

					if (!RadiosityCache.InterpolateLighting(Vertex, true, false, 1.0f, SkyLighting, UnusedSecondLighting, UnusedBackfacingHitsFraction, MappingContext.DebugCacheRecords))
					{
						FFinalGatherSample UniformSampledIncomingRadiance;
						TArray<FVector4f> ImportancePhotonDirections;
						FLightingCacheGatherInfo GatherInfo;
						const int32 NumAdaptiveRefinementLevels = GeneralSettings.IndirectLightingQuality <= 10 ? 1 : 2;

						if (bCacheFinalGatherHitPoints)
						{
							GatherHitPoints.GatherHitPointRanges.Add(FArrayRange(GatherHitPoints.GatherHitPointData.Num()));
							GatherInfo.HitPointRecorder = &GatherHitPoints;
						}

						UniformSampledIncomingRadiance = IncomingRadianceAdaptive<FFinalGatherSample>(
							TextureMapping, 
							Vertex, 
							TexelToVertex.SampleRadius, 
							false,
							TexelToVertex.ElementIndex, 
							2,  /** BounceNumber */
							RBM_ConstantNormalOffset,
							GLM_GatherLightEmitted, /* Gather sky light and emissive only */
							NumAdaptiveRefinementLevels,
							1.0f,
							CachedHemisphereSamplesForRadiosity[0],
							CachedHemisphereSamplesForRadiosityUniforms[0],
							1,
							ImportancePhotonDirections, 
							MappingContext, 
							RandomStream, 
							GatherInfo, 
							true, /* bGatheringForCachedDirectLighting */
							false);

						float OverrideRadius = 0;

						TLightingCache<FFinalGatherSample>::FRecord<FFinalGatherSample> NewRecord(
							Vertex,
							TexelToVertex.ElementIndex,
							GatherInfo,
							TexelToVertex.SampleRadius,
							OverrideRadius,
							IrradianceCachingSettings,
							GeneralSettings,
							UniformSampledIncomingRadiance,
							FVector4f(0, 0, 0, 0),
							FVector4f(0, 0, 0, 0)
							);

						// Add the incident radiance sample to the cache.
						RadiosityCache.AddRecord(NewRecord, false, false);
					}
				}
			}
		}
	}

	FInfluencingRecords& InfluencingRecords = TextureMapping->InfluencingRecordsSurfaceCache;

	if (bCacheFinalGatherHitPoints)
	{
		InfluencingRecords.Ranges.Empty(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);
		InfluencingRecords.Ranges.AddZeroed(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);
		InfluencingRecords.Data.Empty(GatherHitPoints.GatherHitPointData.Num());
	}

	for (int32 Y = 0; Y < TextureMapping->SurfaceCacheSizeY; Y++)
	{
		for (int32 X = 0; X < TextureMapping->SurfaceCacheSizeX; X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == SurfaceCacheDebugY
				&& X == SurfaceCacheDebugX)
			{
				bDebugThisTexel = true;
			}
#endif
			const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
			if (TexelToVertex.TotalSampleWeight > 0.0f)
			{
				FFullStaticLightingVertex CurrentVertex = TexelToVertex.GetFullVertex();

				CurrentVertex.ApplyVertexModifications(TexelToVertex.ElementIndex, MaterialSettings.bUseNormalMapsForLighting, TextureMapping->Mesh);

				const int32 SurfaceCacheIndex = Y * TextureMapping->SurfaceCacheSizeX + X;

				FInfluencingRecordCollector RecordCollector(InfluencingRecords, SurfaceCacheIndex);
				FInfluencingRecordCollector* RecordCollectorPtr = NULL;

				if (bCacheFinalGatherHitPoints)
				{
					RecordCollectorPtr = &RecordCollector;
					InfluencingRecords.Ranges[SurfaceCacheIndex] = FArrayRange(InfluencingRecords.Data.Num());
				}

				FLinearColor IncidentLighting = FLinearColor::Black;
				FLinearColor IncidentLightingForRadiosity = FLinearColor::Black;

				if (GeneralSettings.NumSkyLightingBounces > 0)
				{
					FFinalGatherSample SkyLighting;
					FFinalGatherSample UnusedSecondLighting;
					float UnusedBackfacingHitsFraction;
					RadiosityCache.InterpolateLighting(CurrentVertex, false, false, IrradianceCachingSettings.SkyOcclusionSmoothnessReduction, SkyLighting, UnusedSecondLighting, UnusedBackfacingHitsFraction, MappingContext.DebugCacheRecords, RecordCollectorPtr);

					if (GeneralSettings.ViewSingleBounceNumber < 0 || GeneralSettings.ViewSingleBounceNumber == 1)
					{
						IncidentLighting += SkyLighting.IncidentLighting + SkyLighting.StationarySkyLighting.IncidentLighting;
					}

					IncidentLightingForRadiosity += SkyLighting.IncidentLighting + SkyLighting.StationarySkyLighting.IncidentLighting;
				}
				
				if (ImportanceTracingSettings.bUseRadiositySolverForLightMultibounce)
				{
					FGatheredLightSample DirectLighting;
					float Unused2;
					TArray<FVector3f, TInlineAllocator<1>> VertexOffsets;
					VertexOffsets.Add(FVector3f(0, 0, 0));

					CalculateApproximateDirectLighting(CurrentVertex, TexelToVertex.TexelRadius, VertexOffsets, .1f, true, true, bDebugThisTexel, MappingContext, DirectLighting, Unused2);

					IncidentLightingForRadiosity += DirectLighting.IncidentLighting;
				}
				
				TextureMapping->SurfaceCacheLighting[SurfaceCacheIndex] = IncidentLighting;
				TextureMapping->RadiositySurfaceCache[0][SurfaceCacheIndex] = IncidentLightingForRadiosity;
			}
		}
	}

	if (bCacheFinalGatherHitPoints && bCompressRadiosityCachedData)
	{
		TextureMapping->CompressedGatherHitPoints.Compress(GatherHitPoints);
		GatherHitPoints = FGatherHitPoints();

		TextureMapping->CompressedInfluencingRecords.Compress(InfluencingRecords);
		InfluencingRecords = FInfluencingRecords();
	}

#endif
}

/** */
void FStaticLightingSystem::RunRadiosityIterations()
{
	// First bounce done in setup
	const int32 NumRadiosityIterations = FMath::Max(GeneralSettings.NumSkyLightingBounces - 1, 0);

	if (NumRadiosityIterations > 0)
	{
		const double RadiosityStartTime = FPlatformTime::Seconds();

		for(int32 ThreadIndex = 1; ThreadIndex < NumStaticLightingThreads; ThreadIndex++)
		{
			FMappingProcessingThreadRunnable* ThreadRunnable = new FMappingProcessingThreadRunnable(this, ThreadIndex, StaticLightingTask_RadiosityIterations);
			RadiosityIterationThreads.Add(ThreadRunnable);
			const FString ThreadName = FString::Printf(TEXT("RadiosityIterationThread%u"), ThreadIndex);
			ThreadRunnable->Thread = FRunnableThread::Create(ThreadRunnable, *ThreadName);
		}

		// Start the static lighting thread loop on the main thread, too.
		// Once it returns, all static lighting mappings have begun processing.
		RadiosityIterationThreadLoop(0, true);

		// Stop the static lighting threads.
		for(int32 ThreadIndex = 0;ThreadIndex < RadiosityIterationThreads.Num();ThreadIndex++)
		{
			// Wait for the thread to exit.
			RadiosityIterationThreads[ThreadIndex].Thread->WaitForCompletion();
			// Check that it didn't terminate with an error.
			RadiosityIterationThreads[ThreadIndex].CheckHealth();

			// Destroy the thread.
			delete RadiosityIterationThreads[ThreadIndex].Thread;
		}
		RadiosityIterationThreads.Empty();

		size_t TemporariesSize = 0;

		for (int32 MappingIndex = 0; MappingIndex < AllMappings.Num(); MappingIndex++)
		{
			TemporariesSize += AllMappings[MappingIndex]->FreeRadiosityTemporaries();
		}

		const float RadiosityDuration = FPlatformTime::Seconds() - RadiosityStartTime;
		LogSolverMessage(FString::Printf(TEXT("Radiosity Iterations %.1fs with %.1fMb of cached data"), RadiosityDuration, TemporariesSize / 1024.0f / 1024.0f));
	}
}

/** */
void FStaticLightingSystem::RadiosityIterationThreadLoop(int32 ThreadIndex, bool bIsMainThread)
{
	const int32 NumRadiosityIterations = FMath::Max(GeneralSettings.NumSkyLightingBounces - 1, 0);

	bool bIsDone = false;
	while (!bIsDone)
	{
		// Atomically read and increment the next mapping index to process.
		const int32 TaskIndex = NextMappingToProcessRadiosityIterations.Increment() - 1;
		
		if (TaskIndex < AllMappings.Num() * NumRadiosityIterations)
		{
			const int32 PassIndex = TaskIndex / AllMappings.Num();
			const int32 MappingIndex = TaskIndex - PassIndex * AllMappings.Num();

			// If this is the main thread, update progress and apply completed static lighting.
			if (bIsMainThread)
			{
				// Check the health of all static lighting threads.
				for (int32 ThreadIndexIter = 0; ThreadIndexIter < RadiosityIterationThreads.Num(); ThreadIndexIter++)
				{
					RadiosityIterationThreads[ThreadIndexIter].CheckHealth();
				}
			}

			if (PassIndex > 0)
			{
				// Sleep-loop until other threads have completed the previous pass
				while (NumCompletedRadiosityIterationMappings[PassIndex - 1].GetValue() < AllMappings.Num())
				{
					FPlatformProcess::Sleep(0.0f);
				}
			}

			FStaticLightingTextureMapping* TextureMapping = AllMappings[MappingIndex]->GetTextureMapping();

			if (TextureMapping)
			{
				RadiosityIterationTextureMapping(TextureMapping, PassIndex);

				// Make sure writes to mapping data are seen on other threads before the change to NumCompletedRadiosityIterationMappings
				FPlatformMisc::MemoryBarrier();
				NumCompletedRadiosityIterationMappings[PassIndex].Increment();
			}
		}
		else
		{
			// Processing has begun for all mappings.
			bIsDone = true;
		}
	}

	GSwarm->SendMessage( NSwarm::FTimingMessage( NSwarm::PROGSTATE_Preparing4, ThreadIndex ) );
}

void FStaticLightingSystem::RadiosityIterationTextureMapping(FStaticLightingTextureMapping* TextureMapping, int32 PassIndex)
{
	checkSlow(TextureMapping);
	FStaticLightingMappingContext MappingContext(TextureMapping->Mesh,*this,&StartupDebugOutput);
	LIGHTINGSTAT(FScopedRDTSCTimer CachingTime(MappingContext.Stats.RadiosityIterationThreadTime));
	const FBoxSphereBounds3f ImportanceBounds = Scene.GetImportanceBounds();

	FTexelToVertexMap TexelToVertexMap(TextureMapping->SurfaceCacheSizeX, TextureMapping->SurfaceCacheSizeY);

	bool bDebugThisMapping = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	bDebugThisMapping = TextureMapping == Scene.DebugMapping;
	int32 TexelDebugX = -1;
	int32 TexelDebugY = -1;
	if (bDebugThisMapping)
	{
		TexelDebugX = FMath::TruncToInt(Scene.DebugInput.LocalX / (float)TextureMapping->CachedSizeX * TextureMapping->SurfaceCacheSizeX);
		TexelDebugY = FMath::TruncToInt(Scene.DebugInput.LocalY / (float)TextureMapping->CachedSizeY * TextureMapping->SurfaceCacheSizeY);
	}
#endif

	RasterizeToSurfaceCacheTextureMapping(TextureMapping, bDebugThisMapping, TexelToVertexMap);

	if (ImportanceTracingSettings.bCacheFinalGatherHitPointsForRadiosity)
	{
		RadiosityIterationCachedHitpointsTextureMapping(TexelToVertexMap, TextureMapping, PassIndex);
	}
	else
	{
		const int32 SourceRadiosityBufferIndex = PassIndex % 2;
		const int32 DestRadiosityBufferIndex = 1 - SourceRadiosityBufferIndex;

		int32 NumAdaptiveRefinementLevels = PassIndex == 0 ? 1 : 0;

		if (GeneralSettings.IndirectLightingQuality > 10)
		{
			NumAdaptiveRefinementLevels++;
		}

		const int32 RadiositySampleSet = FMath::Min<int32>(PassIndex, UE_ARRAY_COUNT(CachedHemisphereSamplesForRadiosity) - 1);

		TLightingCache<FFinalGatherSample> RadiosityCache(TextureMapping->Mesh->BoundingBox, *this, 1);
	
		FLMRandomStream RandomStream(0);

		for (int32 Y = 0; Y < TextureMapping->SurfaceCacheSizeY; Y++)
		{
			for (int32 X = 0; X < TextureMapping->SurfaceCacheSizeX; X++)
			{
				bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& Y == TexelDebugY
					&& X == TexelDebugX)
				{
					bDebugThisTexel = true;
				}
#endif
				const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
				if (TexelToVertex.TotalSampleWeight > 0.0f)
				{
					FFullStaticLightingVertex Vertex = TexelToVertex.GetFullVertex();
					Vertex.ApplyVertexModifications(TexelToVertex.ElementIndex, MaterialSettings.bUseNormalMapsForLighting, TextureMapping->Mesh);
					FFinalGatherSample SkyLighting;
					FFinalGatherSample UnusedSecondLighting;
					float UnusedBackfacingHitsFraction;

					if (!RadiosityCache.InterpolateLighting(Vertex, true, false, 1.0f, SkyLighting, UnusedSecondLighting, UnusedBackfacingHitsFraction, MappingContext.DebugCacheRecords))
					{
						FFinalGatherSample UniformSampledIncomingRadiance;
						//@todo - find and pass in photons from the appropriate bounce number to improve bUseRadiositySolverForLightMultibounce quality
						TArray<FVector4f> ImportancePhotonDirections;
						FLightingCacheGatherInfo GatherInfo;

						// Gather previous iteration results (double buffer)
						EHemisphereGatherClassification GatherClassification = SourceRadiosityBufferIndex == 0 ? GLM_GatherRadiosityBuffer0 : GLM_GatherRadiosityBuffer1;

						UniformSampledIncomingRadiance = IncomingRadianceAdaptive<FFinalGatherSample>(
							TextureMapping, 
							Vertex, 
							TexelToVertex.SampleRadius, 
							false,
							TexelToVertex.ElementIndex, 
							PassIndex + 3, /** BounceNumber */
							RBM_ConstantNormalOffset,
							GatherClassification, 
							NumAdaptiveRefinementLevels,
							1.0f,
							CachedHemisphereSamplesForRadiosity[RadiositySampleSet],
							CachedHemisphereSamplesForRadiosityUniforms[RadiositySampleSet],
							1,
							ImportancePhotonDirections, 
							MappingContext, 
							RandomStream, 
							GatherInfo, 
							true, /* bGatheringForCachedDirectLighting */
							false);

						float OverrideRadius = 0;

						TLightingCache<FFinalGatherSample>::FRecord<FFinalGatherSample> NewRecord(
							Vertex,
							TexelToVertex.ElementIndex,
							GatherInfo,
							TexelToVertex.SampleRadius,
							OverrideRadius,
							IrradianceCachingSettings,
							GeneralSettings,
							UniformSampledIncomingRadiance,
							FVector4f(0, 0, 0, 0),
							FVector4f(0, 0, 0, 0)
							);

						// Add the incident radiance sample to the cache.
						RadiosityCache.AddRecord(NewRecord, false, false);
					}
				}
			}
		}

		for (int32 Y = 0; Y < TextureMapping->SurfaceCacheSizeY; Y++)
		{
			for (int32 X = 0; X < TextureMapping->SurfaceCacheSizeX; X++)
			{
				bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& Y == TexelDebugY
					&& X == TexelDebugX)
				{
					bDebugThisTexel = true;
				}
#endif
				const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
				if (TexelToVertex.TotalSampleWeight > 0.0f)
				{
					FFullStaticLightingVertex CurrentVertex = TexelToVertex.GetFullVertex();

					CurrentVertex.ApplyVertexModifications(TexelToVertex.ElementIndex, MaterialSettings.bUseNormalMapsForLighting, TextureMapping->Mesh);

					const int32 SurfaceCacheIndex = Y * TextureMapping->SurfaceCacheSizeX + X;

					FFinalGatherSample IterationLighting;
					FFinalGatherSample UnusedSecondLighting;
					float UnusedBackfacingHitsFraction;
					RadiosityCache.InterpolateLighting(CurrentVertex, false, false, IrradianceCachingSettings.SkyOcclusionSmoothnessReduction, IterationLighting, UnusedSecondLighting, UnusedBackfacingHitsFraction, MappingContext.DebugCacheRecords);
				
					const bool bIsTranslucent = TextureMapping->Mesh->IsTranslucent(TexelToVertex.ElementIndex);
					const FLinearColor Reflectance = (bIsTranslucent ? FLinearColor::Black : TextureMapping->Mesh->EvaluateTotalReflectance(CurrentVertex, TexelToVertex.ElementIndex)) * (float)INV_PI;
					const FLinearColor IterationRadiosity = IterationLighting.IncidentLighting * Reflectance;

					if (GeneralSettings.ViewSingleBounceNumber < 0 || GeneralSettings.ViewSingleBounceNumber == PassIndex + 2)
					{
						// Accumulate this bounce's lighting
						TextureMapping->SurfaceCacheLighting[SurfaceCacheIndex] += IterationRadiosity;
					}
				
					// Store in one of the radiosity buffers for the next iteration
					TextureMapping->RadiositySurfaceCache[DestRadiosityBufferIndex][SurfaceCacheIndex] = IterationRadiosity;
				}
			}
		}
	}
}

void FStaticLightingSystem::RadiosityIterationCachedHitpointsTextureMapping(const FTexelToVertexMap& TexelToVertexMap, FStaticLightingTextureMapping* TextureMapping, int32 PassIndex)
{
	const int32 SourceRadiosityBufferIndex = PassIndex % 2;
	const int32 DestRadiosityBufferIndex = 1 - SourceRadiosityBufferIndex;

	FGatherHitPoints LocalGatherHitPoints;

	if (bCompressRadiosityCachedData)
	{
		TextureMapping->CompressedGatherHitPoints.Decompress(LocalGatherHitPoints);
	}

	const FGatherHitPoints& GatherHitPoints = bCompressRadiosityCachedData ? LocalGatherHitPoints : TextureMapping->UncompressedGatherHitPoints;

	TArray<FLinearColor> IterationRecordRadiosity;
	IterationRecordRadiosity.Empty(GatherHitPoints.GatherHitPointRanges.Num());
	IterationRecordRadiosity.AddUninitialized(GatherHitPoints.GatherHitPointRanges.Num());

	for (int32 LightingCacheRecordIndex = 0; LightingCacheRecordIndex < GatherHitPoints.GatherHitPointRanges.Num(); LightingCacheRecordIndex++)
	{
		FLinearColor NewRadiosity = FLinearColor::Black;

		const FArrayRange& HitPointRange = GatherHitPoints.GatherHitPointRanges[LightingCacheRecordIndex];

		for (int32 HitPointIndex = HitPointRange.StartIndex; HitPointIndex < HitPointRange.StartIndex + HitPointRange.NumEntries; HitPointIndex++)
		{
			const FFinalGatherHitPoint& HitPoint = GatherHitPoints.GatherHitPointData[HitPointIndex];
			const FStaticLightingMapping* HitTextureMapping = AllMappings[HitPoint.MappingIndex];
			const FLinearColor IncomingRadiance = HitTextureMapping->GetCachedRadiosity(SourceRadiosityBufferIndex, HitPoint.MappingSurfaceCoordinate);

			NewRadiosity += IncomingRadiance * HitPoint.Weight.GetFloat();
		}

		IterationRecordRadiosity[LightingCacheRecordIndex] = NewRadiosity;
	}

	FInfluencingRecords LocalInfluencingRecords;

	if (bCompressRadiosityCachedData)
	{
		TextureMapping->CompressedInfluencingRecords.Decompress(LocalInfluencingRecords);
	}

	const FInfluencingRecords& InfluencingRecords = bCompressRadiosityCachedData ? LocalInfluencingRecords : TextureMapping->InfluencingRecordsSurfaceCache;

	for (int32 Y = 0; Y < TextureMapping->SurfaceCacheSizeY; Y++)
	{
		for (int32 X = 0; X < TextureMapping->SurfaceCacheSizeX; X++)
		{
			const FTexelToVertex& TexelToVertex = TexelToVertexMap(X, Y);
			if (TexelToVertex.TotalSampleWeight > 0.0f)
			{
				FFullStaticLightingVertex CurrentVertex = TexelToVertex.GetFullVertex();

				CurrentVertex.ApplyVertexModifications(TexelToVertex.ElementIndex, MaterialSettings.bUseNormalMapsForLighting, TextureMapping->Mesh);

				const int32 SurfaceCacheIndex = Y * TextureMapping->SurfaceCacheSizeX + X;
				FArrayRange Range = InfluencingRecords.Ranges[SurfaceCacheIndex];

				float TotalWeight = 0.0f;
				FLinearColor AccumulatedRadiosity = FLinearColor::Black;

				for (int32 InfluenceIndex = Range.StartIndex; InfluenceIndex < Range.StartIndex + Range.NumEntries; InfluenceIndex++)
				{
					FInfluencingRecord InfluencingRecord = InfluencingRecords.Data[InfluenceIndex];
					float RecordWeight = InfluencingRecord.RecordWeight;
					AccumulatedRadiosity += IterationRecordRadiosity[InfluencingRecord.RecordIndex] * RecordWeight;
					TotalWeight += RecordWeight;
				}

				const bool bIsTranslucent = TextureMapping->Mesh->IsTranslucent(TexelToVertex.ElementIndex);
				const FLinearColor DiffuseReflectance = (bIsTranslucent ? FLinearColor::Black : TextureMapping->Mesh->EvaluateTotalReflectance(CurrentVertex, TexelToVertex.ElementIndex)) * (float)INV_PI;
				checkSlow(TotalWeight > 0.0f);

				FLinearColor IterationRadiosity = (AccumulatedRadiosity / TotalWeight) * DiffuseReflectance;
				
				if (GeneralSettings.ViewSingleBounceNumber < 0 || GeneralSettings.ViewSingleBounceNumber == PassIndex + 2)
				{
					// Accumulate this bounce's lighting
					TextureMapping->SurfaceCacheLighting[SurfaceCacheIndex] += IterationRadiosity;
				}

				// Store in one of the radiosity buffers for the next iteration
				TextureMapping->RadiositySurfaceCache[DestRadiosityBufferIndex][SurfaceCacheIndex] = IterationRadiosity;
			}
		}
	}
}

size_t FStaticLightingMapping::FreeRadiosityTemporaries()
{
	size_t RadiosityTemporariesSize = 0;
	RadiosityTemporariesSize += RadiositySurfaceCache[0].GetAllocatedSize() + RadiositySurfaceCache[1].GetAllocatedSize();
	RadiosityTemporariesSize += CompressedGatherHitPoints.GetAllocatedSize() + UncompressedGatherHitPoints.GetAllocatedSize();
	RadiosityTemporariesSize += CompressedInfluencingRecords.GetAllocatedSize() + InfluencingRecordsSurfaceCache.GetAllocatedSize();

	RadiositySurfaceCache[0].Empty();
	RadiositySurfaceCache[1].Empty();

	CompressedGatherHitPoints = FCompressedGatherHitPoints();
	UncompressedGatherHitPoints = FGatherHitPoints();
	CompressedInfluencingRecords = FCompressedInfluencingRecords();
	InfluencingRecordsSurfaceCache = FInfluencingRecords();

	return RadiosityTemporariesSize;
}

}
