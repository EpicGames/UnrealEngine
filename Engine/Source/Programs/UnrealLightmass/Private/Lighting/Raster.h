// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Lightmass
{

//
//	FTriangleRasterizer - A generic 2d triangle rasterizer. It inherits a templated policy class (RasterPolicyType) and interpolates vertices according to its parameters.
//	RasterPolicyType::ProcessPixel() is called for each rendered pixel.
//
//	RasterPolicyType requires the following properties/methods exist in the templated super class:-
//  RasterPolicyType::InterpolantType
//  RasterPolicyType::GetMinX(),RasterPolicyType::GetMaxX()
//  RasterPolicyType::GetMinY(),RasterPolicyType::GetMaxY()
//	RasterPolicyType::ProcessPixel(int32 X, int32 Y, const InterpolantType& InterpolantValues, bool BackFacing);

template<class RasterPolicyType> class FTriangleRasterizer : public RasterPolicyType
{
public:

	typedef typename RasterPolicyType::InterpolantType InterpolantType;

	void DrawTriangle(const InterpolantType& I0,const InterpolantType& I1,const InterpolantType& I2,const FVector2f& P0,const FVector2f& P1,const FVector2f& P2,bool BackFacing)
	{
		InterpolantType	Interpolants[3] = { I0, I1, I2 };
		FVector2f		Points[3] = { P0, P1, P2 };

		// Find the top point.

		if(Points[1].Y < Points[0].Y && Points[1].Y <= Points[2].Y)
		{
			Exchange(Points[0],Points[1]);
			Exchange(Interpolants[0],Interpolants[1]);
		}
		else if(Points[2].Y < Points[0].Y && Points[2].Y <= Points[1].Y)
		{
			Exchange(Points[0],Points[2]);
			Exchange(Interpolants[0],Interpolants[2]);
		}

		// Find the bottom point.

		if(Points[1].Y > Points[2].Y)
		{
			Exchange(Points[2],Points[1]);
			Exchange(Interpolants[2],Interpolants[1]);
		}

		// Calculate the edge gradients.

		float			TopMinDiffX = (Points[1].X - Points[0].X) / (Points[1].Y - Points[0].Y),
						TopMaxDiffX = (Points[2].X - Points[0].X) / (Points[2].Y - Points[0].Y);
		InterpolantType	TopMinDiffInterpolant = (Interpolants[1] - Interpolants[0]) / (Points[1].Y - Points[0].Y),
						TopMaxDiffInterpolant = (Interpolants[2] - Interpolants[0]) / (Points[2].Y - Points[0].Y);

		float			BottomMinDiffX = (Points[2].X - Points[1].X) / (Points[2].Y - Points[1].Y),
						BottomMaxDiffX = (Points[2].X - Points[0].X) / (Points[2].Y - Points[0].Y);
		InterpolantType	BottomMinDiffInterpolant = (Interpolants[2] - Interpolants[1]) / (Points[2].Y - Points[1].Y),
						BottomMaxDiffInterpolant = (Interpolants[2] - Interpolants[0]) / (Points[2].Y - Points[0].Y);

		DrawTriangleTrapezoid(
			Interpolants[0],
			TopMinDiffInterpolant,
			Interpolants[0],
			TopMaxDiffInterpolant,
			Points[0].X,
			TopMinDiffX,
			Points[0].X,
			TopMaxDiffX,
			Points[0].Y,
			Points[1].Y,
			BackFacing
			);

		DrawTriangleTrapezoid(
			Interpolants[1],
			BottomMinDiffInterpolant,
			Interpolants[0] + TopMaxDiffInterpolant * (Points[1].Y - Points[0].Y),
			BottomMaxDiffInterpolant,
			Points[1].X,
			BottomMinDiffX,
			Points[0].X + TopMaxDiffX * (Points[1].Y - Points[0].Y),
			BottomMaxDiffX,
			Points[1].Y,
			Points[2].Y,
			BackFacing
			);
	}

	FTriangleRasterizer(const RasterPolicyType& InRasterPolicy): RasterPolicyType(InRasterPolicy) {}

private:

	void DrawTriangleTrapezoid(
		const InterpolantType& TopMinInterpolant,
		const InterpolantType& DeltaMinInterpolant,
		const InterpolantType& TopMaxInterpolant,
		const InterpolantType& DeltaMaxInterpolant,
		float TopMinX,
		float DeltaMinX,
		float TopMaxX,
		float DeltaMaxX,
		float MinY,
		float MaxY,
		bool BackFacing
		)
	{
		int32	IntMinY = FMath::Clamp(FMath::CeilToInt(MinY),RasterPolicyType::GetMinY(),RasterPolicyType::GetMaxY() + 1),
			IntMaxY = FMath::Clamp(FMath::CeilToInt(MaxY),RasterPolicyType::GetMinY(),RasterPolicyType::GetMaxY() + 1);

		for(int32 IntY = IntMinY;IntY < IntMaxY;IntY++)
		{
			float			Y = IntY - MinY,
							MinX = TopMinX + DeltaMinX * Y,
							MaxX = TopMaxX + DeltaMaxX * Y;
			InterpolantType	MinInterpolant = TopMinInterpolant + DeltaMinInterpolant * Y,
							MaxInterpolant = TopMaxInterpolant + DeltaMaxInterpolant * Y;

			if(MinX > MaxX)
			{
				Exchange(MinX,MaxX);
				Exchange(MinInterpolant,MaxInterpolant);
			}

			if(MaxX > MinX)
			{
				int32				IntMinX = FMath::Clamp(FMath::CeilToInt(MinX),RasterPolicyType::GetMinX(),RasterPolicyType::GetMaxX() + 1),
								IntMaxX = FMath::Clamp(FMath::CeilToInt(MaxX),RasterPolicyType::GetMinX(),RasterPolicyType::GetMaxX() + 1);
				InterpolantType	DeltaInterpolant = (MaxInterpolant - MinInterpolant) / (MaxX - MinX);

				for(int32 X = IntMinX;X < IntMaxX;X++)
				{
					RasterPolicyType::ProcessPixel(X,IntY,MinInterpolant + DeltaInterpolant * (X - MinX),BackFacing);
				}
			}
		}
	}
};



}
