// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LightingSystem.h"
#include "Raster.h"
#include "MonteCarlo.h"
#include "HAL/PlatformTime.h"
#include "UnrealLightmass.h"

namespace Lightmass
{

typedef FVolumeSampleInterpolationElement FVolumeSampleProximityElement;

typedef TOctree<FVolumeSampleProximityElement,struct FVolumeLightingProximityOctreeSemantics> FVolumeLightingProximityOctree;

struct FVolumeLightingProximityOctreeSemantics
{
	//@todo - evaluate different performance/memory tradeoffs with these
	enum { MaxElementsPerLeaf = 4 };
	enum { MaxNodeDepth = 12 };
	enum { LoosenessDenominator = 16 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	static FBoxCenterAndExtent GetBoundingBox(const FVolumeSampleProximityElement& Element)
	{
		const FVolumeLightingSample& Sample = Element.VolumeSamples[Element.SampleIndex];
		return FBoxCenterAndExtent(FVector4f(Sample.PositionAndRadius, 0.0f), FVector4f(0,0,0));
	}
};

void FVolumeLightingSample::SetFromSHVector(const FSHVectorRGB3& SHVector)
{
	for (int32 CoefficientIndex = 0; CoefficientIndex < LM_NUM_SH_COEFFICIENTS; CoefficientIndex++)
	{
		HighQualityCoefficients[CoefficientIndex][0] = SHVector.R.V[CoefficientIndex];
		HighQualityCoefficients[CoefficientIndex][1] = SHVector.G.V[CoefficientIndex];
		HighQualityCoefficients[CoefficientIndex][2] = SHVector.B.V[CoefficientIndex];
	}
}

/** Constructs an SH environment from this lighting sample. */
void FVolumeLightingSample::ToSHVector(FSHVectorRGB3& SHVector) const
{
	for (int32 CoefficientIndex = 0; CoefficientIndex < LM_NUM_SH_COEFFICIENTS; CoefficientIndex++)
	{
		SHVector.R.V[CoefficientIndex] = HighQualityCoefficients[CoefficientIndex][0];
		SHVector.G.V[CoefficientIndex] = HighQualityCoefficients[CoefficientIndex][1];
		SHVector.B.V[CoefficientIndex] = HighQualityCoefficients[CoefficientIndex][2];
	}
}

/** Returns true if there is an existing sample in VolumeOctree within SearchDistance of Position. */
static bool FindNearbyVolumeSample(const FVolumeLightingProximityOctree& VolumeOctree, const FVector4f& Position, float SearchDistance)
{
	const FBox3f SearchBox = FBox3f::BuildAABB(Position, FVector4f(SearchDistance, SearchDistance, SearchDistance));
	for (FVolumeLightingProximityOctree::TConstIterator<> OctreeIt(VolumeOctree); OctreeIt.HasPendingNodes(); OctreeIt.Advance())
	{
		const FVolumeLightingProximityOctree::FNode& CurrentNode = OctreeIt.GetCurrentNode();
		const FOctreeNodeContext& CurrentContext = OctreeIt.GetCurrentContext();
		{
			// Push children onto the iterator stack if they intersect the query box
			if (!CurrentNode.IsLeaf())
			{
				FOREACH_OCTREE_CHILD_NODE(ChildRef)
				{
					if (CurrentNode.HasChild(ChildRef))
					{
						const FOctreeNodeContext ChildContext = CurrentContext.GetChildContext(ChildRef);
						if (ChildContext.Bounds.GetBox().Intersect(SearchBox))
						{
							OctreeIt.PushChild(ChildRef);
						}
					}
				}
			}
		}

		// Iterate over all samples in the nodes intersecting the query box
		for (FVolumeLightingProximityOctree::ElementConstIt It(CurrentNode.GetConstElementIt()); It; ++It)
		{
			const FVolumeSampleProximityElement& Element = *It;
			const float DistanceSquared = (Element.VolumeSamples[Element.SampleIndex].GetPosition() - Position).SizeSquared3();
			if (DistanceSquared < SearchDistance * SearchDistance)
			{
				return true;
			}
		}
	}
	return false;
}

class FVolumeSamplePlacementRasterPolicy
{
public:

	typedef FStaticLightingVertex InterpolantType;

	/** Initialization constructor. */
	FVolumeSamplePlacementRasterPolicy(
		int32 InSizeX, 
		int32 InSizeY, 
		float InMinSampleDistance, 
		float InSceneBoundingRadius,
		float InSampleRadius,
		FStaticLightingSystem& InSystem,
		FCoherentRayCache& InCoherentRayCache,
		FVolumeLightingProximityOctree& InProximityOctree)
		:
		SizeX(InSizeX),
		SizeY(InSizeY),
		MinSampleDistance(InMinSampleDistance),
		SceneBoundingRadius(InSceneBoundingRadius),
		SampleRadius(InSampleRadius),
		System(InSystem),
		CoherentRayCache(InCoherentRayCache),
		ProximityOctree(InProximityOctree)
	{
		LayerHeightOffsets.Empty(System.DynamicObjectSettings.NumSurfaceSampleLayers);
		LayerHeightOffsets.Add(System.DynamicObjectSettings.FirstSurfaceSampleLayerHeight);
		for (int32 i = 1; i < System.DynamicObjectSettings.NumSurfaceSampleLayers; i++)
		{
			LayerHeightOffsets.Add(System.DynamicObjectSettings.FirstSurfaceSampleLayerHeight + i * System.DynamicObjectSettings.SurfaceSampleLayerHeightSpacing);
		}

		
		TArray<FVector2f> UniformHemisphereSampleUniforms;
		const int32 NumUpperVolumeSamples = 16;
		const float NumThetaStepsFloat = FMath::Sqrt(NumUpperVolumeSamples / (float)PI);
		const int32 NumThetaSteps = FMath::TruncToInt(NumThetaStepsFloat);
		const int32 NumPhiSteps = FMath::TruncToInt(NumThetaStepsFloat * (float)PI);
		FLMRandomStream RandomStream(0);

		GenerateStratifiedUniformHemisphereSamples(NumThetaSteps, NumPhiSteps, RandomStream, UniformHemisphereSamples, UniformHemisphereSampleUniforms);
	}

	void SetLevelGuid(FGuid InLevelGuid)
	{
		LevelGuid = InLevelGuid;
	}

protected:

	// FTriangleRasterizer policy interface.

	int32 GetMinX() const { return 0; }
	int32 GetMaxX() const { return SizeX; }
	int32 GetMinY() const { return 0; }
	int32 GetMaxY() const { return SizeY; }

	void ProcessPixel(int32 X,int32 Y,const InterpolantType& Interpolant,bool BackFacing);

private:

	const int32 SizeX;
	const int32 SizeY;
	const float MinSampleDistance;
	const float SceneBoundingRadius;
	const float SampleRadius;
	FGuid LevelGuid;
	FStaticLightingSystem& System;
	FCoherentRayCache& CoherentRayCache;
	FVolumeLightingProximityOctree& ProximityOctree;
	TArray<float> LayerHeightOffsets;
	TArray<FVector4f> UniformHemisphereSamples;
};

int32 ComputeNumBackfacingHits(
	const FVector4f& SamplePosition, 
	FStaticLightingSystem& System, 
	FCoherentRayCache& CoherentRayCache, 
	float SceneBoundingRadius, 
	const TArray<FVector4f>& UniformHemisphereSamples)
{
	int NumBackfacingHits = 0;

	for (int32 SampleIndex = 0; SampleIndex < UniformHemisphereSamples.Num() * 2; SampleIndex++)
	{
		FVector4f SampleDirection = UniformHemisphereSamples[SampleIndex % UniformHemisphereSamples.Num()];
		SampleDirection.Z *= SampleIndex >= UniformHemisphereSamples.Num() ? -1.0f : 1.0f;

		const FLightRay PathRay(
			SamplePosition,
			SamplePosition + SampleDirection * SceneBoundingRadius,
			NULL,
			NULL
			);

		FLightRayIntersection RayIntersection;
		System.GetAggregateMesh().IntersectLightRay(PathRay, true, false, false, CoherentRayCache, RayIntersection);

		if (RayIntersection.bIntersects && Dot3(PathRay.Direction, -RayIntersection.IntersectionVertex.WorldTangentZ) <= 0.0f)
		{
			NumBackfacingHits++;
		}
	}

	return NumBackfacingHits;
}

void FVolumeSamplePlacementRasterPolicy::ProcessPixel(int32 X,int32 Y,const InterpolantType& Vertex,bool BackFacing)
{
	// Only place samples inside the scene's bounds
	if (System.IsPointInImportanceVolume(Vertex.WorldPosition))
	{
		// Place a sample for each layer
		for (int32 LayerIndex = 0; LayerIndex < LayerHeightOffsets.Num(); LayerIndex++)
		{
			const FVector4f SamplePosition = Vertex.WorldPosition + FVector4f(0, 0, LayerHeightOffsets[LayerIndex]);
			// Only place a sample if there isn't already one nearby, to avoid clumping
			if (!FindNearbyVolumeSample(ProximityOctree, SamplePosition, MinSampleDistance))
			{
				const int32 NumBackfacingHits = ComputeNumBackfacingHits(SamplePosition, System, CoherentRayCache, SceneBoundingRadius, UniformHemisphereSamples);

				// Only place a sample if we are outside of the level geometry (determined by whether we can see backfaces)
				if (NumBackfacingHits < .3f * UniformHemisphereSamples.Num() * 2)
				{
					TArray<FVolumeLightingSample>* VolumeLightingSamples = System.VolumeLightingSamples.Find(LevelGuid);
					check(VolumeLightingSamples);
					// Add a new sample for this layer
					VolumeLightingSamples->Add(FVolumeLightingSample(FVector4f(SamplePosition, SampleRadius)));
					// Add the sample to the proximity octree so we can avoid placing any more samples nearby
					ProximityOctree.AddElement(FVolumeSampleProximityElement(VolumeLightingSamples->Num() - 1, *VolumeLightingSamples));
					if (System.DynamicObjectSettings.bVisualizeVolumeLightInterpolation)
					{
						System.VolumeLightingInterpolationOctree.AddElement(FVolumeSampleInterpolationElement(VolumeLightingSamples->Num() - 1, *VolumeLightingSamples));
					}
				}
			}
		}
	}
}

/** Places volume lighting samples and calculates lighting for them. */
void FStaticLightingSystem::BeginCalculateVolumeSamples()
{
	const double VolumeSampleStartTime = FPlatformTime::Seconds();
	VolumeBounds = GetImportanceBounds(false);
	if (VolumeBounds.SphereRadius < DELTA)
	{
		VolumeBounds = FBoxSphereBounds3f(AggregateMesh->GetBounds());
	}

	// Only place samples if the volume has area
	if (VolumeBounds.BoxExtent.X > 0.0f && VolumeBounds.BoxExtent.Y > 0.0f && VolumeBounds.BoxExtent.Z > 0.0f)
	{
		float LandscapeEstimateNum = 0.f;
		// Estimate Light sample number near Landscape surfaces
		if (DynamicObjectSettings.bUseMaxSurfaceSampleNum && DynamicObjectSettings.MaxSurfaceLightSamples > 100)
		{
			float SquaredSpacing = FMath::Square(DynamicObjectSettings.SurfaceLightSampleSpacing);
			if (SquaredSpacing == 0.f) SquaredSpacing = 1.0f;
			for (int32 MappingIndex = 0; MappingIndex < LandscapeMappings.Num(); MappingIndex++)
			{
				FStaticLightingVertex Vertices[3];
				int32 ElementIndex;
				const FStaticLightingMapping* CurrentMapping = LandscapeMappings[MappingIndex];
				const FStaticLightingMesh* CurrentMesh = CurrentMapping->Mesh;
				CurrentMesh->GetTriangle((CurrentMesh->NumTriangles)>>1, Vertices[0], Vertices[1], Vertices[2], ElementIndex);
				// Only place inside the importance volume
				if (IsPointInImportanceVolume(Vertices[0].WorldPosition))
				{
					FVector4f TriangleNormal = (Vertices[2].WorldPosition - Vertices[0].WorldPosition) ^ (Vertices[1].WorldPosition - Vertices[0].WorldPosition);
					TriangleNormal.Z = 0.f; // approximate only for X-Y plane
					float TotalArea = 0.5f * TriangleNormal.Size3() * CurrentMesh->NumTriangles;
					LandscapeEstimateNum += TotalArea / FMath::Square(DynamicObjectSettings.SurfaceLightSampleSpacing);
				}
			}
			LandscapeEstimateNum *= DynamicObjectSettings.NumSurfaceSampleLayers;

			if (LandscapeEstimateNum > DynamicObjectSettings.MaxSurfaceLightSamples)
			{
				// Increase DynamicObjectSettings.SurfaceLightSampleSpacing to reduce light sample number
				float OldMaxSurfaceLightSamples = DynamicObjectSettings.SurfaceLightSampleSpacing;
				DynamicObjectSettings.SurfaceLightSampleSpacing = DynamicObjectSettings.SurfaceLightSampleSpacing * FMath::Sqrt((float)LandscapeEstimateNum / DynamicObjectSettings.MaxSurfaceLightSamples);
				UE_LOGF(LogLightmass, Log, "Too many LightSamples : DynamicObjectSettings.SurfaceLightSampleSpacing is increased from %g to %g", OldMaxSurfaceLightSamples, DynamicObjectSettings.SurfaceLightSampleSpacing);
				LandscapeEstimateNum = DynamicObjectSettings.MaxSurfaceLightSamples;
			}
		}

		//@todo - can this be presized more accurately?
		VolumeLightingSamples.Empty(FMath::Max<int32>(5000, LandscapeEstimateNum));
		FStaticLightingMappingContext MappingContext(nullptr, *this);
		// Octree used to keep track of where existing samples have been placed
		FVolumeLightingProximityOctree VolumeLightingOctree(VolumeBounds.Origin, VolumeBounds.BoxExtent.GetMax());
		// Octree used for interpolating lighting for debugging
		VolumeLightingInterpolationOctree = FVolumeLightingInterpolationOctree(VolumeBounds.Origin, VolumeBounds.BoxExtent.GetMax());
		// Determine the resolution that the scene should be rasterized at based on SurfaceLightSampleSpacing and the scene's extent
		const int32 RasterSizeX = FMath::TruncToInt(2.0f * VolumeBounds.BoxExtent.X / DynamicObjectSettings.SurfaceLightSampleSpacing);
		const int32 RasterSizeY = FMath::TruncToInt(2.0f * VolumeBounds.BoxExtent.Y / DynamicObjectSettings.SurfaceLightSampleSpacing);

		// Expand the radius to touch a diagonal sample on the grid for a little overlap
		const float DiagonalRadius = DynamicObjectSettings.SurfaceLightSampleSpacing * FMath::Sqrt(2.0f);
		// Make sure the space between layers is covered
		const float SampleRadius = FMath::Max(DiagonalRadius, DynamicObjectSettings.SurfaceSampleLayerHeightSpacing * FMath::Sqrt(2.0f));

		FTriangleRasterizer<FVolumeSamplePlacementRasterPolicy> Rasterizer(
			FVolumeSamplePlacementRasterPolicy(
			RasterSizeX, 
			RasterSizeY, 
			// Use a minimum sample distance slightly less than the SurfaceLightSampleSpacing
			0.9f * FMath::Min(DynamicObjectSettings.SurfaceLightSampleSpacing, DynamicObjectSettings.SurfaceSampleLayerHeightSpacing), 
			FBoxSphereBounds3f(AggregateMesh->GetBounds()).SphereRadius,
			SampleRadius,
			*this,
			MappingContext.RayCache,
			VolumeLightingOctree));

		check(Meshes.Num() == AllMappings.Num());
		// Rasterize all meshes in the scene and place high detail samples on their surfaces.
		// Iterate through mappings and retrieve the mesh from that, so we can make decisions based on whether the mesh is using texture or vertex lightmaps.
		for (int32 MappingIndex = 0; MappingIndex < AllMappings.Num(); MappingIndex++)
		{
			const FStaticLightingMapping* CurrentMapping = AllMappings[MappingIndex];
			const FStaticLightingTextureMapping* TextureMapping = CurrentMapping->GetTextureMapping();
			const FStaticLightingMesh* CurrentMesh = CurrentMapping->Mesh;

			const bool bMeshBelongsToLOD0 = CurrentMesh->DoesMeshBelongToLOD0();

			// Only place samples on shadow casting meshes.
			if ((CurrentMesh->LightingFlags & GI_INSTANCE_CASTSHADOW) && bMeshBelongsToLOD0)
			{
				// Create a new LevelId array if necessary
				if (!VolumeLightingSamples.Find(CurrentMesh->LevelGuid))
				{
					VolumeLightingSamples.Add(CurrentMesh->LevelGuid, TArray<FVolumeLightingSample>());
				}
				// Tell the rasterizer we are adding samples to this mesh's LevelId
				Rasterizer.SetLevelGuid(CurrentMesh->LevelGuid);
				// Rasterize all triangles in the mesh
				for (int32 TriangleIndex = 0; TriangleIndex < CurrentMesh->NumTriangles; TriangleIndex++)
				{
					FStaticLightingVertex Vertices[3];
					int32 ElementIndex;
					CurrentMesh->GetTriangle(TriangleIndex, Vertices[0], Vertices[1], Vertices[2], ElementIndex);

					if (CurrentMesh->IsElementCastingShadow(ElementIndex))
					{
						FVector2f XYPositions[3];
						for (int32 VertIndex = 0; VertIndex < 3; VertIndex++)
						{
							// Transform world space positions from [VolumeBounds.Origin - VolumeBounds.BoxExtent, VolumeBounds.Origin + VolumeBounds.BoxExtent] into [0,1]
							const FVector4f TransformedPosition = (Vertices[VertIndex].WorldPosition - FVector4f(VolumeBounds.Origin, 0.0f) + FVector4f(VolumeBounds.BoxExtent, 0.0f)) / (2.0f * FVector4f(VolumeBounds.BoxExtent, 1.0f));
							// Project positions onto the XY plane and scale to the resolution determined by DynamicObjectSettings.SurfaceLightSampleSpacing
							XYPositions[VertIndex] = FVector2f(TransformedPosition.X * RasterSizeX, TransformedPosition.Y * RasterSizeY);
						}

						const FVector4f TriangleNormal = (Vertices[2].WorldPosition - Vertices[0].WorldPosition) ^ (Vertices[1].WorldPosition - Vertices[0].WorldPosition);
						const float TriangleArea = 0.5f * TriangleNormal.Size3();

						if (TriangleArea > DELTA)
						{
							if (TextureMapping)
							{
								// Triangle vertices in lightmap UV space, scaled by the lightmap resolution
								const FVector2f Vertex0 = Vertices[0].TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->SizeX, TextureMapping->SizeY);
								const FVector2f Vertex1 = Vertices[1].TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->SizeX, TextureMapping->SizeY);
								const FVector2f Vertex2 = Vertices[2].TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->SizeX, TextureMapping->SizeY);

								// Area in lightmap space, or the number of lightmap texels covered by this triangle
								const float LightmapTriangleArea = FMath::Abs(
									Vertex0.X * (Vertex1.Y - Vertex2.Y)
									+ Vertex1.X * (Vertex2.Y - Vertex0.Y)
									+ Vertex2.X * (Vertex0.Y - Vertex1.Y));

								const float TexelDensity = LightmapTriangleArea / TriangleArea;
								// Skip texture lightmapped triangles whose texel density is less than one texel per the area of a right triangle formed by SurfaceLightSampleSpacing.
								// If surface lighting is being calculated at a low resolution, it's unlikely that the volume near that surface needs to have detailed lighting.
								if (TexelDensity < 2.0f / FMath::Square(DynamicObjectSettings.SurfaceLightSampleSpacing))
								{
									continue;
								}
							}

							// Only rasterize upward facing triangles
							if (TriangleNormal.Z > 0.0f)
							{
								Rasterizer.DrawTriangle(
									Vertices[0],
									Vertices[1],
									Vertices[2],
									XYPositions[0],
									XYPositions[1],
									XYPositions[2],
									false
									);
							}
						}
					}
				}
			}
		}

		const float DetailVolumeSpacing = DynamicObjectSettings.DetailVolumeSampleSpacing;
		// Generate samples in a uniform 3d grid inside the detail volumes.  These will handle detail indirect lighting in areas that aren't directly above a surface.
		for (int32 VolumeIndex = 0; VolumeIndex < Scene.CharacterIndirectDetailVolumes.Num(); VolumeIndex++)
		{
			const FBox3f& DetailVolumeBounds = Scene.CharacterIndirectDetailVolumes[VolumeIndex];
			for (float SampleX = DetailVolumeBounds.Min.X; SampleX < DetailVolumeBounds.Max.X + DetailVolumeSpacing; SampleX += DetailVolumeSpacing)
			{
				for (float SampleY = DetailVolumeBounds.Min.Y; SampleY < DetailVolumeBounds.Max.Y + DetailVolumeSpacing; SampleY += DetailVolumeSpacing)
				{
					for (float SampleZ = DetailVolumeBounds.Min.Z; SampleZ < DetailVolumeBounds.Max.Z + DetailVolumeSpacing; SampleZ += DetailVolumeSpacing)
					{
						const FVector4f SamplePosition(SampleX, SampleY, SampleZ);
							
						// Only place a sample if there are no surface lighting samples nearby
						if (!FindNearbyVolumeSample(VolumeLightingOctree, SamplePosition, DynamicObjectSettings.SurfaceLightSampleSpacing))
						{
							const FLightRay Ray(
								SamplePosition,
								SamplePosition - FVector4f(0,0,VolumeBounds.BoxExtent.Z * 2.0f),
								nullptr,
								nullptr
								);
							FLightRayIntersection Intersection;
							// Trace a ray straight down to find which level's geometry we are over, 
							// Since this is how Dynamic Light Environments figure out which level to interpolate indirect lighting from.
							//@todo - could probably reuse the ray trace results for all samples of the same X and Y
							AggregateMesh->IntersectLightRay(Ray, true, false, false, MappingContext.RayCache, Intersection);

							// Place the sample in the intersected level, or the persistent level if there was no intersection
							const FGuid LevelGuid = Intersection.bIntersects ? Intersection.Mesh->LevelGuid : FGuid(0,0,0,0);
							TArray<FVolumeLightingSample>* VolumeLightingSampleArray = VolumeLightingSamples.Find(LevelGuid);
							if (!VolumeLightingSampleArray)
							{
								VolumeLightingSampleArray = &VolumeLightingSamples.Add(LevelGuid, TArray<FVolumeLightingSample>());
							}

							// Add a sample and set its radius such that its influence touches a diagonal sample on the 3d grid.
							VolumeLightingSampleArray->Add(FVolumeLightingSample(FVector4f(SamplePosition, DetailVolumeSpacing * FMath::Sqrt(3.0f))));
							VolumeLightingOctree.AddElement(FVolumeSampleProximityElement(VolumeLightingSampleArray->Num() - 1, *VolumeLightingSampleArray));
							if (DynamicObjectSettings.bVisualizeVolumeLightInterpolation)
							{
								VolumeLightingInterpolationOctree.AddElement(FVolumeSampleInterpolationElement(VolumeLightingSampleArray->Num() - 1, *VolumeLightingSampleArray));
							}
						}
					}
				}
			}
		}

		int32 SurfaceSamples = 0;
		for (TMap<FGuid,TArray<FVolumeLightingSample> >::TIterator It(VolumeLightingSamples); It; ++It)
		{
			SurfaceSamples += It.Value().Num();
		}
		Stats.NumDynamicObjectSurfaceSamples = SurfaceSamples;

		TArray<FVolumeLightingSample>* UniformVolumeSamples = VolumeLightingSamples.Find(FGuid(0,0,0,0));
		if (!UniformVolumeSamples)
		{
			UniformVolumeSamples = &VolumeLightingSamples.Add(FGuid(0,0,0,0), TArray<FVolumeLightingSample>());
		}

		const float VolumeSpacingCubed = DynamicObjectSettings.VolumeLightSampleSpacing * DynamicObjectSettings.VolumeLightSampleSpacing * DynamicObjectSettings.VolumeLightSampleSpacing;
		int32 RequestedVolumeSamples = FMath::TruncToInt(8.0f * VolumeBounds.BoxExtent.X * VolumeBounds.BoxExtent.Y * VolumeBounds.BoxExtent.Z / VolumeSpacingCubed);
		RequestedVolumeSamples = RequestedVolumeSamples == appTruncErrorCode ? INT_MAX : RequestedVolumeSamples;
		float EffectiveVolumeSpacing = DynamicObjectSettings.VolumeLightSampleSpacing;

		// Clamp the number of volume samples generated to DynamicObjectSettings.MaxVolumeSamples if necessary by resizing EffectiveVolumeSpacing
		if (RequestedVolumeSamples > DynamicObjectSettings.MaxVolumeSamples)
		{
			EffectiveVolumeSpacing = FMath::Pow(8.0f * float(VolumeBounds.BoxExtent.X * VolumeBounds.BoxExtent.Y * VolumeBounds.BoxExtent.Z / DynamicObjectSettings.MaxVolumeSamples), .3333333f);		// LWC_TODO: Precision loss - float cast
		}
			
		int32 NumUniformVolumeSamples = 0;
		// Generate samples in a uniform 3d grid inside the importance volume.  These will be used for low resolution lighting in unimportant areas.
		for (float SampleX = VolumeBounds.Origin.X - VolumeBounds.BoxExtent.X; SampleX < VolumeBounds.Origin.X + VolumeBounds.BoxExtent.X + EffectiveVolumeSpacing; SampleX += EffectiveVolumeSpacing)
		{
			for (float SampleY = VolumeBounds.Origin.Y - VolumeBounds.BoxExtent.Y; SampleY < VolumeBounds.Origin.Y + VolumeBounds.BoxExtent.Y + EffectiveVolumeSpacing; SampleY += EffectiveVolumeSpacing)
			{
				for (float SampleZ = VolumeBounds.Origin.Z - VolumeBounds.BoxExtent.Z; SampleZ < VolumeBounds.Origin.Z + VolumeBounds.BoxExtent.Z + EffectiveVolumeSpacing; SampleZ += EffectiveVolumeSpacing)
				{
					const FVector4f SamplePosition(SampleX, SampleY, SampleZ);
					// Only place inside the importance volume
					if (IsPointInImportanceVolume(SamplePosition, EffectiveVolumeSpacing)
						// Only place a sample if there are no surface lighting samples nearby
						&& !FindNearbyVolumeSample(VolumeLightingOctree, SamplePosition, DynamicObjectSettings.SurfaceLightSampleSpacing))
					{
						NumUniformVolumeSamples++;
						// Add a sample and set its radius such that its influence touches a diagonal sample on the 3d grid.
						UniformVolumeSamples->Add(FVolumeLightingSample(FVector4f(SamplePosition, EffectiveVolumeSpacing * FMath::Sqrt(3.0f))));
						VolumeLightingOctree.AddElement(FVolumeSampleProximityElement(UniformVolumeSamples->Num() - 1, *UniformVolumeSamples));
						if (DynamicObjectSettings.bVisualizeVolumeLightInterpolation)
						{
							VolumeLightingInterpolationOctree.AddElement(FVolumeSampleInterpolationElement(UniformVolumeSamples->Num() - 1, *UniformVolumeSamples));
						}
					}
				}
			}
		}

		Stats.NumDynamicObjectVolumeSamples = NumUniformVolumeSamples;
		const int32 VolumeSampleTaskSize = 256;

		for (TMap<FGuid,TArray<FVolumeLightingSample> >::TIterator It(VolumeLightingSamples); It; ++It)
		{
			const TArray<FVolumeLightingSample>& CurrentVolumeSamples = It.Value();

			for (int32 TaskIndex = 0; TaskIndex < FMath::DivideAndRoundUp(CurrentVolumeSamples.Num(), VolumeSampleTaskSize); TaskIndex++)
			{
				const int32 NumTaskSamples = FMath::Min(VolumeSampleTaskSize, CurrentVolumeSamples.Num() - TaskIndex * VolumeSampleTaskSize);
				VolumeSampleTasks.Push(FVolumeSamplesTaskDescription(It.Key(), TaskIndex * VolumeSampleTaskSize, NumTaskSamples));
			}
		}

		Stats.VolumeSamplePlacementThreadTime = FPlatformTime::Seconds() - VolumeSampleStartTime;

		// Make sure writes to VolumeSampleTasks are complete
		FPlatformMisc::MemoryBarrier();
		FPlatformAtomics::InterlockedExchange(&NumVolumeSampleTasksOutstanding, VolumeSampleTasks.Num());
	}
}

void FStaticLightingSystem::ProcessVolumeSamplesTask(const FVolumeSamplesTaskDescription& Task)
{
	const double VolumeSampleStartTime = FPlatformTime::Seconds();

	FLMRandomStream RandomStream(0);
	FStaticLightingMappingContext MappingContext(nullptr, *this);

	TArray<FVector4f> UniformHemisphereSamples;
	TArray<FVector2f> UniformHemisphereSampleUniforms;
	const int32 NumUpperVolumeSamples = ImportanceTracingSettings.NumHemisphereSamples * DynamicObjectSettings.NumHemisphereSamplesScale;
	// Volume samples don't do any importance sampling so they need more samples for the same amount of variance as surface samples
	const float NumThetaStepsFloat = FMath::Sqrt(NumUpperVolumeSamples / (float)PI);
	const int32 NumThetaSteps = FMath::TruncToInt(NumThetaStepsFloat);
	const int32 NumPhiSteps = FMath::TruncToInt(NumThetaStepsFloat * (float)PI);

	GenerateStratifiedUniformHemisphereSamples(NumThetaSteps, NumPhiSteps, RandomStream, UniformHemisphereSamples, UniformHemisphereSampleUniforms);

	FVector4f CombinedVector(0);

	for (int32 SampleIndex = 0; SampleIndex < UniformHemisphereSamples.Num(); SampleIndex++)
	{
		CombinedVector += UniformHemisphereSamples[SampleIndex];
	}

	float MaxUnoccludedLength = (CombinedVector / UniformHemisphereSamples.Num()).Size3();

	TArray<FVolumeLightingSample>& CurrentLevelSamples = VolumeLightingSamples.FindChecked(Task.LevelId);

	for (int32 SampleIndex = Task.StartIndex; SampleIndex < Task.StartIndex + Task.NumSamples; SampleIndex++)
	{
		FVolumeLightingSample& CurrentSample = CurrentLevelSamples[SampleIndex];

		if (GeneralSettings.NumIndirectLightingBounces > 0 
			// Calculating incident radiance for volume samples requires final gathering, since photons are only stored on surfaces.
			&& (!PhotonMappingSettings.bUsePhotonMapping || PhotonMappingSettings.bUseFinalGathering))
		{
			const bool bDebugSamples = false;
			float BackfacingHitsFraction = 0.0f;
			float Unused = 0.0f;

			// Sample radius stores the interpolation radius, but CalculateVolumeSampleIncidentRadiance will use this to push out final gather rays (ignore geometry inside the radius)
			// Save off and restore the sample radius later
			const float SampleRadius = CurrentSample.PositionAndRadius.W;
			CurrentSample.PositionAndRadius.W = 0.0f;

			TArray<FVector3f, TInlineAllocator<1>> VertexOffsets;
			VertexOffsets.Add(FVector3f(0, 0, 0));

			CalculateVolumeSampleIncidentRadiance(UniformHemisphereSamples, UniformHemisphereSampleUniforms, MaxUnoccludedLength, VertexOffsets, CurrentSample, BackfacingHitsFraction, Unused, RandomStream, MappingContext, bDebugSamples);
			
			CurrentSample.PositionAndRadius.W = SampleRadius;
		}
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
		if (Scene.DebugMapping && DynamicObjectSettings.bVisualizeVolumeLightSamples)
		{
			FSHVectorRGB3 IncidentRadiance;
			CurrentSample.ToSHVector(IncidentRadiance);
			VolumeLightingDebugOutput.VolumeLightingSamples.Add(FDebugVolumeLightingSample(CurrentSample.PositionAndRadius, IncidentRadiance.CalcIntegral() / FSHVector2::ConstantBasisIntegral));
		}
#endif
	}

	MappingContext.Stats.TotalVolumeSampleLightingThreadTime += FPlatformTime::Seconds() - VolumeSampleStartTime;
}

/** 
 * Interpolates lighting from the volume lighting samples to a vertex. 
 * This mirrors FPrecomputedLightVolume::InterpolateIncidentRadiance in UE5, used for visualizing interpolation from the lighting volume on surfaces.
 */
FGatheredLightSample FStaticLightingSystem::InterpolatePrecomputedVolumeIncidentRadiance(const FStaticLightingVertex& Vertex, float SampleRadius, FCoherentRayCache& RayCache, bool bDebugThisTexel) const
{
	FGatheredLightSample IncidentRadiance;
	float TotalWeight = 0.0f;

	if (bDebugThisTexel)
	{
		int32 TempBreak = 0;
	}

	// Iterate over the octree nodes containing the query point.
	for (FVolumeLightingInterpolationOctree::TConstElementBoxIterator<> OctreeIt(VolumeLightingInterpolationOctree, FBoxCenterAndExtent(Vertex.WorldPosition, FVector4f(0,0,0)));
		OctreeIt.HasPendingElements();
		OctreeIt.Advance())
	{
		const FVolumeSampleInterpolationElement& Element = OctreeIt.GetCurrentElement();
		const FVolumeLightingSample& VolumeSample = Element.VolumeSamples[Element.SampleIndex];

		const float DistanceSquared = (VolumeSample.GetPosition() - Vertex.WorldPosition).SizeSquared3();
		if (DistanceSquared < FMath::Square(VolumeSample.GetRadius()))
		{
			/*
			FLightRayIntersection Intersection;
			const FLightRay SampleRay
				(Vertex.WorldPosition + Vertex.WorldTangentZ * SceneConstants.VisibilityNormalOffsetSampleRadiusScale * SampleRadius, 
				VolumeSample.GetPosition(), 
				nullptr, 
				nullptr);
			AggregateMesh->IntersectLightRay(SampleRay, false, false, false, RayCache, Intersection);
			if (!Intersection.bIntersects)
			*/
			{
				const float SampleWeight = (1.0f - (Vertex.WorldPosition - VolumeSample.GetPosition()).Size3() / VolumeSample.GetRadius()) / VolumeSample.GetRadius();
				TotalWeight += SampleWeight;
			}
		}
	}

	if (TotalWeight > DELTA)
	{
	}

	return IncidentRadiance;
}

}
