// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMesh.h"
#include "Importer.h"

namespace Lightmass
{

/** 
 * Functions used for transforming a static mesh component based on a spline.  
 * This needs to be updated if the spline functionality changes!
 */

static float SmoothStep(float A, float B, float X)
{
	if (X < A)
	{
		return 0.0f;
	}
	else if (X >= B)
	{
		return 1.0f;
	}
	const float InterpFraction = (X - A) / (B - A);
	return InterpFraction * InterpFraction * (3.0f - 2.0f * InterpFraction);
}

static FVector3f SplineEvalPos(const FVector3f& StartPos, const FVector3f& StartTangent, const FVector3f& EndPos, const FVector3f& EndTangent, float A)
{
	const float A2 = A  * A;
	const float A3 = A2 * A;

	return (((2*A3)-(3*A2)+1) * StartPos) + ((A3-(2*A2)+A) * StartTangent) + ((A3-A2) * EndTangent) + (((-2*A3)+(3*A2)) * EndPos);
}

static FVector3f SplineEvalDir(const FVector3f& StartPos, const FVector3f& StartTangent, const FVector3f& EndPos, const FVector3f& EndTangent, float A)
{
	const FVector3f C = (6*StartPos) + (3*StartTangent) + (3*EndTangent) - (6*EndPos);
	const FVector3f D = (-6*StartPos) - (4*StartTangent) - (2*EndTangent) + (6*EndPos);
	const FVector3f E = StartTangent;

	const float A2 = A  * A;

	return ((C * A2) + (D * A) + E).GetSafeNormal();
}

/** Calculate full transform that defines frame along spline, given the Z of a vertex. */
/** Note:  This is mirrored from USplineMeshComponent::CalcSliceTransform() and LocalVertexShader.usf.  If you update one of these, please update them all! */
static FMatrix44f CalcSliceTransform(float ZPos, const FSplineMeshParams& SplineParams)
{
	// Find how far 'along' mesh we are
	const float Alpha = (ZPos - SplineParams.MeshMinZ) / SplineParams.MeshRangeZ;

	// Apply hermite interp to Alpha if desired
	const float HermiteAlpha = SplineParams.bSmoothInterpRollScale ? SmoothStep(0.0, 1.0, Alpha) : Alpha;

	// Then find the point and direction of the spline at this point along
	FVector3f SplinePos = SplineEvalPos( SplineParams.StartPos, SplineParams.StartTangent, SplineParams.EndPos, SplineParams.EndTangent, Alpha );
	const FVector3f SplineDir = SplineEvalDir( SplineParams.StartPos, SplineParams.StartTangent, SplineParams.EndPos, SplineParams.EndTangent, Alpha );

	// Find base frenet frame
	const FVector3f BaseXVec = (SplineParams.SplineUpDir ^ SplineDir).GetSafeNormal();
	const FVector3f BaseYVec = (SplineDir ^ BaseXVec).GetSafeNormal();

	// Offset the spline by the desired amount
	const FVector2f SliceOffset = FMath::Lerp<FVector2f>(SplineParams.StartOffset, SplineParams.EndOffset, HermiteAlpha);
	SplinePos += SliceOffset.X * BaseXVec;
	SplinePos += SliceOffset.Y * BaseYVec;

	// Apply roll to frame around spline	
	const float UseRoll = FMath::Lerp(SplineParams.StartRoll, SplineParams.EndRoll, HermiteAlpha);
	const float CosAng = FMath::Cos(UseRoll);
	const float SinAng = FMath::Sin(UseRoll);
	const FVector3f XVec = (CosAng * BaseXVec) - (SinAng * BaseYVec);
	const FVector3f YVec = (CosAng * BaseYVec) + (SinAng * BaseXVec);

	// Find scale at this point along spline
	const FVector2f UseScale = FMath::Lerp(SplineParams.StartScale, SplineParams.EndScale, HermiteAlpha);

	// Build overall transform
	FMatrix44f SliceTransform;
	switch (SplineParams.ForwardAxis)
	{
	case ESplineMeshAxis::X:
		SliceTransform = FMatrix44f(SplineDir, UseScale.X * XVec, UseScale.Y * YVec, SplinePos);
		break;
	case ESplineMeshAxis::Y:
		SliceTransform = FMatrix44f(UseScale.Y * YVec, SplineDir, UseScale.X * XVec, SplinePos);
		break;
	case ESplineMeshAxis::Z:
		SliceTransform = FMatrix44f(UseScale.X * XVec, UseScale.Y * YVec, SplineDir, SplinePos);
		break;
	default:
		check(0);
		break;
	}

	return SliceTransform;
}

/** Calculate rotation matrix that defines frame along spline, given the Z of a vertex. */
static FMatrix44f CalcSliceRot(float ZPos, const FSplineMeshParams& SplineParams)
{
	// Find how far 'along' mesh we are
	const float Alpha = (ZPos - SplineParams.MeshMinZ) / SplineParams.MeshRangeZ;

	// Apply hermite interp to Alpha if desired
	const float HermiteAlpha = SplineParams.bSmoothInterpRollScale ? SmoothStep(0.0, 1.0, Alpha) : Alpha;

	// Then find the point and direction of the spline at this point along
	const FVector3f SplineDir = SplineEvalDir( SplineParams.StartPos, SplineParams.StartTangent, SplineParams.EndPos, SplineParams.EndTangent, Alpha );

	// Find base frenet frame
	const FVector3f BaseXVec = (SplineParams.SplineUpDir ^ SplineDir).GetSafeNormal();
	const FVector3f BaseYVec = (SplineDir ^ BaseXVec).GetSafeNormal();

	// Apply roll to frame around spline
	const float UseRoll = FMath::Lerp(SplineParams.StartRoll, SplineParams.EndRoll, HermiteAlpha);
	const float CosAng = FMath::Cos(UseRoll);
	const float SinAng = FMath::Sin(UseRoll);
	const FVector3f XVec = (CosAng * BaseXVec) - (SinAng * BaseYVec);
	const FVector3f YVec = (CosAng * BaseYVec) + (SinAng * BaseXVec);

	// Build rotation transform
	FMatrix44f SliceTransform;
	switch (SplineParams.ForwardAxis)
	{
	case ESplineMeshAxis::X:
		SliceTransform = FMatrix44f(SplineDir, XVec, YVec, FVector3f(0,0,0));
		break;
	case ESplineMeshAxis::Y:
		SliceTransform = FMatrix44f(YVec, SplineDir, XVec, FVector3f(0,0,0));
		break;
	case ESplineMeshAxis::Z:
		SliceTransform = FMatrix44f(XVec, YVec, SplineDir, FVector3f(0,0,0));
		break;
	default:
		check(0);
		break;
	}

	return SliceTransform;
}

/**
 * Creates a static lighting vertex to represent the given static mesh vertex.
 * @param VertexBuffer - The static mesh's vertex buffer.
 * @param VertexIndex - The index of the static mesh vertex to access.
 * @param OutVertex - Upon return, contains a static lighting vertex representing the specified static mesh vertex.
 */
static void GetStaticLightingVertex(
	const FStaticMeshVertex& InVertex,
	const FMatrix44f& LocalToWorld,
	const FMatrix44f& LocalToWorldInverseTranspose,
	bool bIsSplineMesh,
	const FSplineMeshParams& SplineParams,
	FStaticLightingVertex& OutVertex
	)
{
	if (bIsSplineMesh)
	{
		// Make transform for this point along spline
		const FMatrix44f SliceTransform = CalcSliceTransform(InVertex.Position[SplineParams.ForwardAxis], SplineParams);

		// Remove Z (transform will move us along spline)
		FVector4f SlicePos = InVertex.Position;
		SlicePos[SplineParams.ForwardAxis] = 0;

		// Transform into mesh space
		const FVector4f LocalPos = SliceTransform.TransformPosition(SlicePos);

		// Transform from mesh to world space
		OutVertex.WorldPosition = LocalToWorld.TransformPosition(LocalPos);

		const FMatrix44f SliceRot = CalcSliceRot(InVertex.Position[SplineParams.ForwardAxis], SplineParams);

		const FVector4f LocalSpaceTangentX = SliceRot.TransformVector(InVertex.TangentX);
		const FVector4f LocalSpaceTangentY = SliceRot.TransformVector(InVertex.TangentY);
		const FVector4f LocalSpaceTangentZ = SliceRot.TransformVector(InVertex.TangentZ);

		OutVertex.WorldTangentX = LocalToWorld.TransformVector(LocalSpaceTangentX).GetSafeNormal();
		OutVertex.WorldTangentY = LocalToWorld.TransformVector(LocalSpaceTangentY).GetSafeNormal();
		OutVertex.WorldTangentZ = LocalToWorldInverseTranspose.TransformVector(LocalSpaceTangentZ).GetSafeNormal();
	}
	else
	{
		OutVertex.WorldPosition = LocalToWorld.TransformPosition(InVertex.Position);
		OutVertex.WorldTangentX = LocalToWorld.TransformVector(InVertex.TangentX).GetSafeNormal();
		OutVertex.WorldTangentY = LocalToWorld.TransformVector(InVertex.TangentY).GetSafeNormal();
		OutVertex.WorldTangentZ = LocalToWorldInverseTranspose.TransformVector(InVertex.TangentZ).GetSafeNormal();
	}

	// WorldTangentZ can end up a 0 vector if it was small to begin with and LocalToWorld contains large scale factors.
	if (!OutVertex.WorldTangentZ.IsUnit3())
	{
		OutVertex.WorldTangentZ = (OutVertex.WorldTangentX ^ OutVertex.WorldTangentY).GetSafeNormal();
	}

	for(uint32 LightmapTextureCoordinateIndex = 0; LightmapTextureCoordinateIndex < UE_ARRAY_COUNT(InVertex.UVs); LightmapTextureCoordinateIndex++)
	{
		OutVertex.TextureCoordinates[LightmapTextureCoordinateIndex] = InVertex.UVs[LightmapTextureCoordinateIndex];
	}
}

// FStaticLightingMesh interface.

void FStaticMeshStaticLightingMesh::GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const
{
 	const FStaticMeshLOD& LODRenderData = StaticMesh->GetLOD(GetMeshLODIndex());
 
 	// Lookup the triangle's vertex indices.
 	const uint32 I0 = LODRenderData.GetIndex(TriangleIndex * 3 + 0);
 	const uint32 I1 = LODRenderData.GetIndex(TriangleIndex * 3 + (bReverseWinding ? 2 : 1));
 	const uint32 I2 = LODRenderData.GetIndex(TriangleIndex * 3 + (bReverseWinding ? 1 : 2));
 
 	// Translate the triangle's static mesh vertices to static lighting vertices.
 	GetStaticLightingVertex(LODRenderData.GetVertex(I0), LocalToWorld, LocalToWorldInverseTranspose, bIsSplineMesh, SplineParameters, OutV0);
 	GetStaticLightingVertex(LODRenderData.GetVertex(I1), LocalToWorld, LocalToWorldInverseTranspose, bIsSplineMesh, SplineParameters, OutV1);
 	GetStaticLightingVertex(LODRenderData.GetVertex(I2), LocalToWorld, LocalToWorldInverseTranspose, bIsSplineMesh, SplineParameters, OutV2);

	const FStaticMeshLOD& MeshLOD = StaticMesh->GetLOD(GetMeshLODIndex());
	ElementIndex = INDEX_NONE;
	for (uint32 MeshElementIndex = 0; MeshElementIndex < MeshLOD.NumElements; MeshElementIndex++)
	{
		const FStaticMeshElement& CurrentElement = MeshLOD.GetElement(MeshElementIndex);
		if ((uint32)TriangleIndex >= CurrentElement.FirstIndex / 3 && (uint32)TriangleIndex < CurrentElement.FirstIndex / 3 + CurrentElement.NumTriangles)
		{
			ElementIndex = MeshElementIndex;
			break;
		}
	}
	check(ElementIndex >= 0);
}

void FStaticMeshStaticLightingMesh::GetNonTransformedTriangle(int32 TriangleIndex, FStaticLightingVertex& OutV0, FStaticLightingVertex& OutV1, FStaticLightingVertex& OutV2, int32& ElementIndex) const
{
	check(!bIsSplineMesh);

	const FStaticMeshLOD& LODRenderData = StaticMesh->GetLOD(GetMeshLODIndex());

	// Lookup the triangle's vertex indices.
	const uint32 I0 = LODRenderData.GetIndex(TriangleIndex * 3 + 0);
	const uint32 I1 = LODRenderData.GetIndex(TriangleIndex * 3 + 1);
	const uint32 I2 = LODRenderData.GetIndex(TriangleIndex * 3 + 2);

	// Translate the triangle's static mesh vertices to static lighting vertices.
	GetStaticLightingVertex(LODRenderData.GetVertex(I0), FMatrix44f::Identity, FMatrix44f::Identity, false, SplineParameters, OutV0);
	GetStaticLightingVertex(LODRenderData.GetVertex(I1), FMatrix44f::Identity, FMatrix44f::Identity, false, SplineParameters, OutV1);
	GetStaticLightingVertex(LODRenderData.GetVertex(I2), FMatrix44f::Identity, FMatrix44f::Identity, false, SplineParameters, OutV2);

	const FStaticMeshLOD& MeshLOD = StaticMesh->GetLOD(GetMeshLODIndex());
	ElementIndex = INDEX_NONE;
	for (uint32 MeshElementIndex = 0; MeshElementIndex < MeshLOD.NumElements; MeshElementIndex++)
	{
		const FStaticMeshElement& CurrentElement = MeshLOD.GetElement(MeshElementIndex);
		if ((uint32)TriangleIndex >= CurrentElement.FirstIndex / 3 && (uint32)TriangleIndex < CurrentElement.FirstIndex / 3 + CurrentElement.NumTriangles)
		{
			ElementIndex = MeshElementIndex;
			break;
		}
	}
	check(ElementIndex >= 0);
}

void FStaticMeshStaticLightingMesh::GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const
{
 	const FStaticMeshLOD& LODRenderData = StaticMesh->GetLOD(GetMeshLODIndex());
 
 	// Lookup the triangle's vertex indices.
 	OutI0 = LODRenderData.GetIndex(TriangleIndex * 3 + 0);
 	OutI1 = LODRenderData.GetIndex(TriangleIndex * 3 + (bReverseWinding ? 2 : 1));
 	OutI2 = LODRenderData.GetIndex(TriangleIndex * 3 + (bReverseWinding ? 1 : 2));
}

void FStaticMeshStaticLightingMesh::GetNonTransformedTriangleIndices(int32 TriangleIndex, int32& OutI0, int32& OutI1, int32& OutI2) const
{
	const FStaticMeshLOD& LODRenderData = StaticMesh->GetLOD(GetMeshLODIndex());

	// Lookup the triangle's vertex indices.
	OutI0 = LODRenderData.GetIndex(TriangleIndex * 3 + 0);
	OutI1 = LODRenderData.GetIndex(TriangleIndex * 3 + 1);
	OutI2 = LODRenderData.GetIndex(TriangleIndex * 3 + 2);
}

bool FStaticMeshStaticLightingMesh::IsElementCastingShadow(int32 ElementIndex) const
{
	const FStaticMeshLOD& LODRenderData = StaticMesh->GetLOD(GetMeshLODIndex());
	const FStaticMeshElement& Element = LODRenderData.GetElement( ElementIndex );
	return Element.bEnableShadowCasting;
}

void FStaticMeshStaticLightingMesh::Import( FLightmassImporter& Importer )
{
	// import super class
	FStaticLightingMesh::Import( Importer );

	// import the shared data
//	Importer.ImportData( (FStaticMeshStaticLightingMeshData*) this );
	FStaticMeshStaticLightingMeshData TempSMSLMD;
	Importer.ImportData(&TempSMSLMD);
	EncodedLODIndices = TempSMSLMD.EncodedLODIndices;
	EncodedHLODRange = TempSMSLMD.EncodedHLODRange;
	LocalToWorld = TempSMSLMD.LocalToWorld;
	bReverseWinding = TempSMSLMD.bReverseWinding;
	bShouldSelfShadow = TempSMSLMD.bShouldSelfShadow;
	StaticMeshGuid = TempSMSLMD.StaticMeshGuid;
	bIsSplineMesh = TempSMSLMD.bIsSplineMesh;
	SplineParameters = TempSMSLMD.SplineParameters;

	// calculate the inverse transpose
	WorldToLocal = LocalToWorld.InverseFast();
	LocalToWorldInverseTranspose = LocalToWorld.InverseFast().GetTransposed();

	// we have the guid for the mesh, now hook it up to the actual static mesh
	StaticMesh = Importer.ConditionalImportObject<FStaticMesh>(StaticMeshGuid, LM_STATICMESH_VERSION, LM_STATICMESH_EXTENSION, LM_STATICMESH_CHANNEL_FLAGS, Importer.GetStaticMeshes());
	checkf(StaticMesh, TEXT("Failed to import static mesh with GUID %s"), *StaticMeshGuid.ToString());
	check(GetMeshLODIndex() >= 0 && GetMeshLODIndex() < (int32)StaticMesh->NumLODs);
	checkf(StaticMesh->GetLOD(GetMeshLODIndex()).NumElements == MaterialElements.Num(), TEXT("Static mesh element count did not match mesh instance element count!"));
}

void FStaticMeshStaticLightingTextureMapping::Import( FLightmassImporter& Importer )
{
	// import the super class
	FStaticLightingTextureMapping::Import(Importer);
	check(Mesh);
}

} //namespace Lightmass
