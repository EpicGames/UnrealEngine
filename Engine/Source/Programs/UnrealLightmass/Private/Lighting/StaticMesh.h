// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LightingMesh.h"
#include "Mesh.h"
#include "Mappings.h"

namespace Lightmass
{
	/** Represents the triangles of one LOD of a static mesh primitive to the static lighting system. */
	class FStaticMeshStaticLightingMesh : public FStaticLightingMesh, public FStaticMeshStaticLightingMeshData
	{
	public:

		/** The static mesh this mesh represents. */
		FStaticMesh* StaticMesh;

		FStaticLightingMapping* Mapping;

		/** World to local transform, for box intersection w/ instances. */
		FMatrix44f WorldToLocal;

		virtual FStaticMeshStaticLightingMesh* GetInstanceableStaticMesh() override { return !bIsSplineMesh ? this : nullptr; }

		virtual const FStaticMeshStaticLightingMesh* GetInstanceableStaticMesh() const override { return !bIsSplineMesh ? this : nullptr; }

		// FStaticLightingMesh interface.
		/**
		 *	Returns the Guid for the object associated with this lighting mesh.
		 *	Ie, for a StaticMeshStaticLightingMesh, it would return the Guid of the source static mesh.
		 *	The GetObjectType function should also be used to determine the TypeId of the source object.
		 */
		virtual FGuid GetObjectGuid() const
		{
			if (StaticMesh)
			{
				return StaticMesh->Guid;
			}
			return FGuid(0,0,0,0);
		}

		/**
		 *	Returns the SourceObject type id.
		 */
		virtual ESourceObjectType GetObjectType() const
		{
			return SOURCEOBJECTTYPE_StaticMesh;
		}

		virtual void GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const;
		virtual void GetNonTransformedTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2,int32& ElementIndex) const;
		virtual void GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const;
		virtual void GetNonTransformedTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const;

		virtual bool IsElementCastingShadow(int32 ElementIndex) const;

		virtual uint32 GetLODIndices() const { return EncodedLODIndices; }
		virtual uint32 GetHLODRange() const { return EncodedHLODRange; }

		/**
		 * @return the portion of the LOD index variable that is actually the mesh LOD level
		 * It strips off the MassiveLOD portion, which is in the high bytes. The MassiveLOD
		 * portion is needed for disallowing shadow casting between parents/children 
		 */
		virtual uint32 GetMeshLODIndex() const { return EncodedLODIndices & 0xFFFF; }
		virtual uint32 GetMeshHLODIndex() const { return (EncodedLODIndices & 0xFFFF0000) >> 16; }

		virtual uint32 GetMeshHLODRangeStart() const { return EncodedHLODRange & 0xFFFF; }
		virtual uint32 GetMeshHLODRangeEnd() const { return (EncodedHLODRange & 0xFFFF0000) >> 16; }

		virtual void Import( class FLightmassImporter& Importer );

		/** The inverse transpose of the primitive's local to world transform. */
		FMatrix44f LocalToWorldInverseTranspose;
	};

	/** Represents a static mesh primitive with texture mapped static lighting. */
	class FStaticMeshStaticLightingTextureMapping : public FStaticLightingTextureMapping
	{
	public:
	
		virtual void Import( class FLightmassImporter& Importer );

	private:

		/** The LOD this mapping represents. */
		int32 LODIndex;
	};

} //namespace Lightmass
