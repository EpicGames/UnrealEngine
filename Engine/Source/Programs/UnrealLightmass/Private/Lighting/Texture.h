// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Lightmass
{
	/** Texture formats used by FTexture2D */
	enum ETexture2DFormats
	{
		TF_UNKNOWN,
		TF_ARGB8,
		TF_ARGB16F
	};

	/** A 2D texture */
	class FTexture2D
	{
	protected:
		/** Texture dimensions */
		int32 SizeX;
		int32 SizeY;
		/** Format of texture which indicates how to interpret Data */
		ETexture2DFormats Format;
		/** The size of each element in the data array */
		int32 ElementSize;
		/** Mip 0 texture data */
		uint8* Data;

	public:
		FTexture2D() :
			  SizeX(0)
			, SizeY(0)
			, Format(TF_UNKNOWN)
			, ElementSize(0)
			, Data(NULL)
		{
		}

		FTexture2D(ETexture2DFormats InFormat, int32 InSizeX, int32 InSizeY) :
			  SizeX(InSizeX)
			, SizeY(InSizeY)
			, Format(InFormat)
			, ElementSize(0)
			, Data(NULL)
		{
			Init(InFormat, InSizeX, InSizeY);
		}

		virtual ~FTexture2D()
		{
			FMemory::Free(Data);
		}

		/** Accessors */
		int32 GetSizeX() const { return SizeX; }
		int32 GetSizeY() const { return SizeY; }
		uint8* GetData() { return Data; }

		void Init(ETexture2DFormats InFormat, int32 InSizeX, int32 InSizeY)
		{
			check(InSizeX > 0 && InSizeY > 0);
			SizeX = InSizeX;
			SizeY = InSizeY;
			Format = InFormat;

			// Only supporting these formats
			check(InFormat == TF_ARGB8 || InFormat == TF_ARGB16F);

			switch (InFormat)
			{
			case TF_ARGB8:		ElementSize = sizeof(FColor);			break;
			case TF_ARGB16F:	ElementSize = sizeof(FFloat16Color);	break;
			}

			Data = (uint8*)(FMemory::Malloc(ElementSize * SizeX * SizeY));
			FMemory::Memzero(Data, ElementSize * SizeX * SizeY);
		}

		inline uint8* SampleRawPtr(const FVector2f& UV) const
		{
			// Wrapped addressing (uses FMath::FloorToInt and not appFractional, as appFractional causes the
			// following mapping:
			// .4 -> .4
			// -1.4 -> .4
			// (.4 - 1 = -.6) -> .6
			//
			// we need:
			// .4 -> .4
			// -1.4 -> .6
			// (.4 - 1 = -.6) -> .4
			//
			// because when you subtract 1 from a UV it needs to have the exact same fractional part
 			const int32 X = FMath::Clamp(FMath::TruncToInt((UV.X - FMath::FloorToInt(UV.X)) * SizeX), 0, SizeX - 1);
 			const int32 Y = FMath::Clamp(FMath::TruncToInt((UV.Y - FMath::FloorToInt(UV.Y)) * SizeY), 0, SizeY - 1);
			// Byte index into Data
			const int32 DataIndex = Y * SizeX * ElementSize + X * ElementSize;
			return &Data[DataIndex];
		}

		inline FLinearColor Sample(const FVector2f& UV) const
		{
			uint8* RawPtr = SampleRawPtr( UV );

			// Only supporting these formats
			checkSlow(Format == TF_ARGB8 || Format == TF_ARGB16F);

			if (Format == TF_ARGB16F)
			{
				// Lookup and convert to FP32, no filtering
				return FLinearColor(*(FFloat16Color*)RawPtr);
			}
			// Lookup and convert linear space and FP32, no filtering
			return FLinearColor(*(FColor*)RawPtr);
		}

		inline FVector4f SampleNormal(const FVector2f& UV) const
		{
			uint8* RawPtr = SampleRawPtr( UV );

			// Only supporting these formats
			checkSlow(Format == TF_ARGB16F);
			FFloat16Color* Float16Color = (FFloat16Color*)RawPtr;

			FVector4f Normal( Float16Color->R.GetFloat(), Float16Color->G.GetFloat(), Float16Color->B.GetFloat(), 0.0f );
			return Normal;
		}
	};

} //namespace Lightmass
