// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Raster.h"
#include "LightingSystem.h"
#include "LightmassSwarm.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "TextureMappingSetup.h"

namespace Lightmass
{

void FStaticLightingRasterPolicy::ProcessPixel(int32 X,int32 Y,const InterpolantType& Interpolant,bool BackFacing)
{
	FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
	bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisMapping
		&& X == Scene.DebugInput.LocalX
		&& Y == Scene.DebugInput.LocalY)
	{
		bDebugThisTexel = true;
	}
#endif

	if (bUseMaxWeight)
	{
		if (SampleWeight > TexelToVertex.MaxSampleWeight)
		{
			// Use the sample with the largest weight.  
			// This has the disadvantage compared averaging based on weight that it won't be well centered for texels on a UV seam,
			// But it has the advantage that the final position is guaranteed to be valid (ie actually on a triangle),
			// Even for split texels which are mapped to triangles in different parts of the mesh.
			TexelToVertex.MaxSampleWeight = SampleWeight;
			TexelToVertex.WorldPosition = Interpolant.Vertex.WorldPosition;
			TexelToVertex.ElementIndex = Interpolant.ElementIndex;

			for( int32 CurCoordIndex = 0; CurCoordIndex < MAX_TEXCOORDS; ++CurCoordIndex )
			{
				TexelToVertex.TextureCoordinates[ CurCoordIndex ] = Interpolant.Vertex.TextureCoordinates[ CurCoordIndex ];
			}
		}
		
		// Weighted average of normal, improves the case where the position chosen by the max weight has a different normal than the rest of the texel
		// Eg, small extrusions from an otherwise flat surface, and the texel center lies on the perpendicular extrusion
		//@todo - only average normals within the texel radius to improve the split texel case?
		TexelToVertex.WorldTangentX += Interpolant.Vertex.WorldTangentX * SampleWeight;
		TexelToVertex.WorldTangentY += Interpolant.Vertex.WorldTangentY * SampleWeight;
		TexelToVertex.WorldTangentZ += Interpolant.Vertex.WorldTangentZ * SampleWeight;
		checkSlow(!TriangleNormal.ContainsNaN());
		TexelToVertex.TriangleNormal += TriangleNormal * SampleWeight;
		TexelToVertex.TotalSampleWeight += SampleWeight;
	}
	else if (!bUseMaxWeight)
	{
		// Update the sample weight, and compute the scales used to update the sample's averages.
		const float NewTotalSampleWeight = TexelToVertex.TotalSampleWeight + SampleWeight;		
		const float OldSampleWeight = TexelToVertex.TotalSampleWeight / NewTotalSampleWeight;	
		const float NewSampleWeight = SampleWeight / NewTotalSampleWeight;						
		TexelToVertex.TotalSampleWeight = NewTotalSampleWeight;	

		// Add this sample to the mapping.
		TexelToVertex.WorldPosition = TexelToVertex.WorldPosition * OldSampleWeight + Interpolant.Vertex.WorldPosition * NewSampleWeight;
		TexelToVertex.WorldTangentX = FVector4f(TexelToVertex.WorldTangentX) * OldSampleWeight + Interpolant.Vertex.WorldTangentX * NewSampleWeight;
		TexelToVertex.WorldTangentY = FVector4f(TexelToVertex.WorldTangentY) * OldSampleWeight + Interpolant.Vertex.WorldTangentY * NewSampleWeight;
		TexelToVertex.WorldTangentZ = FVector4f(TexelToVertex.WorldTangentZ) * OldSampleWeight + Interpolant.Vertex.WorldTangentZ * NewSampleWeight;
		TexelToVertex.TriangleNormal = TriangleNormal;
		TexelToVertex.ElementIndex = Interpolant.ElementIndex;
		
		for( int32 CurCoordIndex = 0; CurCoordIndex < MAX_TEXCOORDS; ++CurCoordIndex )
		{
			TexelToVertex.TextureCoordinates[ CurCoordIndex ] = TexelToVertex.TextureCoordinates[ CurCoordIndex ] * OldSampleWeight + Interpolant.Vertex.TextureCoordinates[ CurCoordIndex ] * NewSampleWeight;
		}
	}
}

void FFullStaticLightingVertex::ApplyVertexModifications(int32 ElementIndex, bool bUseNormalMapsForLighting, const FStaticLightingMesh* Mesh)
{
	if (bUseNormalMapsForLighting && Mesh->HasImportedNormal(ElementIndex))
	{
		const FVector4f TangentNormal = Mesh->EvaluateNormal(TextureCoordinates[0], ElementIndex);

		const FVector4f WorldTangentRow0(WorldTangentX.X, WorldTangentY.X, WorldTangentZ.X);
		const FVector4f WorldTangentRow1(WorldTangentX.Y, WorldTangentY.Y, WorldTangentZ.Y);
		const FVector4f WorldTangentRow2(WorldTangentX.Z, WorldTangentY.Z, WorldTangentZ.Z);
		const FVector4f WorldVector(
			Dot3(WorldTangentRow0, TangentNormal),
			Dot3(WorldTangentRow1, TangentNormal),
			Dot3(WorldTangentRow2, TangentNormal)
			);

		WorldTangentZ = WorldVector;
	}

	// Normalize the tangent basis and ensure it is orthonormal
	WorldTangentZ = WorldTangentZ.GetUnsafeNormal3();

	const bool bUseVertexNormalForHemisphereGather = Mesh->UseVertexNormalForHemisphereGather(ElementIndex);
	TriangleNormal = bUseVertexNormalForHemisphereGather ? WorldTangentZ : TriangleNormal.GetUnsafeNormal3();
	checkSlow(!TriangleNormal.ContainsNaN());

	const FVector4f OriginalTangentX = WorldTangentX;
	const FVector4f OriginalTangentY = WorldTangentY;

	WorldTangentY = (WorldTangentZ ^ WorldTangentX).GetUnsafeNormal3();
	// Maintain handedness
	if (Dot3(WorldTangentY, OriginalTangentY) < 0)
	{
		WorldTangentY *= -1.0f;
	}
	WorldTangentX = WorldTangentY ^ WorldTangentZ;
	if (Dot3(WorldTangentX, OriginalTangentX) < 0)
	{
		WorldTangentX *= -1.0f;
	}
}

void FStaticLightingTextureMapping::Initialize(FStaticLightingSystem& System)
{
	SurfaceCacheSizeX = FMath::Max(FMath::TruncToInt((float)CachedSizeX / (float)System.GeneralSettings.MappingSurfaceCacheDownsampleFactor), 6);
	SurfaceCacheSizeY = FMath::Max(FMath::TruncToInt((float)CachedSizeY / (float)System.GeneralSettings.MappingSurfaceCacheDownsampleFactor), 6);
}

/** Caches irradiance photons on a single texture mapping. */
void FStaticLightingSystem::CacheIrradiancePhotonsTextureMapping(FStaticLightingTextureMapping* TextureMapping)
{
	checkSlow(TextureMapping);
	FStaticLightingMappingContext MappingContext(TextureMapping->Mesh,*this,&StartupDebugOutput);
	LIGHTINGSTAT(FScopedRDTSCTimer CachingTime(MappingContext.Stats.IrradiancePhotonCachingThreadTime));
	const FBoxSphereBounds3f ImportanceBounds = Scene.GetImportanceBounds();

	FTexelToVertexMap TexelToVertexMap(TextureMapping->SurfaceCacheSizeX, TextureMapping->SurfaceCacheSizeY);

	bool bDebugThisMapping = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	bDebugThisMapping = TextureMapping == Scene.DebugMapping;
	int32 IrradiancePhotonCacheDebugX = -1;
	int32 IrradiancePhotonCacheDebugY = -1;
	if (bDebugThisMapping)
	{
		IrradiancePhotonCacheDebugX = FMath::TruncToInt(Scene.DebugInput.LocalX / (float)TextureMapping->CachedSizeX * TextureMapping->SurfaceCacheSizeX);
		IrradiancePhotonCacheDebugY = FMath::TruncToInt(Scene.DebugInput.LocalY / (float)TextureMapping->CachedSizeY * TextureMapping->SurfaceCacheSizeY);
	}
#endif

	RasterizeToSurfaceCacheTextureMapping(TextureMapping, bDebugThisMapping, TexelToVertexMap);

	// Allocate space for the cached irradiance photons
	TextureMapping->CachedIrradiancePhotons.Empty(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);
	TextureMapping->CachedIrradiancePhotons.AddZeroed(TextureMapping->SurfaceCacheSizeX * TextureMapping->SurfaceCacheSizeY);

	TArray<FIrradiancePhoton*> TempIrradiancePhotons;
	for (int32 Y = 0; Y < TextureMapping->SurfaceCacheSizeY; Y++)
	{
		for (int32 X = 0; X < TextureMapping->SurfaceCacheSizeX; X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == IrradiancePhotonCacheDebugY
				&& X == IrradiancePhotonCacheDebugX)
			{
				bDebugThisTexel = true;
			}
#endif
			const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
			if (TexelToVertex.TotalSampleWeight > 0.0f)
			{
				MappingContext.Stats.NumCachedIrradianceSamples++;
				FFullStaticLightingVertex CurrentVertex = TexelToVertex.GetFullVertex();

				CurrentVertex.ApplyVertexModifications(TexelToVertex.ElementIndex, MaterialSettings.bUseNormalMapsForLighting, TextureMapping->Mesh);

				FIrradiancePhoton* NearestPhoton = NULL;

				// Only search the irradiance photon map if the surface cache position is inside the importance volume,
				// Since irradiance photons are only deposited inside the importance volume.
				if (ImportanceBounds.GetBox().IsInside(CurrentVertex.WorldPosition))
				{
					// Find the nearest irradiance photon and store it on the surface of the mapping
					// Only find visible irradiance photons to prevent light leaking through thin surfaces
					//Note: It's still possible for light to leak if a single texel spans two disjoint lighting areas, for example two planes coming together at a 90 degree angle.
					NearestPhoton = FindNearestIrradiancePhoton(CurrentVertex, MappingContext, TempIrradiancePhotons, true, bDebugThisTexel);
						
					if (NearestPhoton)
					{
						if (!NearestPhoton->IsUsed())
						{
							// An irradiance photon was found that hadn't been marked used yet
							MappingContext.Stats.NumFoundIrradiancePhotons++;
							NearestPhoton->SetUsed();
						}

						TextureMapping->CachedIrradiancePhotons[Y * TextureMapping->SurfaceCacheSizeX + X] = NearestPhoton;
					}
				}
			}
		}
	}
}

/** Cache irradiance photons on surfaces. */
void FStaticLightingSystem::FinalizeSurfaceCache()
{
	for(int32 ThreadIndex = 1; ThreadIndex < NumStaticLightingThreads; ThreadIndex++)
	{
		FMappingProcessingThreadRunnable* ThreadRunnable = new FMappingProcessingThreadRunnable(this, ThreadIndex, StaticLightingTask_FinalizeSurfaceCache);
		FinalizeSurfaceCacheThreads.Add(ThreadRunnable);
		const FString ThreadName = FString::Printf(TEXT("FinalizeSurfaceCacheThread%u"), ThreadIndex);
		ThreadRunnable->Thread = FRunnableThread::Create(ThreadRunnable, *ThreadName);
	}

	// Start the static lighting thread loop on the main thread, too.
	// Once it returns, all static lighting mappings have begun processing.
	FinalizeSurfaceCacheThreadLoop(0, true);

	// Stop the static lighting threads.
	for(int32 ThreadIndex = 0;ThreadIndex < FinalizeSurfaceCacheThreads.Num();ThreadIndex++)
	{
		// Wait for the thread to exit.
		FinalizeSurfaceCacheThreads[ThreadIndex].Thread->WaitForCompletion();
		// Check that it didn't terminate with an error.
		FinalizeSurfaceCacheThreads[ThreadIndex].CheckHealth();

		// Destroy the thread.
		delete FinalizeSurfaceCacheThreads[ThreadIndex].Thread;
	}
	FinalizeSurfaceCacheThreads.Empty();
}

void FStaticLightingSystem::FinalizeSurfaceCacheThreadLoop(int32 ThreadIndex, bool bIsMainThread)
{
	bool bIsDone = false;
	while (!bIsDone)
	{
		// Atomically read and increment the next mapping index to process.
		const int32 MappingIndex = NextMappingToFinalizeSurfaceCache.Increment() - 1;

		if (MappingIndex < AllMappings.Num())
		{
			// If this is the main thread, update progress and apply completed static lighting.
			if (bIsMainThread)
			{
				// Check the health of all static lighting threads.
				for (int32 ThreadIndexIter = 0; ThreadIndexIter < FinalizeSurfaceCacheThreads.Num(); ThreadIndexIter++)
				{
					FinalizeSurfaceCacheThreads[ThreadIndexIter].CheckHealth();
				}
			}

			FStaticLightingTextureMapping* TextureMapping = AllMappings[MappingIndex]->GetTextureMapping();

			if (TextureMapping)
			{
				FinalizeSurfaceCacheTextureMapping(TextureMapping);
			}
		}
		else
		{
			// Processing has begun for all mappings.
			bIsDone = true;
		}
	}
}

void FStaticLightingSystem::RasterizeToSurfaceCacheTextureMapping(FStaticLightingTextureMapping* TextureMapping, bool bDebugThisMapping, FTexelToVertexMap& TexelToVertexMap)
{
	FStaticLightingMappingContext MappingContext(TextureMapping->Mesh, *this, &StartupDebugOutput);

	FTexelToCornersMap TexelToCornersMap(TextureMapping->SurfaceCacheSizeX, TextureMapping->SurfaceCacheSizeY);
	CalculateTexelCorners(TextureMapping->Mesh, TexelToCornersMap, TextureMapping->LightmapTextureCoordinateIndex, bDebugThisMapping);

	// Using conservative rasterization, which uses super sampling to try to detect all texels that should be mapped.
	for(int32 TriangleIndex = 0;TriangleIndex < TextureMapping->Mesh->NumTriangles;TriangleIndex++)
	{
		// Query the mesh for the triangle's vertices.
		FStaticLightingInterpolant V0;
		FStaticLightingInterpolant V1;
		FStaticLightingInterpolant V2;
		int32 Element;
		TextureMapping->Mesh->GetTriangle(TriangleIndex,V0.Vertex,V1.Vertex,V2.Vertex,Element);
		V0.ElementIndex = V1.ElementIndex = V2.ElementIndex = Element;

		const FVector4f TriangleNormal = ((V2.Vertex.WorldPosition - V0.Vertex.WorldPosition) ^ (V1.Vertex.WorldPosition - V0.Vertex.WorldPosition)).GetSafeNormal();

		// Don't rasterize degenerates 
		if (!TriangleNormal.IsNearlyZero3())
		{
			const FVector2f UV0 = V0.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->SurfaceCacheSizeX,TextureMapping->SurfaceCacheSizeY);
			const FVector2f UV1 = V1.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->SurfaceCacheSizeX,TextureMapping->SurfaceCacheSizeY);
			const FVector2f UV2 = V2.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->SurfaceCacheSizeX,TextureMapping->SurfaceCacheSizeY);

			// Odd number of samples so that the center of the pyramid is on one of the samples
			const uint32 NumSamplesX = 5;
			const uint32 NumSamplesY = 5;

			// Rasterize multiple sub-texel samples and linearly combine the results
			// Don't rasterize the first or last row and column as the weight will be 0
			for(int32 Y = 1; Y < NumSamplesY - 1; Y++)
			{
				const float SampleYOffset = -Y / (float)(NumSamplesY - 1);
				for(int32 X = 1; X < NumSamplesX - 1; X++)
				{
					const float SampleXOffset = -X / (float)(NumSamplesX - 1);
					// Weight the sample based on a pyramid centered on the texel.  
					// The sample with the maximum weight is used, which will be the center if it lies on a triangle.
					const float SampleWeight = (1 - FMath::Abs(1 + SampleXOffset * 2)) * (1 - FMath::Abs(1 + SampleYOffset * 2));
					checkSlow(SampleWeight > 0);
					// Rasterize the triangle using the mapping's texture coordinate channel.
					FTriangleRasterizer<FStaticLightingRasterPolicy> TexelMappingRasterizer(FStaticLightingRasterPolicy(
						Scene,
						TexelToVertexMap,
						SampleWeight,
						TriangleNormal,
						bDebugThisMapping,
						GeneralSettings.bUseMaxWeight
						));

					TexelMappingRasterizer.DrawTriangle(
						V0,
						V1,
						V2,
						UV0 + FVector2f(SampleXOffset, SampleYOffset),
						UV1 + FVector2f(SampleXOffset, SampleYOffset),
						UV2 + FVector2f(SampleXOffset, SampleYOffset),
						false
						);
				}
			}
		}
	}

	AdjustRepresentativeSurfelForTexelsTextureMapping(TextureMapping, TexelToVertexMap, TexelToCornersMap, nullptr, MappingContext, bDebugThisMapping);
}

void FStaticLightingSystem::AdjustRepresentativeSurfelForTexelsTextureMapping(
	FStaticLightingTextureMapping* TextureMapping, 
	FTexelToVertexMap& TexelToVertexMap, 
	FTexelToCornersMap& TexelToCornersMap,
	FGatheredLightMapData2D* LightMapData,
	FStaticLightingMappingContext& MappingContext,
	bool bDebugThisMapping) const
{
	// Iterate over each texel and normalize vectors, calculate texel radius
	for (int32 Y = 0; Y < TexelToVertexMap.GetSizeY(); Y++)
	{
		for (int32 X = 0; X < TexelToVertexMap.GetSizeX(); X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == Scene.DebugInput.LocalY
				&& X == Scene.DebugInput.LocalX)
			{
				bDebugThisTexel = true;
			}
#endif
			bool bFoundValidCorner = false;
			const FTexelToCorners& TexelToCorners = TexelToCornersMap(X, Y);
			for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
			{
				bFoundValidCorner = bFoundValidCorner || TexelToCorners.bValid[CornerIndex];
			}

			FTexelToVertex& TexelToVertex = TexelToVertexMap(X, Y);
			if (TexelToVertex.TotalSampleWeight > 0.0f || bFoundValidCorner)
			{
				// Use a corner if none of the other samples were valid
				if (TexelToVertex.TotalSampleWeight < DELTA)
				{
					for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
					{
						if (TexelToCorners.bValid[CornerIndex])
						{
							TexelToVertex.TotalSampleWeight = 1.0f;
							TexelToVertex.WorldPosition = TexelToCorners.Corners[CornerIndex].WorldPosition;
							TexelToVertex.WorldTangentX = TexelToCorners.WorldTangentX;
							TexelToVertex.WorldTangentY = TexelToCorners.WorldTangentY;
							TexelToVertex.WorldTangentZ = TexelToCorners.WorldTangentZ;
							TexelToVertex.TriangleNormal = TexelToCorners.WorldTangentZ;
							break;
						}
					}
				}
				else if (GeneralSettings.bUseMaxWeight)
				{
					// Weighted average
					TexelToVertex.WorldTangentX = TexelToVertex.WorldTangentX / TexelToVertex.TotalSampleWeight;
					TexelToVertex.WorldTangentY = TexelToVertex.WorldTangentY / TexelToVertex.TotalSampleWeight;
					TexelToVertex.WorldTangentZ = TexelToVertex.WorldTangentZ / TexelToVertex.TotalSampleWeight;
					TexelToVertex.TriangleNormal = TexelToVertex.TriangleNormal / TexelToVertex.TotalSampleWeight;

					// Weighted average of opposing vectors can result in a zero vector, fixup with corner
					if (bFoundValidCorner
						&& (TexelToVertex.WorldTangentX.SizeSquared3() < KINDA_SMALL_NUMBER
							|| TexelToVertex.WorldTangentZ.SizeSquared3() < KINDA_SMALL_NUMBER
							|| TexelToVertex.TriangleNormal.SizeSquared3() < KINDA_SMALL_NUMBER))
					{
						for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
						{
							if (TexelToCorners.bValid[CornerIndex])
							{
								TexelToVertex.WorldTangentX = TexelToCorners.WorldTangentX;
								TexelToVertex.WorldTangentY = TexelToCorners.WorldTangentY;
								TexelToVertex.WorldTangentZ = TexelToCorners.WorldTangentZ;
								TexelToVertex.TriangleNormal = TexelToCorners.WorldTangentZ;
								break;
							}
						}
					}
				}

				if (LightMapData != nullptr)
				{
					// Mark the texel as mapped to some geometry in the scene
					FGatheredLightMapSample& CurrentLightSample = (*LightMapData)(X, Y);
					CurrentLightSample.bIsMapped = true;
				}

				if (MaterialSettings.bUseNormalMapsForLighting && TextureMapping->Mesh->HasImportedNormal(TexelToVertex.ElementIndex))
				{
					const FVector4f TangentNormal = TextureMapping->Mesh->EvaluateNormal(TexelToVertex.TextureCoordinates[0], TexelToVertex.ElementIndex);

					const FVector4f WorldTangentRow0(TexelToVertex.WorldTangentX.X, TexelToVertex.WorldTangentY.X, TexelToVertex.WorldTangentZ.X);
					const FVector4f WorldTangentRow1(TexelToVertex.WorldTangentX.Y, TexelToVertex.WorldTangentY.Y, TexelToVertex.WorldTangentZ.Y);
					const FVector4f WorldTangentRow2(TexelToVertex.WorldTangentX.Z, TexelToVertex.WorldTangentY.Z, TexelToVertex.WorldTangentZ.Z);
					const FVector4f WorldVector(
						Dot3(WorldTangentRow0, TangentNormal),
						Dot3(WorldTangentRow1, TangentNormal),
						Dot3(WorldTangentRow2, TangentNormal)
					);

					TexelToVertex.WorldTangentZ = WorldVector;
				}

				// Normalize the tangent basis and ensure it is orthonormal
				TexelToVertex.WorldTangentZ = TexelToVertex.WorldTangentZ.GetUnsafeNormal3();

				const bool bUseVertexNormalForHemisphereGather = TextureMapping->Mesh->UseVertexNormalForHemisphereGather(TexelToVertex.ElementIndex);
				TexelToVertex.TriangleNormal = bUseVertexNormalForHemisphereGather ? TexelToVertex.WorldTangentZ : TexelToVertex.TriangleNormal.GetUnsafeNormal3();
				checkSlow(!TexelToVertex.TriangleNormal.ContainsNaN());

				const FVector4f OriginalTangentX = TexelToVertex.WorldTangentX;
				const FVector4f OriginalTangentY = TexelToVertex.WorldTangentY;

				TexelToVertex.WorldTangentY = (TexelToVertex.WorldTangentZ ^ TexelToVertex.WorldTangentX).GetUnsafeNormal3();
				// Maintain handedness
				if (Dot3(TexelToVertex.WorldTangentY, OriginalTangentY) < 0)
				{
					TexelToVertex.WorldTangentY *= -1.0f;
				}
				TexelToVertex.WorldTangentX = TexelToVertex.WorldTangentY ^ TexelToVertex.WorldTangentZ;
				if (Dot3(TexelToVertex.WorldTangentX, OriginalTangentX) < 0)
				{
					TexelToVertex.WorldTangentX *= -1.0f;
				}
				checkSlow(TexelToVertex.WorldTangentX.IsUnit3());
				checkSlow(TexelToVertex.WorldTangentY.IsUnit3());
				checkSlow(TexelToVertex.WorldTangentZ.IsUnit3());
				checkSlow(TexelToVertex.TriangleNormal.IsUnit3());
				checkSlow(Dot3(TexelToVertex.WorldTangentZ, TexelToVertex.WorldTangentY) < KINDA_SMALL_NUMBER);
				checkSlow(Dot3(TexelToVertex.WorldTangentX, TexelToVertex.WorldTangentY) < KINDA_SMALL_NUMBER);
				checkSlow(Dot3(TexelToVertex.WorldTangentX, TexelToVertex.WorldTangentZ) < KINDA_SMALL_NUMBER);

				// Calculate the bounding radius of the texel
				// Use the closest corner as it's likely that's on the same section of a split texel 
				// (A texel shared by multiple UV charts that has sub samples on triangles in different smoothing groups)
				float MinDistanceSquared = FLT_MAX;
				if (bFoundValidCorner)
				{
					for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
					{
						if (TexelToCorners.bValid[CornerIndex])
						{
							const float CornerDistSquared = (TexelToCorners.Corners[CornerIndex].WorldPosition - TexelToVertex.WorldPosition).SizeSquared3();
							if (CornerDistSquared < MinDistanceSquared)
							{
								MinDistanceSquared = CornerDistSquared;
							}
						}
					}
				}
				else
				{
					MinDistanceSquared = SceneConstants.SmallestTexelRadius;
				}
				TexelToVertex.TexelRadius = FMath::Max(FMath::Sqrt(MinDistanceSquared), SceneConstants.SmallestTexelRadius);
				TexelToVertex.SampleRadius = TexelToVertex.TexelRadius;
				MappingContext.Stats.NumMappedTexels++;

				{
					const FFullStaticLightingVertex FullVertex = TexelToVertex.GetFullVertex();
					const FVector4f TexelCenterOffset = FullVertex.WorldPosition + FullVertex.TriangleNormal * TexelToVertex.TexelRadius * SceneConstants.VisibilityNormalOffsetSampleRadiusScale;

					FLightRayIntersection Intersections[4];
					bool bHitBackfaces[4];

					FVector2f CornerSigns[4];
					CornerSigns[0] = FVector2f(1, 1);
					CornerSigns[1] = FVector2f(-1, 1);
					CornerSigns[2] = FVector2f(1, -1);
					CornerSigns[3] = FVector2f(-1, -1);

					float BackfaceDetectionDistance = TexelToVertex.TexelRadius * 2.5f;

					for (int32 CornerIndex = 0; CornerIndex < UE_ARRAY_COUNT(CornerSigns); CornerIndex++)
					{
						TraceToTexelCorner(
							TexelCenterOffset,
							FullVertex,
							CornerSigns[CornerIndex],
							// Note: Searching the entire influence of the texel after interpolation, which is 2x the sample radius
							BackfaceDetectionDistance,
							MappingContext,
							Intersections[CornerIndex],
							bHitBackfaces[CornerIndex],
							bDebugThisTexel);
					}

					int32 ClosestIntersectionIndex = INDEX_NONE;
					float ClosestIntersectionDistanceSq = FLT_MAX;

					int32 ClosestBackfacingIntersectionIndex = INDEX_NONE;
					// Limit the distance that we will search for an intersecting backface in order to move the shading position to the texel radius
					float ClosestBackfacingIntersectionDistanceSq = BackfaceDetectionDistance * BackfaceDetectionDistance;

					for (int32 CornerIndex = 0; CornerIndex < UE_ARRAY_COUNT(CornerSigns); CornerIndex++)
					{
						if (Intersections[CornerIndex].bIntersects)
						{
							const float DistanceSquared = (Intersections[CornerIndex].IntersectionVertex.WorldPosition - TexelCenterOffset).SizeSquared3();

							if (!bHitBackfaces[CornerIndex] && (ClosestIntersectionIndex == INDEX_NONE || DistanceSquared < ClosestIntersectionDistanceSq))
							{
								ClosestIntersectionDistanceSq = DistanceSquared;
								ClosestIntersectionIndex = CornerIndex;

								// Mark the texel as intersecting another surface so we can avoid filtering across it later
								TexelToVertex.bIntersectingSurface = true;
							}

							if (bHitBackfaces[CornerIndex] && (DistanceSquared < ClosestBackfacingIntersectionDistanceSq && !FMath::IsNearlyEqual(DistanceSquared, ClosestBackfacingIntersectionDistanceSq, SceneConstants.SmallestTexelRadius)))
							{
								ClosestBackfacingIntersectionDistanceSq = DistanceSquared;
								ClosestBackfacingIntersectionIndex = CornerIndex;

								// Mark the texel as intersecting another surface so we can avoid filtering across it later
								TexelToVertex.bIntersectingSurface = true;
							}
						}
					}

					if (ClosestIntersectionIndex != INDEX_NONE)
					{
						checkSlow(Intersections[ClosestIntersectionIndex].bIntersects);

						TexelToVertex.SampleRadius = FMath::Min(TexelToVertex.TexelRadius, FMath::Sqrt(ClosestIntersectionDistanceSq / 2.0f));
					}

					// Give preference to moving the shading position outside of backfaces
					int32 IntersectionIndexForShadingPositionMovement = ClosestBackfacingIntersectionIndex;

					// Note: this is disabled as it causes problems in cracks, the lighting position will be moved inside the object
					/*
					// Even if we didn't hit any backfaces, still move the shading position away from an intersecting frontface if it is close enough
					if (IntersectionIndexForShadingPositionMovement == INDEX_NONE
						&& ClosestIntersectionDistanceSq < (TexelToVertex.TexelRadius / 2) * (TexelToVertex.TexelRadius / 2))
					{
						IntersectionIndexForShadingPositionMovement = ClosestIntersectionIndex;
					}*/

					if (IntersectionIndexForShadingPositionMovement != INDEX_NONE)
					{
						// Move the shading position outside the surface that is intersecting this texel
						const FVector4f OffsetShadingPosition = Intersections[IntersectionIndexForShadingPositionMovement].IntersectionVertex.WorldPosition
							// Move along the intersecting surface's normal but also away from the texel a bit to prevent incorrect self occlusion
							+ (Intersections[IntersectionIndexForShadingPositionMovement].IntersectionVertex.WorldTangentZ + TexelToVertex.TriangleNormal) * 0.5f * TexelToVertex.TexelRadius * SceneConstants.VisibilityNormalOffsetSampleRadiusScale;

						// Project back onto plane of texel to avoid incorrect self occlusion
						TexelToVertex.WorldPosition = OffsetShadingPosition + TexelToVertex.TriangleNormal * Dot3(TexelToVertex.TriangleNormal, TexelToVertex.WorldPosition - OffsetShadingPosition);

						TexelToVertex.SampleRadius = (OffsetShadingPosition - Intersections[IntersectionIndexForShadingPositionMovement].IntersectionVertex.WorldPosition).Size3() / FMath::Sqrt(2.0f);
					}

					TexelToVertex.SampleRadius = FMath::Max(TexelToVertex.SampleRadius, SceneConstants.SmallestTexelRadius);
				}
			}
			else
			{
				if (LightMapData != nullptr)
				{
					// Mark unmapped texels with the supplied 'UnmappedTexelColor'.
					FGatheredLightMapSample& CurrentLightSample = (*LightMapData)(X, Y);
					CurrentLightSample.AddWeighted(FGatheredLightSampleUtil::AmbientLight<2>(Scene.GeneralSettings.UnmappedTexelColor), 1.0f);
				}
			}
		}
	}
}

void FStaticLightingSystem::FinalizeSurfaceCacheTextureMapping(FStaticLightingTextureMapping* TextureMapping)
{
#if LIGHTMASS_DO_PROCESSING
	checkSlow(TextureMapping);
	FStaticLightingMappingContext MappingContext(TextureMapping->Mesh,*this,&StartupDebugOutput);
	const FBoxSphereBounds3f ImportanceBounds = Scene.GetImportanceBounds();

	FTexelToVertexMap TexelToVertexMap(TextureMapping->SurfaceCacheSizeX, TextureMapping->SurfaceCacheSizeY);

	bool bDebugThisMapping = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	bDebugThisMapping = TextureMapping == Scene.DebugMapping;
	int32 CacheDebugX = -1;
	int32 CacheDebugY = -1;
	if (bDebugThisMapping)
	{
		CacheDebugX = FMath::TruncToInt(Scene.DebugInput.LocalX / (float)TextureMapping->CachedSizeX * TextureMapping->SurfaceCacheSizeX);
		CacheDebugY = FMath::TruncToInt(Scene.DebugInput.LocalY / (float)TextureMapping->CachedSizeY * TextureMapping->SurfaceCacheSizeY);
	}
#endif

	RasterizeToSurfaceCacheTextureMapping(TextureMapping, bDebugThisMapping, TexelToVertexMap);

	for (int32 Y = 0; Y < TextureMapping->SurfaceCacheSizeY; Y++)
	{
		for (int32 X = 0; X < TextureMapping->SurfaceCacheSizeX; X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == CacheDebugX
				&& X == CacheDebugY)
			{
				bDebugThisTexel = true;
			}
#endif
			const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);

			if (TexelToVertex.TotalSampleWeight > 0.0f)
			{
				FFullStaticLightingVertex CurrentVertex = TexelToVertex.GetFullVertex();
				CurrentVertex.ApplyVertexModifications(TexelToVertex.ElementIndex, MaterialSettings.bUseNormalMapsForLighting, TextureMapping->Mesh);

				const int32 SurfaceCacheIndex = Y * TextureMapping->SurfaceCacheSizeX + X;

				FLinearColor FinalIncidentLighting = FLinearColor::Black;

				// SurfaceCacheLighting at this point contains 1st and up bounce lighting for the skylight and emissive sources, computed by the radiosity iterations
				FinalIncidentLighting += TextureMapping->SurfaceCacheLighting[SurfaceCacheIndex];

				if ((GeneralSettings.ViewSingleBounceNumber < 0 || GeneralSettings.ViewSingleBounceNumber >= 2)
					&& !ImportanceTracingSettings.bUseRadiositySolverForLightMultibounce
					&& PhotonMappingSettings.bUseIrradiancePhotons)
				{
					const FIrradiancePhoton* NearestPhoton = TextureMapping->CachedIrradiancePhotons[SurfaceCacheIndex];

					if (NearestPhoton)
					{
						// The irradiance photon contains 2nd and up bounce lighting for point / spot / directional lights (since they emit photons)
						FinalIncidentLighting += NearestPhoton->GetIrradiance();
					}
				}

				if (GeneralSettings.ViewSingleBounceNumber < 0 || GeneralSettings.ViewSingleBounceNumber == 1)
				{
					FGatheredLightSample DirectLighting;
					float Unused2;
					TArray<FVector3f, TInlineAllocator<1>> VertexOffsets;
					VertexOffsets.Add(FVector3f(0, 0, 0));

					CalculateApproximateDirectLighting(CurrentVertex, TexelToVertex.TexelRadius, VertexOffsets, .1f, true, true, bDebugThisTexel && PhotonMappingSettings.bVisualizeCachedApproximateDirectLighting, MappingContext, DirectLighting, Unused2);
					FinalIncidentLighting += DirectLighting.IncidentLighting;
				}

				const bool bTranslucent = TextureMapping->Mesh->IsTranslucent(TexelToVertex.ElementIndex);
				const FLinearColor Reflectance = (bTranslucent ? FLinearColor::Black : TextureMapping->Mesh->EvaluateTotalReflectance(CurrentVertex, TexelToVertex.ElementIndex)) * (float)INV_PI;
				// Combine all the lighting and surface reflectance so the final gather ray only needs one memory fetch
				TextureMapping->SurfaceCacheLighting[SurfaceCacheIndex] = FinalIncidentLighting * Reflectance;
			}
		}
	}

	TextureMapping->CachedIrradiancePhotons.Empty();
#endif
}

/**
 * Builds lighting for a texture mapping.
 * @param TextureMapping - The mapping to build lighting for.
 */
void FStaticLightingSystem::ProcessTextureMapping(FStaticLightingTextureMapping* TextureMapping)
{
	FPlatformAtomics::InterlockedIncrement(&TasksInProgressThatWillNeedHelp);
	checkSlow(TextureMapping);
	// calculate the total time just for processing
	double StartTime = FPlatformTime::Seconds();

	bool bDebugThisMapping = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	bDebugThisMapping = TextureMapping == Scene.DebugMapping;
#endif

	TList<FTextureMappingStaticLightingData>* StaticLightingLink = new TList<FTextureMappingStaticLightingData>(FTextureMappingStaticLightingData(),NULL);

	// light guid to shadow map mapping
	TMap<const FLight*, FShadowMapData2D*> ShadowMaps;
	TMap<const FLight*, FSignedDistanceFieldShadowMapData2D*> SignedDistanceFieldShadowMaps;

	if (bDebugThisMapping)
	{
		StaticLightingLink->Element.DebugOutput = StartupDebugOutput;
	}

	FStaticLightingMappingContext MappingContext(TextureMapping->Mesh, *this, &StaticLightingLink->Element.DebugOutput);

	// Allocate light-map data.
	FGatheredLightMapData2D LightMapData(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);

	LightMapData.bHasSkyShadowing = HasSkyShadowing();

	// if we have a debug texel, then only compute the lighting for this mapping
	bool bCalculateThisMapping = true;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	// we want to skip mappings if the setting is enabled, and we have a debug mapping, and it's not this one
	bCalculateThisMapping = !(Scene.bOnlyCalcDebugTexelMappings && Scene.DebugMapping != NULL && !bDebugThisMapping);
#endif

	// Allocate the texel to vertex map.
	FTexelToVertexMap TexelToVertexMap(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);

	const double TexelRasterizationStart = FPlatformTime::Seconds();
	// Allocate a map from texel to the corners of that texel
	FTexelToCornersMap TexelToCornersMap(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);
	SetupTextureMapping(TextureMapping, LightMapData, TexelToVertexMap, TexelToCornersMap, MappingContext, bDebugThisMapping);
	MappingContext.Stats.TexelRasterizationTime += FPlatformTime::Seconds() - TexelRasterizationStart;

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisMapping)
	{
		MappingContext.DebugOutput->bValid = true;
		for (int32 Y = 0;Y < TextureMapping->CachedSizeY;Y++)
		{
			for (int32 X = 0;X < TextureMapping->CachedSizeX;X++)
			{
				const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
				if (TexelToVertex.TotalSampleWeight > 0.0f)
				{
					// Verify that vertex normals are normalized (within some error that is large because of FPackedNormal)
					checkSlow(FVector3f(TexelToVertex.WorldTangentZ).IsUnit(.1f));

					const float DistanceToDebugTexelSq = FVector3f(TexelToVertex.WorldPosition - Scene.DebugInput.Position).SizeSquared();
					if (DistanceToDebugTexelSq < 40000
						|| X == Scene.DebugInput.LocalX && Y == Scene.DebugInput.LocalY)
					{
						if (X == Scene.DebugInput.LocalX && Y == Scene.DebugInput.LocalY)
						{		
							FDebugStaticLightingVertex DebugVertex;
							DebugVertex.VertexNormal = FVector4f(TexelToVertex.WorldTangentZ);
							DebugVertex.VertexPosition = TexelToVertex.WorldPosition;
							MappingContext.DebugOutput->Vertices.Add(DebugVertex);

							MappingContext.DebugOutput->SelectedVertexIndices.Add(MappingContext.DebugOutput->Vertices.Num() - 1);
							MappingContext.DebugOutput->SampleRadius = TexelToVertex.TexelRadius;
						}
					}
				}
			}
		}
	}
#endif

#if LIGHTMASS_DO_PROCESSING

	if (bCalculateThisMapping)
	{
		const double DirectLightingStartTime = FPlatformTime::Seconds();
		const bool bCalculateDirectLightingFromPhotons = PhotonMappingSettings.bUsePhotonMapping && PhotonMappingSettings.bVisualizeCachedApproximateDirectLighting;
		// Only continue if photon mapping will not be used for direct lighting
		if (!bCalculateDirectLightingFromPhotons)
		{
			// Iterate over each light that is relevant to the direct lighting of the mesh
			for (int32 LightIndex = 0; LightIndex < TextureMapping->Mesh->RelevantLights.Num(); LightIndex++)
			{
				const FLight* Light = TextureMapping->Mesh->RelevantLights[LightIndex];

				// skip sky lights for now
				if (Light->GetSkyLight())
				{
					continue;
				}

				if (!Light->AffectsBounds(FBoxSphereBounds3f(TextureMapping->Mesh->BoundingBox)))
				{
					continue;
				}

				if ( ShadowSettings.bUseZeroAreaLightmapSpaceFilteredLights)
				{
					// Calculate direct lighting from lights as if they have no area, and then filter in texture space to create approximate penumbras.
					CalculateDirectLightingTextureMappingFiltered(TextureMapping, MappingContext, LightMapData, ShadowMaps, TexelToVertexMap, bDebugThisMapping, Light);
				}
				else 
				{
					if (!Light->UseStaticLighting()
						&& (Light->LightFlags & GI_LIGHT_CASTSHADOWS) 
						&& (Light->LightFlags & GI_LIGHT_CASTSTATICSHADOWS)
						&& (Light->LightFlags & GI_LIGHT_STORE_SEPARATE_SHADOW_FACTOR)
						&& ShadowSettings.bAllowSignedDistanceFieldShadows)
					{
						if (Light->LightFlags & GI_LIGHT_USE_AREA_SHADOWS_FOR_SEPARATE_SHADOW_FACTOR)
						{
							FShadowMapData2D* ShadowMapData = new FShadowMapData2D(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY);
							CalculateDirectAreaLightingTextureMapping(TextureMapping, MappingContext, LightMapData, ShadowMapData, TexelToVertexMap, bDebugThisMapping, Light);

							if (ShadowMapData)
							{
								FSignedDistanceFieldShadowMapData2D* ConvertedShadowMapData = new FSignedDistanceFieldShadowMapData2D(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);

								for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
								{
									for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
									{
										const FShadowSample& SourceShadowSample = (*ShadowMapData)(X,Y);
										FSignedDistanceFieldShadowSample& DestShadowSample = (*ConvertedShadowMapData)(X,Y);
										DestShadowSample.bIsMapped = SourceShadowSample.bIsMapped;
										// Encode with more precision near 0
										// The decode shader code will undo this in GetPrecomputedShadowMasks
										DestShadowSample.Distance = FMath::Sqrt(FMath::Clamp(SourceShadowSample.Visibility, 0.0f, 1.0f));
										DestShadowSample.PenumbraSize = 1;
									}
								}

								SignedDistanceFieldShadowMaps.Add(Light, ConvertedShadowMapData);
							}
						}
						else
						{
							const bool bUseTextureSpaceDistanceFieldGeneration = true;

							if (bUseTextureSpaceDistanceFieldGeneration)
							{
								// Calculate distance field shadows, where the distance to the nearest shadow transition is stored instead of just a [0,1] shadow factor.
								CalculateDirectSignedDistanceFieldLightingTextureMappingTextureSpace(TextureMapping, MappingContext, LightMapData, SignedDistanceFieldShadowMaps, TexelToVertexMap, TexelToCornersMap, bDebugThisMapping, Light);
							}
							else
							{
								// Experimental method that avoids artifacts due to lightmap seams
								CalculateDirectSignedDistanceFieldLightingTextureMappingLightSpace(TextureMapping, MappingContext, LightMapData, SignedDistanceFieldShadowMaps, TexelToVertexMap, TexelToCornersMap, bDebugThisMapping, Light);
							}
						}
					}
					else if (Light->UseStaticLighting())
					{
						FShadowMapData2D* ShadowMapData = NULL;

						// Calculate direct lighting from area lights
						// Shadow penumbras will be correctly shaped and will be softer for larger light sources and distant shadow casters.
						CalculateDirectAreaLightingTextureMapping(TextureMapping, MappingContext, LightMapData, ShadowMapData, TexelToVertexMap, bDebugThisMapping, Light);

						if (Light->GetMeshAreaLight() == NULL)
						{
							LightMapData.AddLight(Light);
						}
					}
				}
			}
		}

		// Release corner information as it is no longer needed
		TexelToCornersMap.Empty();

		// Calculate direct lighting using the direct photon map.
		// This is only useful for debugging what the final gather rays see.
		if (bCalculateDirectLightingFromPhotons)
		{
			CalculateDirectLightingTextureMappingPhotonMap(TextureMapping, MappingContext, LightMapData, ShadowMaps, TexelToVertexMap, bDebugThisMapping);
		}
		MappingContext.Stats.DirectLightingTime += FPlatformTime::Seconds() - DirectLightingStartTime;

		CalculateIndirectLightingTextureMapping(TextureMapping, MappingContext, LightMapData, TexelToVertexMap, bDebugThisMapping);

		const double ErrorAndMaterialColoringStart = FPlatformTime::Seconds();
		ViewMaterialAttributesTextureMapping(TextureMapping, MappingContext, LightMapData, TexelToVertexMap, bDebugThisMapping);
		ColorInvalidLightmapUVs(TextureMapping, LightMapData, bDebugThisMapping);

		// Count the time doing material coloring and invalid lightmap UV color toward texel setup for now
		MappingContext.Stats.TexelRasterizationTime += FPlatformTime::Seconds() - ErrorAndMaterialColoringStart;
	}
#else
	FPlatformAtomics::InterlockedDecrement(&TasksInProgressThatWillNeedHelp);
#endif

	const double PaddingStart = FPlatformTime::Seconds();
	
	FGatheredLightMapData2D PaddedLightMapData(TextureMapping->SizeX, TextureMapping->SizeY);
	PadTextureMapping(TextureMapping, LightMapData, PaddedLightMapData, ShadowMaps, SignedDistanceFieldShadowMaps);
	LightMapData.Empty();

	// calculate the total time just for processing
	const double ExecutionTimeForColoring = FPlatformTime::Seconds() - StartTime;

	if (!bCalculateThisMapping || Scene.bColorBordersGreen || Scene.bColorByExecutionTime || Scene.bUseRandomColors)
	{
		bool bColorNonBorders = Scene.bColorByExecutionTime || Scene.bUseRandomColors;

		// calculate what color to put in each spot, if overriding
		FLinearColor OverrideColor(0, 0, 0);
		if (Scene.bColorByExecutionTime)
		{
			OverrideColor.R = ExecutionTimeForColoring / (Scene.ExecutionTimeDivisor ? Scene.ExecutionTimeDivisor : 15.0f);
		}
		else if (Scene.bUseRandomColors)
		{
			// make each mapping solid, random colors
			static FLMRandomStream RandomStream(0);

			// make a random color
			OverrideColor.R = RandomStream.GetFraction();
			OverrideColor.G = RandomStream.GetFraction();
			OverrideColor.B = RandomStream.GetFraction();

			if (Scene.bColorBordersGreen)
			{
				// not too green tho so borders show up
				OverrideColor.G /= 2.0f;
			}
		}
		else if (!bCalculateThisMapping)
		{
			OverrideColor = FLinearColor::White;
		}

		FLinearColor Green(0, 1.0, 0);

		for (uint32 Y = 0; Y < PaddedLightMapData.GetSizeY(); Y++)
		{
			for (uint32 X = 0; X < PaddedLightMapData.GetSizeX(); X++)
			{
				FGatheredLightMapSample& Sample = PaddedLightMapData(X, Y);
				bool bIsBorder = (X <= 1 || Y <= 1 || X >= PaddedLightMapData.GetSizeX() - 2 || Y >= PaddedLightMapData.GetSizeY() - 2);
				if (!bCalculateThisMapping || (Sample.bIsMapped && bColorNonBorders) || (bIsBorder && Scene.bColorBordersGreen))
				{
					FLinearColor& SampleColor = (bIsBorder && Scene.bColorBordersGreen) ? Green : OverrideColor;

					Sample.HighQuality.AddWeighted(FGatheredLightSampleUtil::AmbientLight<2>(SampleColor), 1.0f);
					Sample.LowQuality.AddWeighted(FGatheredLightSampleUtil::AmbientLight<2>(SampleColor), 1.0f);
				}
			}
		}
	}

	const int32 PaddedDebugX = TextureMapping->bPadded ? Scene.DebugInput.LocalX + 1 : Scene.DebugInput.LocalX;
	const int32 PaddedDebugY = TextureMapping->bPadded ? Scene.DebugInput.LocalY + 1 : Scene.DebugInput.LocalY;
	FLightMapData2D* FinalLightmapData = PaddedLightMapData.ConvertToLightmap2D(bDebugThisMapping, PaddedDebugX, PaddedDebugY);

	// Count the time doing padding and lightmap coloring toward texel setup
	const double CurrentTime = FPlatformTime::Seconds();
	MappingContext.Stats.TexelRasterizationTime += CurrentTime - PaddingStart;
	const double ExecutionTime = CurrentTime - StartTime;

	// Enqueue the static lighting for application in the main thread.
	StaticLightingLink->Element.Mapping = TextureMapping;
	StaticLightingLink->Element.LightMapData = FinalLightmapData;
	StaticLightingLink->Element.ShadowMaps = ShadowMaps;
	StaticLightingLink->Element.SignedDistanceFieldShadowMaps = SignedDistanceFieldShadowMaps;
	StaticLightingLink->Element.ExecutionTime = ExecutionTime;
	MappingContext.Stats.TotalTextureMappingLightingThreadTime = ExecutionTime;

	const int32 PaddedOffset = TextureMapping->bPadded ? 1 : 0;
	const int32 DebugSampleIndex = (Scene.DebugInput.LocalY + PaddedOffset) * TextureMapping->SizeX + Scene.DebugInput.LocalX + PaddedOffset;

	CompleteTextureMappingList.AddElement(StaticLightingLink);

	const int32 OldNumTexelsCompleted = FPlatformAtomics::InterlockedAdd(&NumTexelsCompleted, TextureMapping->CachedSizeX * TextureMapping->CachedSizeY);
	UpdateInternalStatus(OldNumTexelsCompleted);
}

class FTexelCornerRasterPolicy
{
public:

	typedef FStaticLightingVertex InterpolantType;

	/** Initialization constructor. */
	FTexelCornerRasterPolicy(
		const FScene& InScene,
		FTexelToCornersMap& InTexelToCornersMap,
		int32 InCornerIndex,
		bool bInDebugThisMapping
		):
		Scene(InScene),
		TexelToCornersMap(InTexelToCornersMap),
		CornerIndex(InCornerIndex),
		bDebugThisMapping(bInDebugThisMapping)
	{
	}

protected:

	// FTriangleRasterizer policy interface.

	int32 GetMinX() const { return 0; }
	int32 GetMaxX() const { return TexelToCornersMap.GetSizeX() - 1; }
	int32 GetMinY() const { return 0; }
	int32 GetMaxY() const { return TexelToCornersMap.GetSizeY() - 1; }

	void ProcessPixel(int32 X, int32 Y, const InterpolantType& Interpolant, bool BackFacing);

private:

	const FScene& Scene;
	
	/** The texel to vertex map which is being rasterized to. */
	FTexelToCornersMap& TexelToCornersMap;

	/** Index of the current corner being rasterized */
	const int32 CornerIndex;

	const bool bDebugThisMapping;
};

void FTexelCornerRasterPolicy::ProcessPixel(int32 X, int32 Y, const InterpolantType& Vertex, bool BackFacing)
{
	FTexelToCorners& TexelToCorners = TexelToCornersMap(X, Y);

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisMapping
		&& X == Scene.DebugInput.LocalX
		&& Y == Scene.DebugInput.LocalY)
	{
		int32 TempBreak = 0;
	}
#endif

	TexelToCorners.Corners[CornerIndex].WorldPosition = Vertex.WorldPosition;
	TexelToCorners.WorldTangentX = Vertex.WorldTangentX;
	TexelToCorners.WorldTangentY = Vertex.WorldTangentY;
	TexelToCorners.WorldTangentZ = Vertex.WorldTangentZ;
	TexelToCorners.bValid[CornerIndex] = true;
}

void FStaticLightingSystem::TraceToTexelCorner(
	const FVector4f& TexelCenterOffset, 
	const FFullStaticLightingVertex& FullVertex, 
	FVector2f CornerSigns,
	float TexelRadius, 
	FStaticLightingMappingContext& MappingContext, 
	FLightRayIntersection& Intersection,
	bool& bHitBackface,
	bool bDebugThisTexel) const
{
	// Vector from the center to one of the corners of the texel
	// The FMath::Sqrt(.5f) is to normalize (Vertex.TriangleTangentX + Vertex.TriangleTangentY), which are orthogonal unit vectors.
	const FVector4f CornerOffset = FMath::Sqrt(.5f) * (CornerSigns.X * FullVertex.TriangleTangentX + CornerSigns.Y * FullVertex.TriangleTangentY) * TexelRadius * SceneConstants.VisibilityTangentOffsetSampleRadiusScale;
	const FLightRay TexelRay(
		TexelCenterOffset,
		TexelCenterOffset + CornerOffset,
		NULL,
		NULL
		);

	AggregateMesh->IntersectLightRay(TexelRay, true, false, false, MappingContext.RayCache, Intersection);

	bHitBackface = Intersection.bIntersects && Dot3(Intersection.IntersectionVertex.WorldTangentZ, TexelRay.Direction) >= 0 && !Intersection.Mesh->IsTwoSided(Intersection.ElementIndex);

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisTexel)
	{
		FDebugStaticLightingRay DebugRay(TexelRay.Start, TexelRay.End, Intersection.bIntersects);
		if (Intersection.bIntersects)
		{
			DebugRay.End = Intersection.IntersectionVertex.WorldPosition;
		}
		MappingContext.DebugOutput->ShadowRays.Add(DebugRay);
	}
#endif
}

/** Calculates TexelToVertexMap and initializes each texel's light sample as mapped or not. */
void FStaticLightingSystem::SetupTextureMapping(
	FStaticLightingTextureMapping* TextureMapping, 
	FGatheredLightMapData2D& LightMapData, 
	FTexelToVertexMap& TexelToVertexMap, 
	FTexelToCornersMap& TexelToCornersMap,
	FStaticLightingMappingContext& MappingContext,
	bool bDebugThisMapping) const
{
	CalculateTexelCorners(TextureMapping->Mesh, TexelToCornersMap, TextureMapping->LightmapTextureCoordinateIndex, bDebugThisMapping);

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisMapping)
	{
		const FTexelToCorners& TexelToCorners = TexelToCornersMap(Scene.DebugInput.LocalX, Scene.DebugInput.LocalY);
		for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
		{
			MappingContext.DebugOutput->TexelCorners[CornerIndex] = TexelToCorners.Corners[CornerIndex].WorldPosition;
			MappingContext.DebugOutput->bCornerValid[CornerIndex] = TexelToCorners.bValid[CornerIndex] != 0;
		}
	}
#endif

	// Rasterize the triangles into the texel to vertex map.
	if (GeneralSettings.bUseConservativeTexelRasterization && TextureMapping->bBilinearFilter == true)
	{
		// Using conservative rasterization, which uses super sampling to try to detect all texels that should be mapped.
		for(int32 TriangleIndex = 0;TriangleIndex < TextureMapping->Mesh->NumTriangles;TriangleIndex++)
		{
			// Query the mesh for the triangle's vertices.
			FStaticLightingInterpolant V0;
			FStaticLightingInterpolant V1;
			FStaticLightingInterpolant V2;
			int32 Element;
			TextureMapping->Mesh->GetTriangle(TriangleIndex,V0.Vertex,V1.Vertex,V2.Vertex,Element);
			V0.ElementIndex = V1.ElementIndex = V2.ElementIndex = Element;

			const FVector4f TriangleNormal = ((V2.Vertex.WorldPosition - V0.Vertex.WorldPosition) ^ (V1.Vertex.WorldPosition - V0.Vertex.WorldPosition)).GetSafeNormal();

			// Don't rasterize degenerates 
			if (!TriangleNormal.IsNearlyZero3())
			{
				const FVector2f UV0 = V0.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY);
				const FVector2f UV1 = V1.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY);
				const FVector2f UV2 = V2.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY);

				// Odd number of samples so that the center of the pyramid is on one of the samples
				const uint32 NumSamplesX = 7;
				const uint32 NumSamplesY = 7;

				// Rasterize multiple sub-texel samples and linearly combine the results
				// Don't rasterize the first or last row and column as the weight will be 0
				for(int32 Y = 1; Y < NumSamplesY - 1; Y++)
				{
					const float SampleYOffset = -Y / (float)(NumSamplesY - 1);
					for(int32 X = 1; X < NumSamplesX - 1; X++)
					{
						const float SampleXOffset = -X / (float)(NumSamplesX - 1);
						// Weight the sample based on a pyramid centered on the texel.  
						// The sample with the maximum weight is used, which will be the center if it lies on a triangle.
						const float SampleWeight = (1 - FMath::Abs(1 + SampleXOffset * 2)) * (1 - FMath::Abs(1 + SampleYOffset * 2));
						checkSlow(SampleWeight > 0);
						// Rasterize the triangle using the mapping's texture coordinate channel.
						FTriangleRasterizer<FStaticLightingRasterPolicy> TexelMappingRasterizer(FStaticLightingRasterPolicy(
							Scene,
							TexelToVertexMap,
							SampleWeight,
							TriangleNormal,
							bDebugThisMapping,
							GeneralSettings.bUseMaxWeight
							));

						TexelMappingRasterizer.DrawTriangle(
							V0,
							V1,
							V2,
							UV0 + FVector2f(SampleXOffset, SampleYOffset),
							UV1 + FVector2f(SampleXOffset, SampleYOffset),
							UV2 + FVector2f(SampleXOffset, SampleYOffset),
							false
							);
					}
				}
			}
		}
	}
	else
	{
		// Only rasterizing one sample at the texel's center.  If the center does not lie on a triangle, the texel will not be mapped.
		const float SampleWeight = 1.0f;
		// Rasterize the triangles offset by the random sample location.
		for(int32 TriangleIndex = 0;TriangleIndex < TextureMapping->Mesh->NumTriangles;TriangleIndex++)
		{
			// Query the mesh for the triangle's vertices.
			FStaticLightingInterpolant V0;
			FStaticLightingInterpolant V1;
			FStaticLightingInterpolant V2;
			int32 Element;
			TextureMapping->Mesh->GetTriangle(TriangleIndex,V0.Vertex,V1.Vertex,V2.Vertex,Element);
			V0.ElementIndex = V1.ElementIndex = V2.ElementIndex = Element;

			// Rasterize the triangle using the mapping's texture coordinate channel.
			FTriangleRasterizer<FStaticLightingRasterPolicy> TexelMappingRasterizer(FStaticLightingRasterPolicy(
				Scene,
				TexelToVertexMap,
				SampleWeight,
				FVector4f(0),
				bDebugThisMapping,
				false
				));

			// Only rasterize the center of the texel, any texel whose center does not lie on a triangle will not be mapped.
			TexelMappingRasterizer.DrawTriangle(
				V0,
				V1,
				V2,
				V0.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY) + FVector2f(-0.5f,-0.5f),
				V1.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY) + FVector2f(-0.5f,-0.5f),
				V2.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY) + FVector2f(-0.5f,-0.5f),
				false
				);
		}
	}

	AdjustRepresentativeSurfelForTexelsTextureMapping(TextureMapping, TexelToVertexMap, TexelToCornersMap, &LightMapData, MappingContext, bDebugThisMapping);
}

/** Calculates direct lighting as if all lights were non-area lights, then filters the results in texture space to create approximate soft shadows. */
void FStaticLightingSystem::CalculateDirectLightingTextureMappingFiltered(
	FStaticLightingTextureMapping* TextureMapping, 
	FStaticLightingMappingContext& MappingContext,
	FGatheredLightMapData2D& LightMapData, 
	TMap<const FLight*, FShadowMapData2D*>& ShadowMaps,
	const FTexelToVertexMap& TexelToVertexMap, 
	bool bDebugThisMapping,
	const FLight* Light) const
{
	// Raytrace the texels of the shadow-map that map to vertices on a world-space surface.
	FShadowMapData2D ShadowMapData(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY);
	for (int32 Y = 0;Y < TextureMapping->CachedSizeY;Y++)
	{
		for (int32 X = 0;X < TextureMapping->CachedSizeX;X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == Scene.DebugInput.LocalY
				&& X == Scene.DebugInput.LocalX)
			{
				bDebugThisTexel = true;
			}
#endif
			const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
			if (TexelToVertex.TotalSampleWeight > 0.0f)
			{
				FShadowSample& ShadowSample = ShadowMapData(X,Y);
				ShadowSample.bIsMapped = true;

				// Check if the light is in front of the surface.
				const bool bLightIsInFrontOfTriangle = !IsLightBehindSurface(TexelToVertex.WorldPosition,FVector4f(TexelToVertex.WorldTangentZ),Light);
				if (bLightIsInFrontOfTriangle || TextureMapping->Mesh->IsTwoSided(TexelToVertex.ElementIndex))
				{
					// Compute the shadow factors for this sample from the shadow-mapped lights.
					ShadowSample.Visibility = CalculatePointShadowing(TextureMapping,TexelToVertex.WorldPosition,Light,MappingContext,bDebugThisTexel) 
						? 0.0f : 1.0f;
				}
			}
		}
	}

	// Filter the shadow-map, and detect completely occluded lights.
	FShadowMapData2D* FilteredShadowMapData = new FShadowMapData2D(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY);;
	bool bIsCompletelyOccluded = true;
	for (int32 Y = 0;Y < TextureMapping->CachedSizeY;Y++)
	{
		for (int32 X = 0;X < TextureMapping->CachedSizeX;X++)
		{
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == Scene.DebugInput.LocalY
				&& X == Scene.DebugInput.LocalX)
			{
				int32 TempBreak = 0;
			}
#endif
			if (ShadowMapData(X,Y).bIsMapped)
			{
				uint32 Visibility = 0;
				uint32 Coverage = 0;
				// The shadow-map filter.
				static const uint32 FilterSizeX = 5;
				static const uint32 FilterSizeY = 5;
				static const uint32 FilterMiddleX = (FilterSizeX - 1) / 2;
				static const uint32 FilterMiddleY = (FilterSizeY - 1) / 2;
				static const uint32 Filter[5][5] =
				{
					{ 58,  85,  96,  85, 58 },
					{ 85, 123, 140, 123, 85 },
					{ 96, 140, 159, 140, 96 },
					{ 85, 123, 140, 123, 85 },
					{ 58,  85,  96,  85, 58 }
				};
				// Gather the filtered samples for this texel.
				for (uint32 FilterY = 0;FilterY < FilterSizeX;FilterY++)
				{
					for (uint32 FilterX = 0;FilterX < FilterSizeY;FilterX++)
					{
						int32	SubX = (int32)X - FilterMiddleX + FilterX,
							SubY = (int32)Y - FilterMiddleY + FilterY;
						if (SubX >= 0 && SubX < (int32)TextureMapping->CachedSizeX && SubY >= 0 && SubY < (int32)TextureMapping->CachedSizeY)
						{
							if (ShadowMapData(SubX,SubY).bIsMapped)
							{
								Visibility += FMath::TruncToInt(Filter[FilterX][FilterY] * ShadowMapData(SubX,SubY).Visibility);
								Coverage += Filter[FilterX][FilterY];
							}
						}
					}
				}

				// Keep track of whether any texels have an unoccluded view of the light.
				if (Visibility > 0)
				{
					bIsCompletelyOccluded = false;
				}

				// Write the filtered shadow-map texel.
				(*FilteredShadowMapData)(X,Y).Visibility = (float)Visibility / (float)Coverage;
				(*FilteredShadowMapData)(X,Y).bIsMapped = true;
			}
			else
			{
				(*FilteredShadowMapData)(X,Y).bIsMapped = false;
			}
		}
	}

	if(bIsCompletelyOccluded) 
	{
		// If the light is completely occluded, discard the shadow-map.
		delete FilteredShadowMapData;
		FilteredShadowMapData = NULL;
	}
	else
	{
		// Check whether the light should use a light-map or shadow-map.
		const bool bUseStaticLighting = Light->UseStaticLighting();
		if (bUseStaticLighting)
		{
			// Convert the shadow-map into a light-map.
			for (int32 Y = 0;Y < TextureMapping->CachedSizeY;Y++)
			{
				for (int32 X = 0;X < TextureMapping->CachedSizeX;X++)
				{
					bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
					if (bDebugThisMapping
						&& Y == Scene.DebugInput.LocalY
						&& X == Scene.DebugInput.LocalX)
					{
						bDebugThisTexel = true;
					}
#endif
					if ((*FilteredShadowMapData)(X,Y).bIsMapped)
					{
						const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
						LightMapData(X,Y).bIsMapped = true;

						// Compute the light sample for this texel based on the corresponding vertex and its shadow factor.
						float ShadowFactor = (*FilteredShadowMapData)(X,Y).Visibility;
						if (ShadowFactor > 0.0f)
						{
							// Calculate the lighting for the texel.
							check(TexelToVertex.TotalSampleWeight > 0.0f);
							const FStaticLightingVertex CurrentVertex = TexelToVertex.GetVertex();
							const FLinearColor LightIntensity = Light->GetDirectIntensity(CurrentVertex.WorldPosition, false);
							const FGatheredLightSample DirectLighting = CalculatePointLighting(TextureMapping, CurrentVertex, TexelToVertex.ElementIndex, Light, LightIntensity, FLinearColor::White);
							if (GeneralSettings.ViewSingleBounceNumber < 1)
							{
								LightMapData(X,Y).AddWeighted(DirectLighting, ShadowFactor);
							}
						}
					}
				}
			}

			// Add the light to the light-map's light list.
			LightMapData.AddLight(Light);

			// Free the shadow-map.
			delete FilteredShadowMapData;
		}
		// only allow for shadow maps if shadow casting is enabled
		else if ((Light->LightFlags & GI_LIGHT_CASTSHADOWS) && (Light->LightFlags & GI_LIGHT_CASTSTATICSHADOWS))
		{
			ShadowMaps.Add(Light,FilteredShadowMapData);
		}
		else
		{
			delete FilteredShadowMapData;
			FilteredShadowMapData = NULL;
		}
	}
}

/**
 * Calculate lighting from area lights, with filtering in texture space only optionally across severe gradients
 * in the shadow factor. Shadow penumbras will be correctly shaped and will be softer for larger light sources
 * and distant shadow casters.
 */
void FStaticLightingSystem::CalculateDirectAreaLightingTextureMapping(
	FStaticLightingTextureMapping* TextureMapping, 
	FStaticLightingMappingContext& MappingContext,
	FGatheredLightMapData2D& LightMapData, 
	FShadowMapData2D*& ShadowMapData,
	const FTexelToVertexMap& TexelToVertexMap, 
	bool bDebugThisMapping,
	const FLight* Light) const
{
	LIGHTINGSTAT(FScopedRDTSCTimer AreaShadowsTimer(MappingContext.Stats.AreaShadowsThreadTime));

	bool bIsCompletelyOccluded = true;
	FLMRandomStream SampleGenerator(0);

	// Used for the optional lightmap gradient filtering pass
	bool bShadowFactorFilterPassEnabled = false;
	FShadowMapData2D UnfilteredShadowFactorData(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);
	FShadowMapData2D FilteredShadowFactorData(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);
	TArray<FLinearColor> TransmissionCache;
	TransmissionCache.Empty(TextureMapping->CachedSizeX * TextureMapping->CachedSizeY);
	TransmissionCache.AddZeroed(TextureMapping->CachedSizeX * TextureMapping->CachedSizeY);
	TArray<FLinearColor> LightIntensityCache;
	LightIntensityCache.Empty(TextureMapping->CachedSizeX * TextureMapping->CachedSizeY);
	LightIntensityCache.AddZeroed(TextureMapping->CachedSizeX * TextureMapping->CachedSizeY);

	for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
	{
		for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == Scene.DebugInput.LocalY
				&& X == Scene.DebugInput.LocalX)
			{
				bDebugThisTexel = true;
			}
#endif
			FGatheredLightMapSample& CurrentLightSample = LightMapData(X, Y);
			const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);

			if (CurrentLightSample.bIsMapped
				&& Light->AffectsBounds(FBoxSphereBounds3f(FSphere3f(TexelToVertex.WorldPosition, TexelToVertex.TexelRadius * 2))))
			{
				UnfilteredShadowFactorData(X, Y).bIsMapped = true;

				if (ShadowMapData)
				{
					FShadowSample& CurrentShadowSample = (*ShadowMapData)(X,Y);
					CurrentShadowSample.bIsMapped = true;
				}

				// Only continue if some part of the light is in front of the surface
				const FStaticLightingVertex Vertex = TexelToVertex.GetVertex();

				// @todo: Because we test for rays backfacing the smoothed triangle normal, this code
				// will not skip lighting texels whose tangent space normals are still light-facing,
				// potentially yielding a lighting seam.  We should change this code to only cull
				// rays that are backfacing both the tangent space normal and the smoothed vertex normal
				// by a reasonably small threshold, and then make sure the lighting code handles rays
				// that aren't necessarily in front of the triangle robustly.
				//
				//		const FVector4f Normal = Vertex.TransformTangentVectorToWorld(TextureMapping->Mesh->EvaluateNormal(Vertex.TextureCoordinates[0], TexelToVertex.ElementIndex)) :*/
				const FVector4f Normal = Vertex.WorldTangentZ;

				const bool bLightIsInFrontOfTriangle = !Light->BehindSurface(TexelToVertex.WorldPosition, Normal);
				if (bLightIsInFrontOfTriangle || TextureMapping->Mesh->IsTwoSided(TexelToVertex.ElementIndex))
				{
					const FStaticLightingVertex CurrentVertex = TexelToVertex.GetVertex();
					FLinearColor LightIntensity = FLinearColor::White;
					bool bTraceShadowRays = true;

					// Potentially avoid additional work below if this light has no meaningful contribution
					if (bTraceShadowRays)
					{
						// Compute the incident lighting of the light on the vertex.
						LightIntensity = Light->GetDirectIntensity(CurrentVertex.WorldPosition, false);
						if ((LightIntensity.R <= KINDA_SMALL_NUMBER) &&
							(LightIntensity.G <= KINDA_SMALL_NUMBER) &&
							(LightIntensity.B <= KINDA_SMALL_NUMBER) &&
							(LightIntensity.A <= KINDA_SMALL_NUMBER))
						{
							bTraceShadowRays = false;
						}
					}

					if (bTraceShadowRays)
					{
						// Approximate the integral over the light's surface to calculate incident direct radiance
						// As AverageVisibility * AverageIncidentRadiance
						//@todo - switch to the physically correct formulation which will allow us to handle area lights correctly,
						// Especially area lights with spatially varying emission
						float ShadowFactor = 0.0f;
						FLinearColor Transmission;
						const TArray<FLightSurfaceSample>& LightSurfaceSamples = Light->GetCachedSurfaceSamples(0, false);
						FLinearColor UnnormalizedTransmission;

						const FVector2f ShadowValue = CalculatePointAreaShadowing(
							TextureMapping, 
							CurrentVertex, 
							TexelToVertex.ElementIndex,
							TexelToVertex.TexelRadius,
							Light, 
							MappingContext,
							SampleGenerator,
							UnnormalizedTransmission,
							LightSurfaceSamples,
							bDebugThisTexel && GeneralSettings.ViewSingleBounceNumber == 0);

						if (ShadowValue.X > 0.0f)
						{
							if (ShadowValue.X < ShadowValue.Y)
							{
								// Trace more shadow rays if we are in the penumbra
								const TArray<FLightSurfaceSample>& PenumbraLightSurfaceSamples = Light->GetCachedSurfaceSamples(0, true);
								FLinearColor UnnormalizedPenumbraTransmission;

								const FVector2f ShadowValuePenumbra = CalculatePointAreaShadowing(
									TextureMapping, 
									CurrentVertex, 
									TexelToVertex.ElementIndex,
									TexelToVertex.TexelRadius,
									Light, 
									MappingContext,
									SampleGenerator, 
									UnnormalizedPenumbraTransmission,
									PenumbraLightSurfaceSamples,
									bDebugThisTexel && GeneralSettings.ViewSingleBounceNumber == 0);

								// Linear combination of uniform and penumbra shadow samples
								ShadowFactor = (ShadowValue.X + ShadowValuePenumbra.X) / (ShadowValue.Y + ShadowValuePenumbra.Y);
								// Weight each transmission by the fraction of total unshadowed rays that contributed to it
								Transmission = (UnnormalizedTransmission + UnnormalizedPenumbraTransmission) / (ShadowValue.Y + ShadowValuePenumbra.Y);
							}
							else
							{
								// The texel is completely out of shadow, fully lit, with an explicit shadow factor of 1.0f
								ShadowFactor = 1.0f;
								Transmission = UnnormalizedTransmission / ShadowValue.Y;
							}
						}
						else
						{
							Transmission = FLinearColor::Black;
							// The texel is completely in shadow, with an implicit shadow factor of 0.0f
						}

						// Cache off the computed values that we'll use later
						checkSlow(TexelToVertex.TotalSampleWeight > 0.0f);
						TransmissionCache[(Y * TextureMapping->CachedSizeX) + X] = Transmission;
						LightIntensityCache[(Y * TextureMapping->CachedSizeX) + X] = LightIntensity;
						// Greyscale transmission for shadowmaps
						UnfilteredShadowFactorData(X, Y).Visibility = ShadowFactor;// * FLinearColorUtils::LinearRGBToXYZ(Transmission).G;
						// We have valid shadow factor values, enable the filter pass
						bShadowFactorFilterPassEnabled = true;
					}
				}
			}
		}
	}

	// Optional shadow factor filter pass
	if (bShadowFactorFilterPassEnabled && Scene.ShadowSettings.bFilterShadowFactor)
	{
		// Filter in texture space across nearest neighbors
		const float ThresholdForFilteringPenumbra = Scene.ShadowSettings.ShadowFactorGradientTolerance;
		const int32 KernelSizeX = 3; // Expected to be odd
		const int32 KernelSizeY = 3; // Expected to be odd
		const float FilterKernel3x3[KernelSizeX * KernelSizeY] = {
			.5f * 0.150f, .5f * 0.332f, .5f * 0.150f,
			.5f * 0.332f, .5f * 1.000f, .5f * 0.332f,
			.5f * 0.150f, .5f * 0.332f, .5f * 0.150f,
		};
		for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
		{
			for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
			{
				bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& Y == Scene.DebugInput.LocalY
					&& X == Scene.DebugInput.LocalX)
				{
					bDebugThisTexel = true;
				}
#endif
				// If this texel is valid, look for sharp gradients in nearby texels
				if (UnfilteredShadowFactorData(X, Y).bIsMapped)
				{
					float UnfilteredValue = UnfilteredShadowFactorData(X, Y).Visibility;
					const bool bIntersectingSurface = TexelToVertexMap(X, Y).bIntersectingSurface;
					const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
					const bool bLightIsInFrontOfTriangle = !Light->BehindSurface(TexelToVertex.WorldPosition, TexelToVertex.WorldTangentZ);

					float FilteredValueNumerator = 0.0f;
					float FilteredValueDenominator = 0.0f;
					float CenterValueWeight = 1.0f;
					
					if (ShadowMapData)
					{
						// Lower the self weight on backfaces
						// We want to spread frontface values onto backfaces for shadowmaps where the normal falloff will happen per-pixel
						CenterValueWeight = bLightIsInFrontOfTriangle ? 1.0f : .1f;
					}

					// Compare (up to) the full grid of adjacent texels
					int32 X1, Y1;
					int32 FilterStepX = ((KernelSizeX - 1) / 2);
					int32 FilterStepY = ((KernelSizeY - 1) / 2);

					for (int32 KernelIndexY = -FilterStepY; KernelIndexY <= FilterStepY; KernelIndexY++)
					{
						// If this row is out of bounds, skip it
						Y1 = Y + KernelIndexY;
						if ((Y1 < 0) ||
							(Y1 > (TextureMapping->CachedSizeY - 1)))
						{
							continue;
						}

						for (int32 KernelIndexX = -FilterStepX; KernelIndexX <= FilterStepX; KernelIndexX++)
						{
							// If this row is out of bounds, skip it
							X1 = X + KernelIndexX;
							if ((X1 < 0) ||
								(X1 > (TextureMapping->CachedSizeX - 1)))
							{
								continue;
							}

							// Only include the texel if it's not completely in shadow
							if (UnfilteredShadowFactorData(X1, Y1).bIsMapped
								&& !(X1 == X && Y1 == Y)
								// Don't filter across intersecting surface boundaries
								&& (bIntersectingSurface == TexelToVertexMap(X1, Y1).bIntersectingSurface))
							{
								float ComparisonValue = UnfilteredShadowFactorData(X1, Y1).Visibility;
								float DifferenceValue = FMath::Abs(UnfilteredValue - ComparisonValue);
								const FTexelToVertex& NeighborTexelToVertex = TexelToVertexMap(X1, Y1);
								const bool bNeighborLightIsInFrontOfTriangle = !Light->BehindSurface(NeighborTexelToVertex.WorldPosition, NeighborTexelToVertex.WorldTangentZ);

								if (DifferenceValue > ThresholdForFilteringPenumbra 
									// If we are filtering shadow factors for a shadowmap, only gather shadow values from frontfaces
									&& (!ShadowMapData || bNeighborLightIsInFrontOfTriangle))
								{
									int32 FilterKernelIndex = ((KernelIndexY + FilterStepY) * KernelSizeX) + (KernelIndexX + FilterStepX);
									float FilterKernelValue = FilterKernel3x3[FilterKernelIndex];

									FilteredValueNumerator += (ComparisonValue * FilterKernelValue);
									FilteredValueDenominator += FilterKernelValue;
								}
							}
						}
					}

					float FinalShadowFactorValue;
					if (FilteredValueDenominator > 0.0f)
					{
						FinalShadowFactorValue = (FilteredValueNumerator + UnfilteredValue * CenterValueWeight) / (FilteredValueDenominator + CenterValueWeight);
					}
					else
					{
						FinalShadowFactorValue = UnfilteredValue;
					}

					FilteredShadowFactorData(X, Y).Visibility = FinalShadowFactorValue;
					FilteredShadowFactorData(X, Y).bIsMapped = true;
				}
			}
		}
	}

	int32 NumUnoccludedTexels = 0;
	int32 NumMappedTexels = 0;
	if (bShadowFactorFilterPassEnabled)
	{
		LIGHTINGSTAT(FScopedRDTSCTimer AreaLightingTimer(MappingContext.Stats.AreaLightingThreadTime));
		for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
		{
			for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
			{
				bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& Y == Scene.DebugInput.LocalY
					&& X == Scene.DebugInput.LocalX)
				{
					bDebugThisTexel = true;
				}
#endif
				float ShadowFactor;
				bool bIsMapped;
				if (Scene.ShadowSettings.bFilterShadowFactor)
				{
					bIsMapped = FilteredShadowFactorData(X, Y).bIsMapped;
					ShadowFactor = FilteredShadowFactorData(X, Y).Visibility;
				}
				else
				{
					bIsMapped = UnfilteredShadowFactorData(X, Y).bIsMapped;
					ShadowFactor = UnfilteredShadowFactorData(X, Y).Visibility;
				}

				NumMappedTexels += bIsMapped ? 1 : 0;
				if (bIsMapped && ShadowFactor > 0.0f)
				{
					NumUnoccludedTexels++;
					// Get any cached values
					const float AdjustedShadowFactor = FMath::Pow(ShadowFactor, Light->ShadowExponent);

					if (GeneralSettings.ViewSingleBounceNumber < 1)
					{
						if (ShadowMapData)
						{
							FShadowSample& CurrentShadowSample = (*ShadowMapData)(X,Y);
							CurrentShadowSample.Visibility = AdjustedShadowFactor;
							if ( CurrentShadowSample.Visibility > 0.0001f )
							{
								bIsCompletelyOccluded = false;
							}
						}
						else
						{
							// Calculate any derived values
							const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
							const FStaticLightingVertex CurrentVertex = TexelToVertex.GetVertex();
							const FLinearColor LightIntensity = LightIntensityCache[(Y * TextureMapping->CachedSizeX) + X];
							const FLinearColor Transmission = TransmissionCache[(Y * TextureMapping->CachedSizeX) + X];
							const FGatheredLightSample DirectLighting = CalculatePointLighting(TextureMapping, CurrentVertex, TexelToVertex.ElementIndex, Light, LightIntensity, Transmission);

							FGatheredLightMapSample& CurrentLightSample = LightMapData(X,Y);
							CurrentLightSample.AddWeighted(DirectLighting, AdjustedShadowFactor);
						}
					}
				}
			}
		}
	}

	if (ShadowMapData && (bIsCompletelyOccluded || NumUnoccludedTexels < NumMappedTexels * ShadowSettings.MinUnoccludedFraction))
	{
		delete ShadowMapData;
		ShadowMapData = NULL;
	}
}

/** 
 * Sample data for the low and high resolution source data that the distance field for shadowing is generated off of.
 * The defaults for all members are implicitly 0 since any uses of this class zero the memory after allocating it.
 */
class FVisibilitySample
{
protected:
	/** World space position in XYZ, Distance to the nearest occluder in W, only valid if !bVisible. */
	FVector4f PositionAndOccluderDistance;
	/** World space normal */
	float NormalX, NormalY, NormalZ;
	/** Whether this sample is visible to the light. */
	uint32 bVisible : 1;
	/** True if this sample maps to a valid point on a surface. */
	uint32 bIsMapped : 1;
	/** Whether this sample needs high resolution sampling. */
	uint32 bNeedsHighResSampling : 1;

public:
	inline FVector4f GetPosition() const { return FVector4f(PositionAndOccluderDistance, 0.0f); }
	inline float GetOccluderDistance() const { return PositionAndOccluderDistance.W; }
	inline FVector4f GetNormal() const { return FVector4f(NormalX, NormalY, NormalZ); }
	inline bool IsVisible() const { return bVisible; }
	inline bool IsMapped() const { return bIsMapped; }
	inline bool NeedsHighResSampling() const { return bNeedsHighResSampling; }

	inline void SetPosition(const FVector4f& InPosition)  
	{  
		PositionAndOccluderDistance.X = InPosition.X;
		PositionAndOccluderDistance.Y = InPosition.Y;
		PositionAndOccluderDistance.Z = InPosition.Z;
	}
	inline void SetOccluderDistance(float InOccluderDistance)  
	{  
		PositionAndOccluderDistance.W = InOccluderDistance;
	}
	inline void SetNormal(const FVector4f& InNormal)  
	{  
		NormalX = InNormal.X;
		NormalY = InNormal.Y;
		NormalZ = InNormal.Z;
	}
	inline void SetVisible(bool bInVisible) { bVisible = bInVisible; }
	inline void SetMapped(bool bInMapped) { bIsMapped = bInMapped; }
};

/** 
 * Sample data for the low resolution visibility data that is populated initially for distance field generation.
 * Each low resolution sample contains a set of high resolution samples if the low resolution sample is next to a shadow transition. 
 */
class FLowResolutionVisibilitySample : public FVisibilitySample
{
public:
	uint16 ElementIndex;

	/** High resolution samples corresponding to this low resolution sample, only allocated if bNeedsHighResSampling == true. */
	TArray<FVisibilitySample> HighResolutionSamples;

	inline void SetNeedsHighResSampling(bool bInNeedsHighResSampling, int32 UpsampleFactor) 
	{ 
		if (bInNeedsHighResSampling)
		{
			HighResolutionSamples.Empty(UpsampleFactor * UpsampleFactor);
			HighResolutionSamples.AddZeroed(UpsampleFactor * UpsampleFactor);
		}
		bNeedsHighResSampling = bInNeedsHighResSampling; 
	}
};

/** 2D array of FLowResolutionVisibilitySample's */
class FTexelVisibilityData2D : public FShadowMapData2DData
{
public:
	FTexelVisibilityData2D(uint32 InSizeX,uint32 InSizeY) :
		FShadowMapData2DData(InSizeX, InSizeY)
	{
		Data.Empty(InSizeX * InSizeY);
		Data.AddZeroed(InSizeX * InSizeY);
	}

	// Accessors.
	const FLowResolutionVisibilitySample& operator()(uint32 X,uint32 Y) const { return Data[SizeX * Y + X]; }
	FLowResolutionVisibilitySample& operator()(uint32 X,uint32 Y) { return Data[SizeX * Y + X]; }
	uint32 GetSizeX() const { return SizeX; }
	uint32 GetSizeY() const { return SizeY; }
	void Empty() { Data.Empty(); }
	SIZE_T GetAllocatedSize() const { return Data.GetAllocatedSize(); }

private:
	TArray<FLowResolutionVisibilitySample> Data;
};

class FDistanceFieldRasterPolicy
{
public:

	typedef FStaticLightingInterpolant InterpolantType;

	/** Initialization constructor. */
	FDistanceFieldRasterPolicy(FTexelVisibilityData2D& InLowResolutionVisibilityData, int32 InUpsampleFactor, int32 InSizeX, int32 InSizeY) :
		LowResolutionVisibilityData(InLowResolutionVisibilityData),
		UpsampleFactor(InUpsampleFactor),
		SizeX(InSizeX),
		SizeY(InSizeY)
	{}

protected:

	// FTriangleRasterizer policy interface.

	int32 GetMinX() const { return 0; }
	int32 GetMaxX() const { return SizeX - 1; }
	int32 GetMinY() const { return 0; }
	int32 GetMaxY() const { return SizeY - 1; }

	void ProcessPixel(int32 X,int32 Y,const InterpolantType& Interpolant,bool BackFacing);

private:

	FTexelVisibilityData2D& LowResolutionVisibilityData;
	const int32 UpsampleFactor;
	const int32 SizeX;
	const int32 SizeY;
};

void FDistanceFieldRasterPolicy::ProcessPixel(int32 X,int32 Y,const InterpolantType& Interpolant,bool BackFacing)
{
	FLowResolutionVisibilitySample& LowResSample = LowResolutionVisibilityData(X / UpsampleFactor, Y / UpsampleFactor);
	LowResSample.ElementIndex = Interpolant.ElementIndex;
	if (LowResSample.NeedsHighResSampling())
	{
		FVisibilitySample& Sample = LowResSample.HighResolutionSamples[Y % UpsampleFactor * UpsampleFactor + X % UpsampleFactor];
		Sample.SetPosition(Interpolant.Vertex.WorldPosition);
		Sample.SetNormal(Interpolant.Vertex.WorldTangentZ);
		Sample.SetMapped(true);
	}
}

/** 
 * Calculate signed distance field shadowing from a single light,  
 * Based on the paper "Improved Alpha-Tested Magnification for Vector Textures and Special Effects" by Valve.
 */
void FStaticLightingSystem::CalculateDirectSignedDistanceFieldLightingTextureMappingTextureSpace(
	FStaticLightingTextureMapping* TextureMapping, 
	FStaticLightingMappingContext& MappingContext,
	FGatheredLightMapData2D& LightMapData, 
	TMap<const FLight*, FSignedDistanceFieldShadowMapData2D*>& ShadowMaps,
	const FTexelToVertexMap& TexelToVertexMap, 
	const FTexelToCornersMap& TexelToCornersMap,
	bool bDebugThisMapping,
	const FLight* Light) const
{
	LIGHTINGSTAT(FManualRDTSCTimer FirstPassSourceTimer(MappingContext.Stats.SignedDistanceFieldSourceFirstPassThreadTime));
	TArray<FStaticLightingInterpolant> MeshVertices;
	MeshVertices.Empty(TextureMapping->Mesh->NumTriangles * 3);
	MeshVertices.AddZeroed(TextureMapping->Mesh->NumTriangles * 3);
	float AverageTexelDensity = 0.0f;
	for (int32 TriangleIndex = 0; TriangleIndex < TextureMapping->Mesh->NumTriangles; TriangleIndex++)
	{
		// Query the mesh for the triangle's vertices.
		int32 Element;
		TextureMapping->Mesh->GetTriangle(TriangleIndex, MeshVertices[TriangleIndex * 3].Vertex, MeshVertices[TriangleIndex * 3 + 1].Vertex, MeshVertices[TriangleIndex * 3 + 2].Vertex, Element);
		MeshVertices[TriangleIndex * 3].ElementIndex = MeshVertices[TriangleIndex * 3 + 1].ElementIndex = MeshVertices[TriangleIndex * 3 + 2].ElementIndex = Element;

		const FVector4f TriangleNormal = (MeshVertices[TriangleIndex * 3 + 2].Vertex.WorldPosition - MeshVertices[TriangleIndex * 3].Vertex.WorldPosition) ^ (MeshVertices[TriangleIndex * 3 + 1].Vertex.WorldPosition - MeshVertices[TriangleIndex].Vertex.WorldPosition);
		const float TriangleArea = 0.5f * TriangleNormal.Size3();

		if (TriangleArea > DELTA)
		{
			// Triangle vertices in lightmap UV space, scaled by the lightmap resolution
			const FVector2f Vertex0 = MeshVertices[TriangleIndex * 3 + 0].Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);
			const FVector2f Vertex1 = MeshVertices[TriangleIndex * 3 + 1].Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);
			const FVector2f Vertex2 = MeshVertices[TriangleIndex * 3 + 2].Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);

			// Area in lightmap space, or the number of lightmap texels covered by this triangle
			const float LightmapTriangleArea = FMath::Abs(
				Vertex0.X * (Vertex1.Y - Vertex2.Y)
				+ Vertex1.X * (Vertex2.Y - Vertex0.Y)
				+ Vertex2.X * (Vertex0.Y - Vertex1.Y));

			// Accumulate the texel density
			AverageTexelDensity += LightmapTriangleArea / TriangleArea;
		}
	}

	int32 UpsampleFactor = 1;
	if (AverageTexelDensity > DELTA)
	{
		// Normalize the average
		AverageTexelDensity /= TextureMapping->Mesh->NumTriangles;
		// Calculate the length of one side of a right isosceles triangle with texel density equal to the mesh's average texel density
		const float RightTriangleSide = FMath::Sqrt(2.0f * AverageTexelDensity);
		// Choose an upsample factor based on the average texels/world space ratio
		// The result is that small, high resolution meshes will not upsample as much, since they don't need it, 
		// But large, low resolution meshes will upsample a lot.
		const int32 TargetUpsampleFactor = FMath::TruncToInt(ShadowSettings.ApproximateHighResTexelsPerMaxTransitionDistance / (RightTriangleSide * ShadowSettings.MaxTransitionDistanceWorldSpace));
		// Round up to the nearest odd factor, so each destination texel has a high resolution source texel at its center
		// Clamp the upscale factor to be less than 13, since the quality improvements of upsampling higher than that are negligible.
		UpsampleFactor = FMath::Clamp(TargetUpsampleFactor - TargetUpsampleFactor % 2 + 1, ShadowSettings.MinDistanceFieldUpsampleFactor, 13);
	}
	MappingContext.Stats.AccumulatedSignedDistanceFieldUpsampleFactors += UpsampleFactor;
	MappingContext.Stats.NumSignedDistanceFieldCalculations++;

	bool bIsCompletelyOccluded = true;
	int32 NumUnoccludedTexels = 0;
	int32 NumMappedTexels = 0;
	// Calculate visibility at the resolution of the final distance field in a first pass
	FTexelVisibilityData2D LowResolutionVisibilityData(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);
	for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
	{
		for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == Scene.DebugInput.LocalY
				&& X == Scene.DebugInput.LocalX)
			{
				bDebugThisTexel = true;
			}
#endif
			const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
			if (TexelToVertex.TotalSampleWeight > 0.0f)
			{
				NumMappedTexels++;
				// Note: not checking for backfacing normals because some of the high resolution samples corresponding to this texel may be frontfacing
				if (Light->AffectsBounds(FBoxSphereBounds3f(TexelToVertex.WorldPosition, FVector4f(0,0,0),0)))
				{
					FLowResolutionVisibilitySample& CurrentSample = LowResolutionVisibilityData(X, Y);
					CurrentSample.SetPosition(TexelToVertex.WorldPosition);
					CurrentSample.SetNormal(TexelToVertex.WorldTangentZ);
					// Only mark the texel as mapped if we are inside the light's influence
					// This is important because stationary lights are assigned shadowmap channels based on overlap,
					// And multiple shadowmaps on the same object may be merged together, but only if each one marks the area that it has valid data
					CurrentSample.SetMapped(true);

					const FVector4f LightPosition = Light->LightCenterPosition(TexelToVertex.WorldPosition, TexelToVertex.WorldTangentZ);
					const FVector4f LightVector = (LightPosition - TexelToVertex.WorldPosition).GetSafeNormal();

					FVector4f NormalForOffset = CurrentSample.GetNormal();
					// Flip the normal used for offsetting the start of the ray for two sided materials if a flipped normal would be closer to the light.
					// This prevents incorrect shadowing where using the frontface normal would cause the ray to start inside a nearby object.
					const bool bIsTwoSided = TextureMapping->Mesh->IsTwoSided(CurrentSample.ElementIndex);
					if (bIsTwoSided && Dot3(-NormalForOffset, LightVector) > Dot3(NormalForOffset, LightVector))
					{
						NormalForOffset = -NormalForOffset;
					}

					const FLightRay LightRay(
						// Offset the start of the ray by some fraction along the direction of the ray and some fraction along the vertex normal.
						TexelToVertex.WorldPosition 
						+ LightVector * SceneConstants.VisibilityRayOffsetDistance 
						+ NormalForOffset * SceneConstants.VisibilityNormalOffsetDistance,
						LightPosition,
						TextureMapping,
						Light
						);

					FLightRayIntersection Intersection;
					MappingContext.Stats.NumSignedDistanceFieldAdaptiveSourceRaysFirstPass++;
					// Could trace a boolean visibility ray, no other information is needed,
					// However FStaticLightingAggregateMesh::IntersectLightRay currently does not handle masked materials correctly with boolean visibility rays.
					AggregateMesh->IntersectLightRay(LightRay, true, false, true, MappingContext.RayCache, Intersection);
					if (!Intersection.bIntersects)
					{
						NumUnoccludedTexels++;
						bIsCompletelyOccluded = false;
						CurrentSample.SetVisible(true);
					}

#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
					if (bDebugThisTexel && GeneralSettings.ViewSingleBounceNumber == 0)
					{
						FDebugStaticLightingRay DebugRay(LightRay.Start, LightRay.End, Intersection.bIntersects);
						if (Intersection.bIntersects)
						{
							DebugRay.End = Intersection.IntersectionVertex.WorldPosition;
						}
						MappingContext.DebugOutput->ShadowRays.Add(DebugRay);
					}
#endif
				}
			}
		}
	}
	FirstPassSourceTimer.Stop();

	if (!bIsCompletelyOccluded && NumUnoccludedTexels > NumMappedTexels * ShadowSettings.MinUnoccludedFraction)
	{
		LIGHTINGSTAT(FManualRDTSCTimer SecondPassSourceTimer(MappingContext.Stats.SignedDistanceFieldSourceSecondPassThreadTime));
		check(UpsampleFactor % 2 == 1 && UpsampleFactor >= 1);
		const int32 HighResolutionSignalSizeX = TextureMapping->CachedSizeX * UpsampleFactor;
		const int32 HighResolutionSignalSizeY = TextureMapping->CachedSizeY * UpsampleFactor;
		// Allocate the final distance field shadow map on the heap, since it will be passed out of this function
		FSignedDistanceFieldShadowMapData2D* ShadowMapData = new FSignedDistanceFieldShadowMapData2D(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);

		// Neighbor texel coordinates - the order in which these are stored matters later
		const FIntPoint Neighbors[] = 
		{
			FIntPoint(0, 1),
			FIntPoint(0, -1),
			FIntPoint(1, 0),
			FIntPoint(-1, 0)
		};

		// Offsets to the high resolution samples corresponding to the corners of a low resolution sample
		const FIntPoint Corners[] = 
		{
			FIntPoint(0, 0),
			FIntPoint(0, UpsampleFactor - 1),
			FIntPoint(UpsampleFactor - 1, 0),
			FIntPoint(UpsampleFactor - 1, UpsampleFactor - 1)
		};

		// Traverse the visibility data collected at the resolution of the final distance field, detecting where additional sampling is required.
		for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
		{
			for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
			{
				bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& Y == Scene.DebugInput.LocalY
					&& X == Scene.DebugInput.LocalX)
				{
					bDebugThisTexel = true;
				}
#endif
				FLowResolutionVisibilitySample& CurrentSample = LowResolutionVisibilityData(X, Y);
				if (CurrentSample.IsMapped())
				{
					FSignedDistanceFieldShadowSample& FinalShadowSample = (*ShadowMapData)(X, Y);
					FinalShadowSample.bIsMapped = true;
					if (CurrentSample.IsVisible())
					{
						// Initialize the final distance field data, since it will only be written to after this if it gets scattered to during the search.
						FinalShadowSample.Distance = 1.0f;
						FinalShadowSample.PenumbraSize = 1.0f;
					}

					// Search for a neighbor with different visibility
					bool bNeighborsDifferent = false;
					for (int32 i = 0 ; i < UE_ARRAY_COUNT(Neighbors); i++)
					{
						if (X + Neighbors[i].X > 0
							&& X + Neighbors[i].X < TextureMapping->CachedSizeX
							&& Y + Neighbors[i].Y > 0
							&& Y + Neighbors[i].Y < TextureMapping->CachedSizeY)
						{
							const FLowResolutionVisibilitySample& NeighborSample = LowResolutionVisibilityData(X + Neighbors[i].X, Y + Neighbors[i].Y);
							if (CurrentSample.IsVisible() != NeighborSample.IsVisible() && NeighborSample.IsMapped())
							{
								bNeighborsDifferent = true;
								break;
							}
						}
					}

					// Mark the low resolution sample as needing high resolution sampling, since it is next to a shadow transition
					if (bNeighborsDifferent)
					{
						CurrentSample.SetNeedsHighResSampling(bNeighborsDifferent, UpsampleFactor);
					}
				}
			}
		}

		FDistanceFieldRasterPolicy RasterPolicy(LowResolutionVisibilityData, UpsampleFactor, HighResolutionSignalSizeX, HighResolutionSignalSizeY);
		FTriangleRasterizer<FDistanceFieldRasterPolicy> DistanceFieldRasterizer(RasterPolicy);
		// Rasterize the mesh at the upsampled source data resolution
		for (int32 TriangleIndex = 0; TriangleIndex < MeshVertices.Num() / 3; TriangleIndex++)
		{
			const FStaticLightingInterpolant& V0 = MeshVertices[TriangleIndex * 3];
			const FStaticLightingInterpolant& V1 = MeshVertices[TriangleIndex * 3 + 1];
			const FStaticLightingInterpolant& V2 = MeshVertices[TriangleIndex * 3 + 2];

			DistanceFieldRasterizer.DrawTriangle(
				V0,
				V1,
				V2,
				V0.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(HighResolutionSignalSizeX, HighResolutionSignalSizeY) + FVector2f(-0.5f,-0.5f),
				V1.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(HighResolutionSignalSizeX, HighResolutionSignalSizeY) + FVector2f(-0.5f,-0.5f),
				V2.Vertex.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex] * FVector2f(HighResolutionSignalSizeX, HighResolutionSignalSizeY) + FVector2f(-0.5f,-0.5f),
				false
				);
		}
		MeshVertices.Empty();

		// Check for edge cases where the low resolution sample is mapped, but none of the high resolution samples got mapped.
		for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
		{
			for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
			{
				bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& Y == Scene.DebugInput.LocalY
					&& X == Scene.DebugInput.LocalX)
				{
					bDebugThisTexel = true;
				}
#endif
				FLowResolutionVisibilitySample& CurrentSample = LowResolutionVisibilityData(X, Y);
				if (CurrentSample.IsMapped() && CurrentSample.NeedsHighResSampling())
				{
					bool bAnyHighResSamplesMapped = false;
					// Iterate over all the upsampled source data texels corresponding to this texel
					for (int32 HighResY = 0; HighResY < UpsampleFactor; HighResY++)
					{
						for (int32 HighResX = 0; HighResX < UpsampleFactor; HighResX++)
						{
							FVisibilitySample& CurrentHighResSample = CurrentSample.HighResolutionSamples[HighResY * UpsampleFactor + HighResX];
							if (CurrentHighResSample.IsMapped())
							{
								bAnyHighResSamplesMapped = true;
							}
						}
					}

					// If none of the high res samples are mapped, but the low resolution sample is mapped, 
					// Propagate the low resolution corner information to the corresponding high resolution samples.
					// This handles texels along UV seams where only the corner of the texel is mapped.
					if (!bAnyHighResSamplesMapped)
					{
						const FTexelToCorners& TexelToCorners = TexelToCornersMap(X, Y);
						for (int32 CornerIndex = 0; CornerIndex < UE_ARRAY_COUNT(Corners); CornerIndex++)
						{						
							if (TexelToCorners.bValid[CornerIndex])
							{
								FVisibilitySample& CornerHighResSample = CurrentSample.HighResolutionSamples[Corners[CornerIndex].Y * UpsampleFactor + Corners[CornerIndex].X];
								CornerHighResSample.SetMapped(true);
								CornerHighResSample.SetPosition(TexelToCorners.Corners[CornerIndex].WorldPosition);
								CornerHighResSample.SetNormal(TexelToCorners.WorldTangentZ);
							}
						}
					}
				}
			}
		}

		for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
		{
			for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
			{
				bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& Y == Scene.DebugInput.LocalY
					&& X == Scene.DebugInput.LocalX)
				{
					bDebugThisTexel = true;
				}
#endif
				FLowResolutionVisibilitySample& CurrentSample = LowResolutionVisibilityData(X, Y);
				// Do high resolution sampling if necessary
				if (CurrentSample.IsMapped() && CurrentSample.NeedsHighResSampling())
				{
					const bool bIsTwoSided = TextureMapping->Mesh->IsTwoSided(CurrentSample.ElementIndex);
					for (int32 HighResY = 0; HighResY < UpsampleFactor; HighResY++)
					{
						for (int32 HighResX = 0; HighResX < UpsampleFactor; HighResX++)
						{
							FVisibilitySample& HighResSample = CurrentSample.HighResolutionSamples[HighResY * UpsampleFactor + HighResX];
							const bool bLightIsInFrontOfTriangle = !IsLightBehindSurface(HighResSample.GetPosition(),HighResSample.GetNormal(),Light);

							if ((bLightIsInFrontOfTriangle || bIsTwoSided) 
								&& Light->AffectsBounds(FBoxSphereBounds3f(HighResSample.GetPosition(), FVector4f(0,0,0),0)))
							{
								const FVector4f LightPosition = Light->LightCenterPosition(HighResSample.GetPosition(), HighResSample.GetNormal());
								const FVector4f LightVector = (LightPosition - HighResSample.GetPosition()).GetSafeNormal();

								FVector4f NormalForOffset = HighResSample.GetNormal();
								// Flip the normal used for offsetting the start of the ray for two sided materials if a flipped normal would be closer to the light.
								// This prevents incorrect shadowing where using the frontface normal would cause the ray to start inside a nearby object.
								if (bIsTwoSided && Dot3(-NormalForOffset, LightVector) > Dot3(NormalForOffset, LightVector))
								{
									NormalForOffset = -NormalForOffset;
								}
								const FLightRay LightRay(
									// Offset the start of the ray by some fraction along the direction of the ray and some fraction along the vertex normal.
									HighResSample.GetPosition() 
									+ LightVector * SceneConstants.VisibilityRayOffsetDistance 
									+ NormalForOffset * SceneConstants.VisibilityNormalOffsetDistance,
									LightPosition,
									TextureMapping, 
									Light
									);

								FLightRayIntersection Intersection;
								MappingContext.Stats.NumSignedDistanceFieldAdaptiveSourceRaysSecondPass++;
								// Have to calculate the closest intersection so we know the distance to the nearest occluder
								//@todo - for the occluder distance to be correct, the ray should actually go from the light to the receiver
								AggregateMesh->IntersectLightRay(LightRay, true, false, true, MappingContext.RayCache, Intersection);
								if (Intersection.bIntersects)
								{
									HighResSample.SetOccluderDistance((LightRay.Start - Intersection.IntersectionVertex.WorldPosition).Size3());
								}
								else
								{
									HighResSample.SetVisible(true);
								}
							}
						}
					}
				}
			}
		}
		SecondPassSourceTimer.Stop();

		int32 NumScattersToSelectedTexel = 0;
		LIGHTINGSTAT(FScopedRDTSCTimer SearchTimer(MappingContext.Stats.SignedDistanceFieldSearchThreadTime));
		// Traverse the high resolution source data by going over low res samples that that need high resolution sampling, and at each texel that is next to a transition,
		// Scatter the distance to that texel onto all low resolution distance field texels within a certain world space distance from the transition texel.
		// The end result is that each low resolution texel in the distance field has the world space distance to the nearest transition in the high resolution visibility data.
		// Using a scatter from the high res transition texels is significantly faster than a brute force gather from the low resolution distance field texels, 
		// Because only a small set of the high resolution texels are next to the shadow transition.
		for (int32 LowResY = 0; LowResY < TextureMapping->CachedSizeY; LowResY++)
		{
			for (int32 LowResX = 0; LowResX < TextureMapping->CachedSizeX; LowResX++)
			{
				bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& LowResY == Scene.DebugInput.LocalY
					&& LowResX == Scene.DebugInput.LocalX)
				{
					bDebugThisTexel = true;
				}
#endif
				FLowResolutionVisibilitySample& CurrentLowResSample = LowResolutionVisibilityData(LowResX, LowResY);
				if (CurrentLowResSample.IsMapped() && CurrentLowResSample.NeedsHighResSampling())
				{
					for (int32 HighResY = 0; HighResY < UpsampleFactor; HighResY++)
					{
						for (int32 HighResX = 0; HighResX < UpsampleFactor; HighResX++)
						{
							FVisibilitySample& HighResSample = CurrentLowResSample.HighResolutionSamples[HighResY * UpsampleFactor + HighResX];
							// Only texels that needed high resolution sampling can be next to the shadow transition
							// Only operate on shadowed texels, since they know the distance to the nearest occluder, which is necessary for calculating penumbra size
							// As a result, the reconstructed shadow transition will be slightly offset
							if (HighResSample.IsMapped() && !HighResSample.IsVisible())
							{
								// Detect texels next to the shadow transition
								bool bNeighborsDifferent = false;
								for (int32 i = 0 ; i < UE_ARRAY_COUNT(Neighbors); i++)
								{
									// Calculate the high resolution indices, which may go into neighboring low resolution samples
									const int32 HighResNeighborX = LowResX * UpsampleFactor + HighResX + Neighbors[i].X;
									const int32 HighResNeighborY = LowResY * UpsampleFactor + HighResY + Neighbors[i].Y;
									const int32 LowResNeighborX = HighResNeighborX / UpsampleFactor;
									const int32 LowResNeighborY = HighResNeighborY / UpsampleFactor;
									if (LowResNeighborX > 0
										&& LowResNeighborX < TextureMapping->CachedSizeX
										&& LowResNeighborY > 0
										&& LowResNeighborY < TextureMapping->CachedSizeY)
									{
										const FLowResolutionVisibilitySample& LowResNeighborSample = LowResolutionVisibilityData(LowResNeighborX, LowResNeighborY);
										// If the low res neighbor sample has high resolution samples, check the neighboring high resolution sample's visibility
										if (LowResNeighborSample.NeedsHighResSampling())
										{
											const FVisibilitySample& HighResNeighborSample = LowResNeighborSample.HighResolutionSamples[(HighResNeighborY % UpsampleFactor) * UpsampleFactor + HighResNeighborX % UpsampleFactor];
											if (HighResNeighborSample.IsMapped() && HighResNeighborSample.IsVisible())
											{
												bNeighborsDifferent = true;
												break;
											}
										}
										else
										{
											// The low res neighbor sample didn't have high resolution samples, use its visibility
											if (LowResNeighborSample.IsMapped() && LowResNeighborSample.IsVisible())
											{
												bNeighborsDifferent = true;
												break;
											}
										}
									}
								}

								if (bNeighborsDifferent)
								{
									float WorldSpacePerHighResTexelX = FLT_MAX;
									float WorldSpacePerHighResTexelY = FLT_MAX;
									// Determine how far to scatter transition distance by measuring the world space distance between this texel and its neighbors
									for (int32 i = 0 ; i < UE_ARRAY_COUNT(Neighbors); i++)
									{
										if (HighResX + Neighbors[i].X > 0
											&& HighResX + Neighbors[i].X < UpsampleFactor
											&& HighResY + Neighbors[i].Y > 0
											&& HighResY + Neighbors[i].Y < UpsampleFactor)
										{
											const FVisibilitySample& NeighborSample = CurrentLowResSample.HighResolutionSamples[(HighResY + Neighbors[i].Y) * UpsampleFactor + HighResX + Neighbors[i].X];
											if (NeighborSample.IsMapped())
											{
												// Last two neighbor offsets are in X
												if (i >= 2)
												{
													WorldSpacePerHighResTexelX = FMath::Min(WorldSpacePerHighResTexelX, (NeighborSample.GetPosition() - HighResSample.GetPosition()).Size3());
												}
												else
												{
													WorldSpacePerHighResTexelY = FMath::Min(WorldSpacePerHighResTexelY, (NeighborSample.GetPosition() - HighResSample.GetPosition()).Size3());
												}
											}
										}
									}

									if (WorldSpacePerHighResTexelX == FLT_MAX && WorldSpacePerHighResTexelY == FLT_MAX)
									{
										WorldSpacePerHighResTexelX = 1.0f;
										WorldSpacePerHighResTexelY = 1.0f;
									}
									else if (WorldSpacePerHighResTexelX == FLT_MAX)
									{
										WorldSpacePerHighResTexelX = WorldSpacePerHighResTexelY;
									}
									else if (WorldSpacePerHighResTexelY == FLT_MAX)
									{
										WorldSpacePerHighResTexelY = WorldSpacePerHighResTexelX;
									}

									// Scatter to all distance field texels within MaxTransitionDistanceWorldSpace, rounded up.
									// This is an approximation to the actual set of distance field texels that are within MaxTransitionDistanceWorldSpace that tends to work out well.
									// Apply a clamp to avoid a performance cliff with some texels, whose adjacent texel in lightmap space is actually far away in world space
									const int32 NumLowResScatterTexelsY = FMath::Min(FMath::TruncToInt(ShadowSettings.MaxTransitionDistanceWorldSpace / (WorldSpacePerHighResTexelY * UpsampleFactor)) + 1, 100);
									const int32 NumLowResScatterTexelsX = FMath::Min(FMath::TruncToInt(ShadowSettings.MaxTransitionDistanceWorldSpace / (WorldSpacePerHighResTexelX * UpsampleFactor)) + 1, 100);
									MappingContext.Stats.NumSignedDistanceFieldScatters++;
									for (int32 ScatterOffsetY = -NumLowResScatterTexelsY; ScatterOffsetY <= NumLowResScatterTexelsY; ScatterOffsetY++)
									{
										const int32 LowResScatterY = LowResY + ScatterOffsetY;
										if (LowResScatterY < 0 || LowResScatterY >= TextureMapping->CachedSizeY)
										{
											continue;
										}
										for (int32 ScatterOffsetX = -NumLowResScatterTexelsX; ScatterOffsetX <= NumLowResScatterTexelsX; ScatterOffsetX++)
										{
											const int32 LowResScatterX = LowResX + ScatterOffsetX;
											if (LowResScatterX < 0 || LowResScatterX >= TextureMapping->CachedSizeX)
											{
												continue;
											}

											bool bDebugThisScatterTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING 
											// Debug when the selected texel is being scattered to
											// This may get hit any number of times, only the closest transition distance will be kept in the end
											if (bDebugThisMapping
												&& LowResScatterY == Scene.DebugInput.LocalY
												&& LowResScatterX == Scene.DebugInput.LocalX)
											{
												bDebugThisScatterTexel = true;
											}
#endif
											const FLowResolutionVisibilitySample& LowResScatterSample = LowResolutionVisibilityData(LowResScatterX, LowResScatterY);
											// Only scatter transition distance to mapped texels
											if (LowResScatterSample.IsMapped())
											{
												bool CurrentRegion = false;
												FVector4f ScatterPosition;
												FVector4f ScatterNormal;
												bool bFoundScatterPosition = false;
												
												if (LowResScatterSample.NeedsHighResSampling())
												{
													// If the low res scatter sample has high resolution samples, use the center high resolution sample's visibility
													const FVisibilitySample& HighResScatterSample = LowResScatterSample.HighResolutionSamples[(UpsampleFactor / 2) * UpsampleFactor + UpsampleFactor / 2];
													if (HighResScatterSample.IsMapped())
													{
														CurrentRegion = HighResScatterSample.IsVisible();
														ScatterPosition = HighResScatterSample.GetPosition();
														ScatterNormal = HighResScatterSample.GetNormal();
														bFoundScatterPosition = true;
													}
													else
													{
														// If the centered high resolution texel is not mapped, 
														// Search all of the high resolution texels corresponding to the low resolution distance field texel for the closest mapped texel.
														float ClosestMappedSubSampleDistSquared = FLT_MAX;
														for (int32 SubY = 0; SubY < UpsampleFactor; SubY++)
														{
															for (int32 SubX = 0; SubX < UpsampleFactor; SubX++)
															{
																const FVisibilitySample& SubHighResSample = LowResScatterSample.HighResolutionSamples[SubY * UpsampleFactor + SubX];
																const float SubSampleDistanceSquared = FMath::Square(SubX - UpsampleFactor / 2) + FMath::Square(SubY - UpsampleFactor / 2);
																if (SubHighResSample.IsMapped() && SubSampleDistanceSquared < ClosestMappedSubSampleDistSquared)
																{
																	ClosestMappedSubSampleDistSquared = SubSampleDistanceSquared;
																	CurrentRegion = SubHighResSample.IsVisible();
																	ScatterPosition = SubHighResSample.GetPosition();
																	ScatterNormal = SubHighResSample.GetNormal();
																	bFoundScatterPosition = true;
																}
															}
														}
													}
												}

												// No high resolution scatter samples were found, use the position and visibility of the low resolution sample
												if (!bFoundScatterPosition) 
												{
													CurrentRegion = LowResScatterSample.IsVisible();
													ScatterPosition = LowResScatterSample.GetPosition();
													ScatterNormal = LowResScatterSample.GetNormal();
												}

												// World space distance from the distance field texel to the nearest shadow transition
												const float TransitionDistance = (ScatterPosition - HighResSample.GetPosition()).Size3();
												const float NormalizedDistance = FMath::Clamp(TransitionDistance / ShadowSettings.MaxTransitionDistanceWorldSpace, 0.0f, 1.0f);
												FSignedDistanceFieldShadowSample& FinalShadowSample = (*ShadowMapData)(LowResScatterX, LowResScatterY);
												// If LowResScatterSample.IsMapped() is true, the distance field texel must be mapped.
												checkSlow(FinalShadowSample.bIsMapped);
												// Only write to distance field texels whose existing transition distance is further than the transition distance being scattered.
												if (NormalizedDistance * .5f < FMath::Abs(FinalShadowSample.Distance - .5f))
												{
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
													// Debug when the selected texel is being scattered to
													// This may get hit any number of times, only the last hit will get stored in the distance field
													if (bDebugThisScatterTexel)
													{
														NumScattersToSelectedTexel++;
													}
#endif
													// Encode the transition distance so that [.5,0] corresponds to [0,1] for shadowed texels, and [.5,1] corresponds to [0,1] for unshadowed texels.
													// .5 of the encoded distance lies exactly on the shadow transition.
													FinalShadowSample.Distance = CurrentRegion ? (NormalizedDistance) * .5f + .5f : .5f - NormalizedDistance * .5f;
													// Approximate the penumbra size using PenumbraSize = (ReceiverDistanceFromLight - OccluderDistanceFromLight) * LightSize / OccluderDistanceFromLight,
													// Which is from the paper "Percentage-Closer Soft Shadows" by Randima Fernando
													const float ReceiverDistanceFromLight = (Light->LightCenterPosition(ScatterPosition, ScatterNormal) - ScatterPosition).Size3();
													// World space distance from center of penumbra to fully shadowed or fully lit transition
													const float PenumbraSize = HighResSample.GetOccluderDistance() * Light->LightSourceRadius / (ReceiverDistanceFromLight - HighResSample.GetOccluderDistance());
													// Normalize the penumbra size so it is a fraction of MaxTransitionDistanceWorldSpace
													FinalShadowSample.PenumbraSize = FMath::Clamp(PenumbraSize / ShadowSettings.MaxTransitionDistanceWorldSpace, 0.01f, 1.0f);
												}
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}

		ShadowMaps.Add(Light, ShadowMapData);
	}
}

void FStaticLightingSystem::CalculateDirectSignedDistanceFieldLightingTextureMappingLightSpace(
	FStaticLightingTextureMapping* TextureMapping, 
	FStaticLightingMappingContext& MappingContext,
	FGatheredLightMapData2D& LightMapData, 
	TMap<const FLight*, FSignedDistanceFieldShadowMapData2D*>& ShadowMaps,
	const FTexelToVertexMap& TexelToVertexMap, 
	const FTexelToCornersMap& TexelToCornersMap,
	bool bDebugThisMapping,
	const FLight* Light) const
{
	const FBoxSphereBounds3f MeshInfluenceBounds(TextureMapping->Mesh->BoundingBox.ExpandBy(ShadowSettings.MaxTransitionDistanceWorldSpace));

	if (Light->AffectsBounds(MeshInfluenceBounds))
	{
		const FBoxSphereBounds3f SceneBounds = FBoxSphereBounds3f(AggregateMesh->GetBounds());
		const FDirectionalLight* DirectionalLight = Light->GetDirectionalLight();
		const FSpotLight* SpotLight = Light->GetSpotLight();
		const FPointLight* PointLight = Light->GetPointLight();
		check(DirectionalLight || SpotLight || PointLight);

		if (DirectionalLight)
		{
			LIGHTINGSTAT(FManualRDTSCTimer FirstPassSourceTimer(MappingContext.Stats.SignedDistanceFieldSourceFirstPassThreadTime));
			FVector4f XAxis, YAxis;
			DirectionalLight->Direction.FindBestAxisVectors3(XAxis, YAxis);
			// Create a coordinate system for the directional light, with the z axis corresponding to the light's direction
			FMatrix44f WorldToLight = FBasisVectorMatrix44f(XAxis, YAxis, DirectionalLight->Direction, FVector4f(0,0,0));

			const FBox3f LightSpaceImportanceBounds = MeshInfluenceBounds.GetBox().TransformBy(WorldToLight);

			FStaticShadowDepthMap ShadowDepthMap;

			int32 ShadowMapSizeX = FMath::TruncToInt(FMath::Max(LightSpaceImportanceBounds.GetExtent().X * 2.0f * 100.0f / ShadowSettings.MaxTransitionDistanceWorldSpace, 4.0f));
			ShadowMapSizeX = ShadowMapSizeX == appTruncErrorCode ? INT_MAX : ShadowMapSizeX;
			int32 ShadowMapSizeY = FMath::TruncToInt(FMath::Max(LightSpaceImportanceBounds.GetExtent().Y * 2.0f * 100.0f / ShadowSettings.MaxTransitionDistanceWorldSpace, 4.0f));
			ShadowMapSizeY = ShadowMapSizeY == appTruncErrorCode ? INT_MAX : ShadowMapSizeY;

			uint64 ShadowDepthMapMaxSamples = 4194304;

			// Clamp the number of dominant shadow samples generated if necessary while maintaining aspect ratio
			if ((uint64)ShadowMapSizeX * (uint64)ShadowMapSizeY > (uint64)ShadowDepthMapMaxSamples)
			{
				const float AspectRatio = ShadowMapSizeX / (float)ShadowMapSizeY;
				ShadowMapSizeY = FMath::TruncToInt(FMath::Sqrt(ShadowDepthMapMaxSamples / AspectRatio));
				ShadowMapSizeX = ShadowDepthMapMaxSamples / ShadowMapSizeY;
			}

			TArray<float> ShadowMap;

			// Allocate the shadow map
			ShadowMap.Empty(ShadowMapSizeX * ShadowMapSizeY);
			ShadowMap.AddZeroed(ShadowMapSizeX * ShadowMapSizeY);
			const float ShadowMapStart = LightSpaceImportanceBounds.Max.Z - SceneBounds.SphereRadius * 2;
			const FMatrix44f LightToWorld = WorldToLight.InverseFast();

			for (int32 Y = 0; Y < ShadowMapSizeY; Y++)
			{
				const float YFraction = (Y + .5f) / (float)(ShadowMapSizeY - 1);

				for (int32 X = 0; X < ShadowMapSizeX; X++)
				{
					const float XFraction = (X + .5f) / (float)(ShadowMapSizeX - 1);

					const FVector4f LightSpaceEndPosition(
						LightSpaceImportanceBounds.Min.X + XFraction * (LightSpaceImportanceBounds.Max.X - LightSpaceImportanceBounds.Min.X),
						LightSpaceImportanceBounds.Min.Y + YFraction * (LightSpaceImportanceBounds.Max.Y - LightSpaceImportanceBounds.Min.Y),
						LightSpaceImportanceBounds.Max.Z);
					const FVector4f WorldSpaceEndPosition = LightToWorld.TransformPosition(LightSpaceEndPosition);

					const FVector4f LightSpaceStartPosition(LightSpaceEndPosition.X, LightSpaceEndPosition.Y, ShadowMapStart);
					const FVector4f WorldSpaceStartPosition = LightToWorld.TransformPosition(LightSpaceStartPosition);

					const FLightRay LightRay(
						WorldSpaceStartPosition,
						WorldSpaceEndPosition,
						NULL,
						NULL,
						// We are tracing from the light instead of to the light,
						// So flip sidedness so that backface culling matches up with tracing to the light
						LIGHTRAY_FLIP_SIDEDNESS
						);

					FLightRayIntersection Intersection;
					AggregateMesh->IntersectLightRay(LightRay, true, false, true, MappingContext.RayCache, Intersection);

					float MaxSampleDistance = SceneBounds.SphereRadius * 2;

					if (Intersection.bIntersects)
					{
						MaxSampleDistance = (Intersection.IntersectionVertex.WorldPosition - WorldSpaceStartPosition).Size3();
					}

					ShadowMap[Y * ShadowMapSizeX + X] = MaxSampleDistance;
				}
			}

			FirstPassSourceTimer.Stop();

			LIGHTINGSTAT(FScopedRDTSCTimer SearchTimer(MappingContext.Stats.SignedDistanceFieldSearchThreadTime));
			FSignedDistanceFieldShadowMapData2D* ShadowMapData = new FSignedDistanceFieldShadowMapData2D(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);
			const int32 TransitionSearchTexelRadiusX = FMath::TruncToInt(ShadowMapSizeX * ShadowSettings.MaxTransitionDistanceWorldSpace / LightSpaceImportanceBounds.GetSize().X);
			const int32 TransitionSearchTexelRadiusY = FMath::TruncToInt(ShadowMapSizeY * ShadowSettings.MaxTransitionDistanceWorldSpace / LightSpaceImportanceBounds.GetSize().Y);
			const float BoundsCellSizeX = (LightSpaceImportanceBounds.Max.X - LightSpaceImportanceBounds.Min.X) / ShadowMapSizeX;
			const float BoundsCellSizeY = (LightSpaceImportanceBounds.Max.Y - LightSpaceImportanceBounds.Min.Y) / ShadowMapSizeY;
			const float DepthBias = FMath::Max(BoundsCellSizeX, BoundsCellSizeY);

			for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
			{
				for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
				{
					bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
					if (bDebugThisMapping
						&& Y == Scene.DebugInput.LocalY
						&& X == Scene.DebugInput.LocalX)
					{
						bDebugThisTexel = true;
					}
#endif
					const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);

					if (TexelToVertex.TotalSampleWeight > 0.0f)
					{
						const FVector4f LightSpacePosition = WorldToLight.TransformPosition(TexelToVertex.WorldPosition);
						const FVector3f LightSpaceNormal = WorldToLight.TransformVector(TexelToVertex.WorldTangentZ);
						const float SinThetaX = LightSpaceNormal.X;
						const float TanThetaX = SinThetaX / FMath::Sqrt(1 - SinThetaX * SinThetaX);
						const float SinThetaY = LightSpaceNormal.Y;
						const float TanThetaY = SinThetaY / FMath::Sqrt(1 - SinThetaY * SinThetaY);
						const float SurfaceDepth = LightSpacePosition.Z - ShadowMapStart;

						const int32 ShadowMapX = FMath::Clamp(FMath::TruncToInt((LightSpacePosition.X - LightSpaceImportanceBounds.Min.X) / BoundsCellSizeX), 0, ShadowMapSizeX - 1);
						const int32 ShadowMapY = FMath::Clamp(FMath::TruncToInt((LightSpacePosition.Y - LightSpaceImportanceBounds.Min.Y) / BoundsCellSizeY), 0, ShadowMapSizeY - 1);

						const float TexelShadowMapDepth = ShadowMap[ShadowMapY * ShadowMapSizeX + ShadowMapX];
						const float SlopeScaledDepthBias = 4 * FMath::Max(BoundsCellSizeX * FMath::Abs(TanThetaX), BoundsCellSizeY * FMath::Abs(TanThetaY));
						const bool bTexelVisible = TexelShadowMapDepth > SurfaceDepth - SlopeScaledDepthBias - DepthBias;
						float ClosestTransition = ShadowSettings.MaxTransitionDistanceWorldSpace;
						//float ClosestTransitionPenumbraSize = 1;
						float MostShadowingTransition = 1;
						float MostShadowingTransitionDistance = 1;
						float MostShadowingTransitionPenumbraSize = 1;

						for (int32 SearchY = FMath::Max(ShadowMapY - TransitionSearchTexelRadiusY, 0); SearchY < FMath::Min(ShadowMapY + TransitionSearchTexelRadiusY, ShadowMapSizeY); SearchY++)
						{
							for (int32 SearchX = FMath::Max(ShadowMapX - TransitionSearchTexelRadiusX, 0); SearchX < FMath::Min(ShadowMapX + TransitionSearchTexelRadiusX, ShadowMapSizeX); SearchX++)
							{
								const FVector2f LightSpaceXYOffset((SearchX - ShadowMapX) * BoundsCellSizeX, (SearchY - ShadowMapY) * BoundsCellSizeY);
								const float PlaneHeightOffsetX = LightSpaceXYOffset.X * TanThetaX;
								const float PlaneHeightOffsetY = LightSpaceXYOffset.Y * TanThetaY;
								const float PlaneHeightOffset = PlaneHeightOffsetX + PlaneHeightOffsetY;

								const float SearchShadowMapDepth = ShadowMap[SearchY * ShadowMapSizeX + SearchX];
								const float ExtrapolatedSurfaceDepth = SurfaceDepth + PlaneHeightOffset;
								const float SearchTransition = LightSpaceXYOffset.Size();
								const float SameSurfaceDepthBias = SearchTransition * 1.0f;
								const bool bSearchTexelVisible = SearchShadowMapDepth > ExtrapolatedSurfaceDepth - SlopeScaledDepthBias - DepthBias - SameSurfaceDepthBias;

								if (bTexelVisible)
								{
									const float SearchNormalizedDistance = FMath::Clamp(SearchTransition / ShadowSettings.MaxTransitionDistanceWorldSpace, 0.0f, 1.0f);
									const float SearchEncodedDistance = bTexelVisible ? (SearchNormalizedDistance) * .5f + .5f : .5f - SearchNormalizedDistance * .5f;

									const float ReceiverDistanceFromLight = SurfaceDepth;
									const float OccluderDistanceFromLight = bTexelVisible ? SearchShadowMapDepth : TexelShadowMapDepth;
									// World space distance from center of penumbra to fully shadowed or fully lit transition
									const float SearchPenumbraSize = (ReceiverDistanceFromLight - OccluderDistanceFromLight) * Light->LightSourceRadius / OccluderDistanceFromLight;
									const float SearchEncodedPenumbraSize = FMath::Clamp(SearchPenumbraSize / ShadowSettings.MaxTransitionDistanceWorldSpace, 0.01f, 1.0f);

									const float SearchShadowing = FMath::Clamp(SearchEncodedDistance / SearchEncodedPenumbraSize - .5f / SearchEncodedPenumbraSize + .5f, 0.0f, 1.0f);

									if (bSearchTexelVisible != bTexelVisible
										/* && SearchTransition < ClosestTransition*/
										&& SearchShadowing < MostShadowingTransition)
									{
										MostShadowingTransition = SearchShadowing;
										MostShadowingTransitionDistance = SearchEncodedDistance;
										MostShadowingTransitionPenumbraSize = SearchEncodedPenumbraSize;
										//ClosestTransition = SearchTransition;
										/*
										if (bTexelVisible)
										{
											// Approximate the penumbra size using PenumbraSize = (ReceiverDistanceFromLight - OccluderDistanceFromLight) * LightSize / OccluderDistanceFromLight,
											// Which is from the paper "Percentage-Closer Soft Shadows" by Randima Fernando
											const float ReceiverDistanceFromLight = SurfaceDepth;
											const float OccluderDistanceFromLight = SearchShadowMapDepth;
											// World space distance from center of penumbra to fully shadowed or fully lit transition
											const float PenumbraSize = (ReceiverDistanceFromLight - OccluderDistanceFromLight) * Light->LightSourceRadius / OccluderDistanceFromLight;
											// Normalize the penumbra size so it is a fraction of MaxTransitionDistanceWorldSpace
											ClosestTransitionPenumbraSize = FMath::Clamp(PenumbraSize / ShadowSettings.MaxTransitionDistanceWorldSpace, 0.01f, 1.0f);
										}*/
									}
								}
								else
								{
									if (bSearchTexelVisible != bTexelVisible
										&& SearchTransition < ClosestTransition)
									{
										ClosestTransition = SearchTransition;
									}
								}
							}
						}

						FSignedDistanceFieldShadowSample& FinalShadowSample = (*ShadowMapData)(X, Y);
						FinalShadowSample.bIsMapped = true;
						FinalShadowSample.Distance = MostShadowingTransitionDistance;
						FinalShadowSample.PenumbraSize = MostShadowingTransitionPenumbraSize;

						if (!bTexelVisible)
						{
							const float NormalizedDistance = FMath::Clamp(ClosestTransition / ShadowSettings.MaxTransitionDistanceWorldSpace, 0.0f, 1.0f);
							// Encode the transition distance so that [.5,0] corresponds to [0,1] for shadowed texels, and [.5,1] corresponds to [0,1] for unshadowed texels.
							// .5 of the encoded distance lies exactly on the shadow transition.
							FinalShadowSample.Distance = bTexelVisible ? (NormalizedDistance) * .5f + .5f : .5f - NormalizedDistance * .5f;

							const float ReceiverDistanceFromLight = SurfaceDepth;
							const float OccluderDistanceFromLight = TexelShadowMapDepth;
							const float PenumbraSize = (ReceiverDistanceFromLight - OccluderDistanceFromLight) * Light->LightSourceRadius / OccluderDistanceFromLight;
							FinalShadowSample.PenumbraSize = FMath::Clamp(PenumbraSize / ShadowSettings.MaxTransitionDistanceWorldSpace, 0.01f, 1.0f);
						}
						/*
						const float NormalizedDistance = FMath::Clamp(ClosestTransition / ShadowSettings.MaxTransitionDistanceWorldSpace, 0.0f, 1.0f);
						// Encode the transition distance so that [.5,0] corresponds to [0,1] for shadowed texels, and [.5,1] corresponds to [0,1] for unshadowed texels.
						// .5 of the encoded distance lies exactly on the shadow transition.
						FinalShadowSample.Distance = bTexelVisible ? (NormalizedDistance) * .5f + .5f : .5f - NormalizedDistance * .5f;

						if (bTexelVisible)
						{
							FinalShadowSample.PenumbraSize = ClosestTransitionPenumbraSize;
						}
						else
						{
							const float ReceiverDistanceFromLight = SurfaceDepth;
							const float OccluderDistanceFromLight = TexelShadowMapDepth;
							const float PenumbraSize = (ReceiverDistanceFromLight - OccluderDistanceFromLight) * Light->LightSourceRadius / OccluderDistanceFromLight;
							FinalShadowSample.PenumbraSize = FMath::Clamp(PenumbraSize / ShadowSettings.MaxTransitionDistanceWorldSpace, 0.01f, 1.0f);
						}
						*/
					}
				}
			}

			ShadowMaps.Add(Light, ShadowMapData);
		}
	}
}

/**
 * Estimate direct lighting using the direct photon map.
 * This is only useful for debugging what the final gather rays see.
 */
void FStaticLightingSystem::CalculateDirectLightingTextureMappingPhotonMap(
	FStaticLightingTextureMapping* TextureMapping, 
	FStaticLightingMappingContext& MappingContext,
	FGatheredLightMapData2D& LightMapData, 
	TMap<const FLight*, FShadowMapData2D*>& ShadowMaps,
	const FTexelToVertexMap& TexelToVertexMap, 
	bool bDebugThisMapping) const
{
	for (int32 LightIndex = 0; LightIndex < TextureMapping->Mesh->RelevantLights.Num(); LightIndex++)
	{
		FLight* Light = TextureMapping->Mesh->RelevantLights[LightIndex];
		if (Light->GetMeshAreaLight() == NULL)
		{
			LightMapData.AddLight(Light);
		}
	}

	TArray<FIrradiancePhoton*> TempIrradiancePhotons;
	// Calculate direct lighting for each texel.
	for (int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
	{
		for (int32 X = 0; X < TextureMapping->CachedSizeX; X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == Scene.DebugInput.LocalY
				&& X == Scene.DebugInput.LocalX)
			{
				bDebugThisTexel = true;
			}
#endif
			FGatheredLightMapSample& CurrentLightSample = LightMapData(X,Y);
			if (CurrentLightSample.bIsMapped)
			{
				const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
				FStaticLightingVertex CurrentVertex = TexelToVertex.GetVertex();

				if (PhotonMappingSettings.bUseIrradiancePhotons)
				{
					FLinearColor DirectLighting;

					const FIrradiancePhoton* NearestPhoton = NULL;
					if (PhotonMappingSettings.bCacheIrradiancePhotonsOnSurfaces)
					{
						// Trace a ray into the current texel to get a good representation of what the final gather will see.
						// Speed does not matter here since bVisualizeCachedApproximateDirectLighting is only used for debugging.
						const FLightRay TexelRay(
							CurrentVertex.WorldPosition + CurrentVertex.WorldTangentZ * TexelToVertex.TexelRadius,
							CurrentVertex.WorldPosition - CurrentVertex.WorldTangentZ * TexelToVertex.TexelRadius,
							TextureMapping,
							NULL
							);

						FLightRayIntersection Intersection;
						AggregateMesh->IntersectLightRay(TexelRay, true, false, false, MappingContext.RayCache, Intersection);

						if (Intersection.bIntersects && TextureMapping == Intersection.Mapping)
						{
							CurrentVertex = Intersection.IntersectionVertex;
						}
						else
						{
							// Fall back to using the UV's of this texel
							CurrentVertex.TextureCoordinates[1] = FVector2f(X / (float)TextureMapping->CachedSizeX, Y / (float)TextureMapping->CachedSizeY);
						}

						// Find the nearest irradiance photon that was cached on this surface

						checkf(0, TEXT("No longer implemented"));
					}
					else
					{
						// Find the nearest irradiance photon by searching the irradiance photon map
						NearestPhoton = FindNearestIrradiancePhoton(CurrentVertex, MappingContext, TempIrradiancePhotons, false, bDebugThisTexel);

						FGatheredLightSample DirectLightingSample;
						float Unused2;
						TArray<FVector3f, TInlineAllocator<1>> VertexOffsets;
						VertexOffsets.Add(FVector3f(0, 0, 0));

						CalculateApproximateDirectLighting(CurrentVertex, TexelToVertex.TexelRadius, VertexOffsets, .1f, true, true, bDebugThisTexel, MappingContext, DirectLightingSample, Unused2);

						DirectLighting = DirectLightingSample.IncidentLighting;
					}
					const FLinearColor& PhotonIrradiance = NearestPhoton ? NearestPhoton->GetIrradiance() : FLinearColor::Black;
					if (GeneralSettings.ViewSingleBounceNumber < 1)
					{
						FLinearColor FinalLighting = PhotonIrradiance;

						if (!PhotonMappingSettings.bUsePhotonDirectLightingInFinalGather)
						{
							FinalLighting += DirectLighting;
						}

						//@todo - can't visualize accurately using AmbientLight with directional lightmaps
						//CurrentLightSample.AddWeighted(FGatheredLightSampleUtil::AmbientLight<2>(FinalLighting), 1.0f);
						CurrentLightSample.AddWeighted(FGatheredLightSampleUtil::PointLightWorldSpace<2>(FinalLighting, FVector4f(0, 0, 1), CurrentVertex.WorldTangentZ), 1.0f);
					}
				}
				else
				{
					// Estimate incident radiance from the photons in the direct photon map
					const FGatheredLightSample PhotonIncidentRadiance = CalculatePhotonIncidentRadiance(DirectPhotonMap, NumPhotonsEmittedDirect, PhotonMappingSettings.DirectPhotonSearchDistance, CurrentVertex, bDebugThisTexel);
					if (GeneralSettings.ViewSingleBounceNumber < 1)
					{
						CurrentLightSample.AddWeighted(PhotonIncidentRadiance, 1.0f);
					}
				}
			}
		}
	}
}

/** 
 * Builds an irradiance cache for a given mapping task.  
 * This can be called from any thread, not just the thread that owns the mapping, so called code must be thread safe in that manner.
 */
void FStaticLightingSystem::ProcessCacheIndirectLightingTask(FCacheIndirectTaskDescription* Task, bool bProcessedByMappingThread)
{
	const double StartTime = FPlatformTime::Seconds();
	FLMRandomStream SampleGenerator(Task->StartY * Task->TextureMapping->CachedSizeX + Task->StartX);

	// Calculate incident radiance from indirect lighting
	// With irradiance caching this is just the first pass, the results are added to the cache
	//@todo - use a hierarchical traversal to minimize the number of samples created
	// See "Problems and Solutions: Implementation Details" from the SIGGRAPH 2008 class titled "Practical Global Illumination with Irradiance Caching"
	for (int32 Y = Task->StartY; Y < Task->StartY + Task->SizeY; Y++)
	{
		for (int32 X = Task->StartX; X < Task->StartX + Task->SizeX; X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (Task->bDebugThisMapping
				&& Y == Scene.DebugInput.LocalY
				&& X == Scene.DebugInput.LocalX)
			{
				bDebugThisTexel = true;
			}
#endif
			FGatheredLightMapSample& CurrentLightSample = (*Task->LightMapData)(X,Y);
			if (CurrentLightSample.bIsMapped)
			{
				const FTexelToVertex& TexelToVertex = (*Task->TexelToVertexMap)(X,Y);
				checkSlow(TexelToVertex.TotalSampleWeight > 0.0f);
				FFullStaticLightingVertex TexelVertex = TexelToVertex.GetFullVertex();
				TexelVertex.TextureCoordinates[1] = FVector2f(X / (float)Task->TextureMapping->CachedSizeX, Y / (float)Task->TextureMapping->CachedSizeY);

				// Calculate incoming radiance for the frontface
				FGatheredLightSample IndirectLightingSample = CachePointIncomingRadiance(
					Task->TextureMapping, 
					TexelVertex, 
					TexelToVertex.ElementIndex,
					TexelToVertex.TexelRadius, 
					TexelToVertex.SampleRadius, 
					TexelToVertex.bIntersectingSurface,
					Task->MappingContext, 
					SampleGenerator, 
					bDebugThisTexel);

				if (Task->TextureMapping->Mesh->UsesTwoSidedLighting(TexelToVertex.ElementIndex))
				{
					TexelVertex.WorldTangentX = -TexelVertex.WorldTangentX;
					TexelVertex.WorldTangentY = -TexelVertex.WorldTangentY;
					TexelVertex.WorldTangentZ = -TexelVertex.WorldTangentZ;

					// Calculate incoming radiance for the backface
					const FGatheredLightSample BackFaceIndirectLightingSample = CachePointIncomingRadiance(
						Task->TextureMapping, 
						TexelVertex, 
						TexelToVertex.ElementIndex,
						TexelToVertex.TexelRadius, 
						TexelToVertex.SampleRadius, 
						TexelToVertex.bIntersectingSurface,
						Task->MappingContext, 
						SampleGenerator, 
						bDebugThisTexel);
					// Average front and back face incident lighting
					IndirectLightingSample = (BackFaceIndirectLightingSample + IndirectLightingSample) * 0.5f;
				}

				if (!IrradianceCachingSettings.bAllowIrradianceCaching)
				{
					CurrentLightSample.AddWeighted(IndirectLightingSample, 1.0f);
				}
			}
		}
	}

	const float TaskExecutionTime = FPlatformTime::Seconds() - StartTime;

	if (bProcessedByMappingThread)
	{
		Task->MappingContext.Stats.IndirectLightingCacheTaskThreadTime += TaskExecutionTime;
	}
	else
	{
		Task->MappingContext.Stats.IndirectLightingCacheTaskThreadTimeSeparateTask += TaskExecutionTime;
	}
}

/** 
 * Interpolates from the irradiance cache for a given mapping task.  
 * This can be called from any thread, not just the thread that owns the mapping, so called code must be thread safe in that manner.
 */
void FStaticLightingSystem::ProcessInterpolateTask(FInterpolateIndirectTaskDescription* Task, bool bProcessedByMappingThread)
{
	const double StartTime = FPlatformTime::Seconds();

	// Interpolate irradiance cache samples in a separate shading pass
	// This avoids interpolating to positions where more samples will be added later, which would create a discontinuity
	// Also allows us to use more lenient restrictions in this pass, which effectively smooths the irradiance cache results
	for (int32 Y = Task->StartY; Y < Task->StartY + Task->SizeY; Y++)
	{
		for (int32 X = Task->StartX; X < Task->StartX + Task->SizeX; X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING 
			if (Task->bDebugThisMapping
				&& Y == Scene.DebugInput.LocalY
				&& X == Scene.DebugInput.LocalX)
			{
				bDebugThisTexel = true;
			}
#endif
			FGatheredLightMapSample& CurrentLightSample = (*Task->LightMapData)(X,Y);
			if (CurrentLightSample.bIsMapped)
			{
				const FTexelToVertex& TexelToVertex = (*Task->TexelToVertexMap)(X,Y);
				checkSlow(TexelToVertex.TotalSampleWeight > 0.0f);
				FFullStaticLightingVertex TexelVertex = TexelToVertex.GetFullVertex();
				FFinalGatherSample IndirectLighting;
				FFinalGatherSample SecondInterpolatedIndirectLighting;
				float BackfacingHitsFraction = 0.0f;
				float BackFaceBackfacingHitsFraction = 1.0f;
				// Interpolate the indirect lighting from the irradiance cache
				// Interpolation must succeed since this is the second pass
				verify(Task->FirstBounceCache->InterpolateLighting(TexelVertex, false, bDebugThisTexel && GeneralSettings.ViewSingleBounceNumber == 1, IrradianceCachingSettings.SkyOcclusionSmoothnessReduction, IndirectLighting, SecondInterpolatedIndirectLighting, BackfacingHitsFraction, Task->MappingContext.DebugCacheRecords));

				// Replace sky occlusion in the lighting sample that will be written into the lightmap with the interpolated sky occlusion using IrradianceCachingSettings.SkyOcclusionSmoothnessReduction
				IndirectLighting.SkyOcclusion = SecondInterpolatedIndirectLighting.SkyOcclusion;
				IndirectLighting.StationarySkyLighting = SecondInterpolatedIndirectLighting.StationarySkyLighting;
				
				if (Task->TextureMapping->Mesh->UsesTwoSidedLighting(TexelToVertex.ElementIndex))
				{
					TexelVertex.WorldTangentX = -TexelVertex.WorldTangentX;
					TexelVertex.WorldTangentY = -TexelVertex.WorldTangentY;
					TexelVertex.WorldTangentZ = -TexelVertex.WorldTangentZ;

					FFinalGatherSample BackFaceIndirectLighting;
					FFinalGatherSample BackFaceSecondInterpolatedIndirectLighting;
					// Interpolate indirect lighting for the back face
					verify(Task->FirstBounceCache->InterpolateLighting(TexelVertex, false, bDebugThisTexel && GeneralSettings.ViewSingleBounceNumber == 1, IrradianceCachingSettings.SkyOcclusionSmoothnessReduction, BackFaceIndirectLighting, BackFaceSecondInterpolatedIndirectLighting, BackFaceBackfacingHitsFraction, Task->MappingContext.DebugCacheRecords));
					BackFaceIndirectLighting.SkyOcclusion = BackFaceSecondInterpolatedIndirectLighting.SkyOcclusion;
					// Average front and back face incident lighting
					IndirectLighting = (BackFaceIndirectLighting + IndirectLighting) * 0.5f;
				}

				if (BackfacingHitsFraction > 0.5f && BackFaceBackfacingHitsFraction > 0.5f)
				{
					CurrentLightSample.bIsMapped = false;
				}

				float IndirectOcclusion = 1.0f;
				if (AmbientOcclusionSettings.bUseAmbientOcclusion)
				{
					const float DirectOcclusion = 1.0f - AmbientOcclusionSettings.DirectIlluminationOcclusionFraction * IndirectLighting.Occlusion;
					// Apply occlusion to direct lighting, assuming CurrentLightSample only contains direct lighting
					CurrentLightSample.HighQuality = CurrentLightSample.HighQuality * DirectOcclusion;
					CurrentLightSample.LowQuality = CurrentLightSample.LowQuality * DirectOcclusion;
					IndirectOcclusion = 1.0f - AmbientOcclusionSettings.IndirectIlluminationOcclusionFraction * IndirectLighting.Occlusion;
				}

				IndirectLighting.ApplyOcclusion(IndirectOcclusion);

				// Apply occlusion to indirect lighting and add this texel's indirect lighting to its running total
				CurrentLightSample.AddWeighted(IndirectLighting, 1);
				CurrentLightSample.HighQuality.AOMaterialMask = IndirectLighting.Occlusion;

				if (AmbientOcclusionSettings.bUseAmbientOcclusion && AmbientOcclusionSettings.bVisualizeAmbientOcclusion)
				{
					//@todo - this will only be the correct intensity for simple lightmaps
					const FGatheredLightSample OcclusionVisualization = FGatheredLightSampleUtil::AmbientLight<2>(
						FLinearColor(1.0f - IndirectLighting.Occlusion, 1.0f - IndirectLighting.Occlusion, 1.0f - IndirectLighting.Occlusion) * 0.5f);
					// Overwrite the lighting accumulated so far
					CurrentLightSample = OcclusionVisualization;
					CurrentLightSample.bIsMapped = true;
				}
			}
		}
	}

	const float TaskExecutionTime = FPlatformTime::Seconds() - StartTime;

	if (bProcessedByMappingThread)
	{
		Task->MappingContext.Stats.SecondPassIrradianceCacheInterpolationTime += TaskExecutionTime;
	}
	else
	{
		Task->MappingContext.Stats.SecondPassIrradianceCacheInterpolationTimeSeparateTask += TaskExecutionTime;
	}
}

/** Handles indirect lighting calculations for a single texture mapping. */
void FStaticLightingSystem::CalculateIndirectLightingTextureMapping(
	FStaticLightingTextureMapping* TextureMapping,
	FStaticLightingMappingContext& MappingContext,
	FGatheredLightMapData2D& LightMapData, 
	const FTexelToVertexMap& TexelToVertexMap, 
	bool bDebugThisMapping)
{
	// Whether to debug the task containing the selected texel only
	const bool bDebugSelectedTaskOnly = true;

	if (GeneralSettings.NumIndirectLightingBounces > 0 || AmbientOcclusionSettings.bUseAmbientOcclusion || SkyLights.Num() > 0)
	{
		const double StartCacheTime = FPlatformTime::Seconds();

		const int32 CacheTaskSize = IrradianceCachingSettings.CacheTaskSize;
		int32 NumTasksSubmitted = 0;

		// Break this mapping into multiple caching tasks in texture space blocks
		for (int32 TaskY = 0; TaskY < TextureMapping->CachedSizeY; TaskY += CacheTaskSize)
		{
			for (int32 TaskX = 0; TaskX < TextureMapping->CachedSizeX; TaskX += CacheTaskSize)
			{
				FCacheIndirectTaskDescription* NewTask = new FCacheIndirectTaskDescription(TextureMapping->Mesh, *this);
				NewTask->StartX = TaskX;
				NewTask->StartY = TaskY;
				NewTask->SizeX = FMath::Min(CacheTaskSize, TextureMapping->CachedSizeX - TaskX);
				NewTask->SizeY = FMath::Min(CacheTaskSize, TextureMapping->CachedSizeY - TaskY);
				NewTask->TextureMapping = TextureMapping;
				NewTask->LightMapData = &LightMapData;
				NewTask->TexelToVertexMap = &TexelToVertexMap;
				
				NewTask->bDebugThisMapping = bDebugThisMapping
					&& (!bDebugSelectedTaskOnly 
						|| (Scene.DebugInput.LocalX >= TaskX && Scene.DebugInput.LocalX < TaskX + CacheTaskSize
						&& Scene.DebugInput.LocalY >= TaskY && Scene.DebugInput.LocalY < TaskY + CacheTaskSize));

				NumTasksSubmitted++;

				// Add to the queue so other lighting threads can pick up these tasks
				FPlatformAtomics::InterlockedIncrement(&TextureMapping->NumOutstandingCacheTasks);
				CacheIndirectLightingTasks.Push(NewTask);
			}
		}

		do 
		{
			// Process caching tasks from any threads until this mapping's tasks are complete
			FCacheIndirectTaskDescription* NextTask = CacheIndirectLightingTasks.Pop();

			if (NextTask)
			{
				NextTask->bProcessedOnMainThread = true;
				ProcessCacheIndirectLightingTask(NextTask, true);
				// Add to the mapping's queue when complete
				NextTask->TextureMapping->CompletedCacheIndirectLightingTasks.Push(NextTask);
				FPlatformAtomics::InterlockedDecrement(&NextTask->TextureMapping->NumOutstandingCacheTasks);
			}
		} 
		while (TextureMapping->NumOutstandingCacheTasks > 0);

		TArray<FCacheIndirectTaskDescription*> CompletedCILTasks;
		TextureMapping->CompletedCacheIndirectLightingTasks.PopAll(CompletedCILTasks);
		check(CompletedCILTasks.Num() == NumTasksSubmitted);

		int32 NextRecordId = 0;

		for (int32 TaskIndex = 0; TaskIndex < CompletedCILTasks.Num(); TaskIndex++)
		{
			FCacheIndirectTaskDescription* Task = CompletedCILTasks[TaskIndex];

			TArray<TLightingCache<FFinalGatherSample>::FRecord<FFinalGatherSample> > Records;
			Task->MappingContext.FirstBounceCache.GetAllRecords(Records);

			// Merge the first bounce irradiance caches into one
			for (int32 RecordIndex = 0; RecordIndex < Records.Num(); RecordIndex++)
			{
				Records[RecordIndex].Id += NextRecordId;
				MappingContext.FirstBounceCache.AddRecord(Records[RecordIndex], false, false);
			}

			for (int32 RecordIndex = 0; RecordIndex < Task->MappingContext.DebugCacheRecords.Num(); RecordIndex++)
			{
				Task->MappingContext.DebugCacheRecords[RecordIndex].RecordId += NextRecordId;
			}

			MappingContext.DebugCacheRecords.Append(Task->MappingContext.DebugCacheRecords);

			NextRecordId += Records.Num();

			// Note: the task's MappingContext stats will be merged into the global stats automatically due to the MappingContext destructor

			delete CompletedCILTasks[TaskIndex];
		}

		const double EndCacheTime = FPlatformTime::Seconds();

		MappingContext.Stats.BlockOnIndirectLightingCacheTasksTime += EndCacheTime - StartCacheTime;
		
		if (IrradianceCachingSettings.bAllowIrradianceCaching)
		{
			const int32 InterpolationTaskSize = IrradianceCachingSettings.InterpolateTaskSize;
			int32 NumIILTasksSubmitted = 0;

			// Break this mapping into multiple interpolation tasks in texture space blocks
			for (int32 TaskY = 0; TaskY < TextureMapping->CachedSizeY; TaskY += InterpolationTaskSize)
			{
				for (int32 TaskX = 0; TaskX < TextureMapping->CachedSizeX; TaskX += InterpolationTaskSize)
				{
					FInterpolateIndirectTaskDescription* NewTask = new FInterpolateIndirectTaskDescription(TextureMapping->Mesh, *this);
					NewTask->StartX = TaskX;
					NewTask->StartY = TaskY;
					NewTask->SizeX = FMath::Min(InterpolationTaskSize, TextureMapping->CachedSizeX - TaskX);
					NewTask->SizeY = FMath::Min(InterpolationTaskSize, TextureMapping->CachedSizeY - TaskY);
					NewTask->TextureMapping = TextureMapping;
					NewTask->LightMapData = &LightMapData;
					NewTask->TexelToVertexMap = &TexelToVertexMap;
					NewTask->FirstBounceCache = &MappingContext.FirstBounceCache;
					NewTask->MappingContext.DebugCacheRecords = MappingContext.DebugCacheRecords;

					NewTask->bDebugThisMapping = bDebugThisMapping
						&& (!bDebugSelectedTaskOnly 
							|| (Scene.DebugInput.LocalX >= TaskX && Scene.DebugInput.LocalX < TaskX + InterpolationTaskSize
							&& Scene.DebugInput.LocalY >= TaskY && Scene.DebugInput.LocalY < TaskY + InterpolationTaskSize));

					NumIILTasksSubmitted++;
					FPlatformAtomics::InterlockedIncrement(&TextureMapping->NumOutstandingInterpolationTasks);
					InterpolateIndirectLightingTasks.Push(NewTask);
				}
			}

			do 
			{
				FInterpolateIndirectTaskDescription* NextTask = InterpolateIndirectLightingTasks.Pop();
				 
				if (NextTask)
				{
					ProcessInterpolateTask(NextTask, true);
					NextTask->TextureMapping->CompletedInterpolationTasks.Push(NextTask);
					FPlatformAtomics::InterlockedDecrement(&NextTask->TextureMapping->NumOutstandingInterpolationTasks);
				}
			} 
			while (TextureMapping->NumOutstandingInterpolationTasks > 0);

			TArray<FInterpolateIndirectTaskDescription*> CompletedTasks;
			TextureMapping->CompletedInterpolationTasks.PopAll(CompletedTasks);
			check(CompletedTasks.Num() == NumIILTasksSubmitted);

			for (int32 TaskIndex = 0; TaskIndex < CompletedTasks.Num(); TaskIndex++)
			{
				FInterpolateIndirectTaskDescription* Task = CompletedTasks[TaskIndex];

				check(Task->MappingContext.DebugCacheRecords.Num() == MappingContext.DebugCacheRecords.Num());

				for (int32 CacheRecordIndex = 0; CacheRecordIndex < MappingContext.DebugCacheRecords.Num(); CacheRecordIndex++)
				{
					// Combine results
					MappingContext.DebugCacheRecords[CacheRecordIndex].bAffectsSelectedTexel |= Task->MappingContext.DebugCacheRecords[CacheRecordIndex].bAffectsSelectedTexel;
				}

				delete CompletedTasks[TaskIndex];
			}

			MappingContext.DebugOutput->CacheRecords = MappingContext.DebugCacheRecords;
		}

		MappingContext.Stats.BlockOnIndirectLightingInterpolateTasksTime += FPlatformTime::Seconds() - EndCacheTime;
	}

	FPlatformAtomics::InterlockedDecrement(&TasksInProgressThatWillNeedHelp);
}

/** Overrides LightMapData with material attributes if MaterialSettings.ViewMaterialAttribute != VMA_None */
void FStaticLightingSystem::ViewMaterialAttributesTextureMapping(
	FStaticLightingTextureMapping* TextureMapping,
	FStaticLightingMappingContext& MappingContext,
	FGatheredLightMapData2D& LightMapData, 
	const FTexelToVertexMap& TexelToVertexMap, 
	bool bDebugThisMapping) const
{
	if (MaterialSettings.ViewMaterialAttribute != VMA_None)
	{
		for(int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
		{
			for(int32 X = 0; X < TextureMapping->CachedSizeX; X++)
			{
				bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
				if (bDebugThisMapping
					&& Y == Scene.DebugInput.LocalY
					&& X == Scene.DebugInput.LocalX)
				{
					bDebugThisTexel = true;
				}
#endif
				FGatheredLightMapSample& CurrentLightSample = LightMapData(X,Y);
				if (CurrentLightSample.bIsMapped)
				{
					const FTexelToVertex& TexelToVertex = TexelToVertexMap(X,Y);
					checkSlow(TexelToVertex.TotalSampleWeight > 0.0f);
					FStaticLightingVertex CurrentVertex = TexelToVertex.GetVertex();

					// Trace a ray into the current texel to get a good representation of what material lookups from ray intersections will see.
					// Speed does not matter here since this visualization is only used for debugging.
					const FLightRay TexelRay(
						CurrentVertex.WorldPosition + CurrentVertex.WorldTangentZ * TexelToVertex.TexelRadius,
						CurrentVertex.WorldPosition - CurrentVertex.WorldTangentZ * TexelToVertex.TexelRadius,
						TextureMapping,
						NULL
						);

					FLightRayIntersection Intersection;
					AggregateMesh->IntersectLightRay(TexelRay, true, true, false, MappingContext.RayCache, Intersection);
					CurrentLightSample = GetVisualizedMaterialAttribute(TextureMapping, Intersection);
				}
			}
		}
	}
}

/** A map from texel to the number of triangles mapped to that texel. */
class FTexelToNumTrianglesMap
{
public:

	/** Stores information about a texel needed for determining the validity of the lightmap UVs. */
	struct FTexelToNumTriangles
	{
		bool bWrappingUVs;
		int32 NumTriangles;
	};

	/** Initialization constructor. */
	FTexelToNumTrianglesMap(int32 InSizeX, int32 InSizeY) :
		SizeX(InSizeX),
		SizeY(InSizeY)
	{
		// Clear the map to zero.
		Data.AddZeroed(SizeX * SizeY);
	}

	// Accessors.
	FTexelToNumTriangles& operator()(int32 X, int32 Y)
	{
		const uint32 TexelIndex = Y * SizeX + X;
		return Data[TexelIndex];
	}
	const FTexelToNumTriangles& operator()(int32 X, int32 Y) const
	{
		const int32 TexelIndex = Y * SizeX + X;
		return Data[TexelIndex];
	}

	int32 GetSizeX() const { return SizeX; }
	int32 GetSizeY() const { return SizeY; }

private:

	/** The mapping data. */
	TArray<FTexelToNumTriangles> Data;

	/** The width of the mapping data. */
	int32 SizeX;

	/** The height of the mapping data. */
	int32 SizeY;
};

/** Rasterization policy for verifying unique lightmap UVs. */
class FUniqueMappingRasterPolicy
{
public:

	typedef int32 InterpolantType;

	/** Initialization constructor. */
	FUniqueMappingRasterPolicy(
		const FScene& InScene,
		FTexelToNumTrianglesMap& InTexelToNumTrianglesMap,
		bool bInDebugThisMapping
		) :
		Scene(InScene),
		TexelToNumTrianglesMap(InTexelToNumTrianglesMap),
		TotalPixelsWritten(0),
		TotalPixelOverlapsOccured(0),
		bDebugThisMapping(bInDebugThisMapping)
	{}

	int32 GetTotalPixelsWritten() const
	{
		return TotalPixelsWritten;
	}
	int32 GetTotalPixelOverlapsOccured() const
	{
		return TotalPixelOverlapsOccured;
	}
protected:

	// FTriangleRasterizer policy interface.
	int32 GetMinX() const { return 0; }
	int32 GetMaxX() const { return TexelToNumTrianglesMap.GetSizeX() - 1; }
	int32 GetMinY() const { return 0; }
	int32 GetMaxY() const { return TexelToNumTrianglesMap.GetSizeY() - 1; }

	void ProcessPixel(int32 X, int32 Y, const InterpolantType& Interpolant, bool BackFacing);

private:

	const FScene& Scene;

	/** The texel to vertex map which is being rasterized to. */
	FTexelToNumTrianglesMap& TexelToNumTrianglesMap;

	int32 TotalPixelsWritten;
	int32 TotalPixelOverlapsOccured;

	const bool bDebugThisMapping;
};

void FUniqueMappingRasterPolicy::ProcessPixel(int32 X, int32 Y, const InterpolantType& bWrappingUVs, bool BackFacing)
{
	FTexelToNumTrianglesMap::FTexelToNumTriangles& TexelToNumTriangles = TexelToNumTrianglesMap(X, Y);
	bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
	if (bDebugThisMapping
		&& X == Scene.DebugInput.LocalX
		&& Y == Scene.DebugInput.LocalY)
	{
		bDebugThisTexel = true;
	}
#endif
	TexelToNumTriangles.NumTriangles++;
	if (TexelToNumTriangles.NumTriangles > 1)
	{
		TotalPixelOverlapsOccured++;
	}
	TotalPixelsWritten++;
	TexelToNumTriangles.bWrappingUVs = !!bWrappingUVs;
}

/** Colors texels with invalid lightmap UVs to make it obvious that they are wrong. */
void FStaticLightingSystem::ColorInvalidLightmapUVs(
	const FStaticLightingTextureMapping* TextureMapping, 
	FGatheredLightMapData2D& LightMapData, 
	bool bDebugThisMapping) const
{
	FTexelToNumTrianglesMap TexelToNumTrianglesMap(TextureMapping->CachedSizeX, TextureMapping->CachedSizeY);

	// Rasterize the triangle using the mapping's texture coordinate channel.
	FTriangleRasterizer<FUniqueMappingRasterPolicy> TexelMappingRasterizer(FUniqueMappingRasterPolicy(
		Scene,
		TexelToNumTrianglesMap,
		bDebugThisMapping
		));

	const int32 TriangleCount = TextureMapping->Mesh->NumTriangles;
	// Rasterize the triangles
	for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; TriangleIndex++)
	{
		// Query the mesh for the triangle's vertices.
		FStaticLightingVertex V0;
		FStaticLightingVertex V1;
		FStaticLightingVertex V2;
		int32 DummyElement;
		TextureMapping->Mesh->GetTriangle(TriangleIndex,V0,V1,V2,DummyElement);

		const FVector2f UV0 = V0.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
		const FVector2f UV1 = V1.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];
		const FVector2f UV2 = V2.TextureCoordinates[TextureMapping->LightmapTextureCoordinateIndex];

		bool bHasWrappingLightmapUVs = false;
		//@todo - remove the thresholds and fixup existing content
		if (UV0.X < -DELTA || UV0.X >= 1.0f + DELTA
			|| UV0.Y < -DELTA || UV0.Y >= 1.0f + DELTA
			|| UV1.X < -DELTA || UV1.X >= 1.0f + DELTA
			|| UV1.Y < -DELTA || UV1.Y >= 1.0f + DELTA
			|| UV2.X < -DELTA || UV2.X >= 1.0f + DELTA
			|| UV2.Y < -DELTA || UV2.Y >= 1.0f + DELTA)
		{
			bHasWrappingLightmapUVs = true;
		}

		// Only rasterize the center of the texel
		TexelMappingRasterizer.DrawTriangle(
			bHasWrappingLightmapUVs,
			bHasWrappingLightmapUVs,
			bHasWrappingLightmapUVs,
			UV0 * FVector2f(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY) + FVector2f(-0.5f,-0.5f),
			UV1 * FVector2f(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY) + FVector2f(-0.5f,-0.5f),
			UV2 * FVector2f(TextureMapping->CachedSizeX,TextureMapping->CachedSizeY) + FVector2f(-0.5f,-0.5f),
			false 
			);
	}

	bool bHasWrappingUVs = false;
	bool bHasOverlappedUVs = false;
	for(int32 Y = 0; Y < TextureMapping->CachedSizeY; Y++)
	{
		// Color texels belonging to vertices with wrapping lightmap UV's bright green
		// Color texels that have more than one triangle mapped to them bright orange
		for(int32 X = 0; X < TextureMapping->CachedSizeX; X++)
		{
			bool bDebugThisTexel = false;
#if ALLOW_LIGHTMAP_SAMPLE_DEBUGGING
			if (bDebugThisMapping
				&& Y == Scene.DebugInput.LocalY
				&& X == Scene.DebugInput.LocalX)
			{
				bDebugThisTexel = true;
			}
#endif
			FGatheredLightMapSample& CurrentLightSample = LightMapData(X,Y);
			if (CurrentLightSample.bIsMapped)
			{
				const FTexelToNumTrianglesMap::FTexelToNumTriangles& TexelToNumTriangles = TexelToNumTrianglesMap(X,Y);
				if (TexelToNumTriangles.bWrappingUVs)
				{
					bHasWrappingUVs = true;
					if (Scene.GeneralSettings.bUseErrorColoring && MaterialSettings.ViewMaterialAttribute == VMA_None)
					{
						// Color texels belonging to vertices with wrapping lightmap UV's bright green
						if (TextureMapping->Mesh->ShouldColorInvalidTexels())
						{
							CurrentLightSample = FGatheredLightSampleUtil::AmbientLight<2>(FLinearColor(0.5f, 2.0f, 0.0f));
							CurrentLightSample.bIsMapped = true;
						}
					}
				}
				else if (TexelToNumTriangles.NumTriangles > 1)
				{
					bHasOverlappedUVs = true;
					if (Scene.GeneralSettings.bUseErrorColoring && MaterialSettings.ViewMaterialAttribute == VMA_None)
					{
						// Color texels that have more than one triangle mapped to them bright orange
						if (TextureMapping->Mesh->ShouldColorInvalidTexels())
						{
							CurrentLightSample = FGatheredLightSampleUtil::AmbientLight<2>(FLinearColor(2.0f, 0.7f, 0.0f));
							CurrentLightSample.bIsMapped = true;
						}
					}
				}
			}
		}
	}

	const float OverlapThreshold = 1.0f / 100.0f;
	float NormalizedOverlap = (float)TexelMappingRasterizer.GetTotalPixelOverlapsOccured() / (float)TexelMappingRasterizer.GetTotalPixelsWritten();
	if (bHasWrappingUVs || bHasOverlappedUVs)
	{
		int32 TypeId = TextureMapping->Mesh->GetObjectType();
		FGuid ObjectGuid = TextureMapping->Mesh->GetObjectGuid();
		if (bHasWrappingUVs)
		{
			GSwarm->SendAlertMessage(NSwarm::ALERT_LEVEL_WARNING, ObjectGuid, TypeId, TEXT("LightmassError_ObjectWrappedUVs"));
		}
		if (bHasOverlappedUVs && NormalizedOverlap > OverlapThreshold)
		{
			GSwarm->SendAlertMessage(NSwarm::ALERT_LEVEL_WARNING, ObjectGuid, TypeId, TEXT("LightmassError_ObjectOverlappedUVs"));
			FString Info = FString::Printf(TEXT("Lightmap UV are overlapping by %0.1f%%. Please adjust content - Enable Error Coloring to visualize."), NormalizedOverlap * 100.0f);
			GSwarm->SendAlertMessage(NSwarm::ALERT_LEVEL_INFO, ObjectGuid, TypeId, Info.GetCharArray().GetData());
		}
	}
}

/** Adds a texel of padding around texture mappings and copies the nearest texel into the padding. */
void FStaticLightingSystem::PadTextureMapping(
	const FStaticLightingTextureMapping* TextureMapping,
	const FGatheredLightMapData2D& LightMapData,
	FGatheredLightMapData2D& PaddedLightMapData,
	TMap<const FLight*, FShadowMapData2D*>& ShadowMaps,
	TMap<const FLight*, FSignedDistanceFieldShadowMapData2D*>& SignedDistanceFieldShadowMaps) const
{
	if (TextureMapping->bPadded)
	{
		check(TextureMapping->SizeX == TextureMapping->CachedSizeX + 2);
		check(TextureMapping->SizeY == TextureMapping->CachedSizeY + 2);
		// We need to expand it back out...
		uint32 TrueSizeX = TextureMapping->SizeX;
		uint32 TrueSizeY = TextureMapping->SizeY;
		FGatheredLightMapSample DebugLightSample = FGatheredLightSampleUtil::AmbientLight<2>(FLinearColor(1.0f, 0.0f, 1.0f));
		for (uint32 CopyY = 0; CopyY < TrueSizeY; CopyY++)
		{
			if (CopyY == 0)
			{
				// The first row, left corner
				PaddedLightMapData(0,0) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugLightSample : LightMapData(0,0);
				// The rest of the row, short of the right corner
				for (uint32 TempX = 0; TempX < (uint32)(TextureMapping->CachedSizeX); TempX++)
				{
					PaddedLightMapData(TempX+1,0) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugLightSample : LightMapData(TempX,0);
				}
				// The right corner
				PaddedLightMapData(TrueSizeX-1,0) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugLightSample : LightMapData(TextureMapping->CachedSizeX-1,0);
			}
			else if (CopyY == TrueSizeY - 1)
			{
				// The last row, left corner
				PaddedLightMapData(0,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugLightSample : LightMapData(0,TextureMapping->CachedSizeY-1);
				// The rest of the row, short of the right corner
				for (uint32 TempX = 0; TempX < (uint32)(TextureMapping->CachedSizeX); TempX++)
				{
					PaddedLightMapData(TempX+1,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugLightSample : LightMapData(TempX,TextureMapping->CachedSizeY-1);
				}
				// The right corner
				PaddedLightMapData(TrueSizeX-1,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugLightSample : LightMapData(TextureMapping->CachedSizeX-1,TextureMapping->CachedSizeY-1);
			}
			else
			{
				// The last row, left corner
				PaddedLightMapData(0,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugLightSample : LightMapData(0,CopyY-1);
				// The rest of the row, short of the right corner
				for (uint32 TempX = 0; TempX < (uint32)(TextureMapping->CachedSizeX); TempX++)
				{
					PaddedLightMapData(TempX+1,CopyY) =LightMapData(TempX,CopyY-1);
				}
				// The right corner
				PaddedLightMapData(TrueSizeX-1,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugLightSample : LightMapData(TextureMapping->CachedSizeX-1,CopyY-1);
			}
		}
		PaddedLightMapData.Lights = LightMapData.Lights;
		PaddedLightMapData.bHasSkyShadowing = LightMapData.bHasSkyShadowing;

		FShadowSample DebugShadowSample;
		DebugShadowSample.bIsMapped = true;
		DebugShadowSample.Visibility = 0.7f;
		for (TMap<const FLight*, FShadowMapData2D*>::TIterator It(ShadowMaps); It; ++It)
		{
			const FLight* Key = It.Key();
			FShadowMapData2D* ShadowMapData = It.Value();
			FShadowMapData2D* TempShadowMapData = new FShadowMapData2D(TrueSizeX, TrueSizeY);

			// Expand it
			for (uint32 CopyY = 0; CopyY < TrueSizeY; CopyY++)
			{
				if (CopyY == 0)
				{
					// The first row, left corner
					(*TempShadowMapData)(0,0) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugShadowSample : (*ShadowMapData)(0,0);
					// The rest of the row, short of the right corner
					for (uint32 TempX = 0; TempX < (uint32)(TextureMapping->CachedSizeX); TempX++)
					{
						(*TempShadowMapData)(TempX+1,0) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugShadowSample : (*ShadowMapData)(TempX,0) * 2.0f - (*ShadowMapData)(TempX,1);
					}
					// The right corner
					(*TempShadowMapData)(TrueSizeX-1,0) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugShadowSample : (*ShadowMapData)(TextureMapping->CachedSizeX-1,0);
				}
				else if (CopyY == TrueSizeY - 1)
				{
					// The last row, left corner
					(*TempShadowMapData)(0,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugShadowSample : (*ShadowMapData)(0,TextureMapping->CachedSizeY-1);
					// The rest of the row, short of the right corner
					for (uint32 TempX = 0; TempX < (uint32)(TextureMapping->CachedSizeX); TempX++)
					{
						(*TempShadowMapData)(TempX+1,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugShadowSample : (*ShadowMapData)(TempX,TextureMapping->CachedSizeY-1) * 2.0f - (*ShadowMapData)(TempX,TextureMapping->CachedSizeY-2);
					}
					// The right corner
					(*TempShadowMapData)(TrueSizeX-1,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugShadowSample : (*ShadowMapData)(TextureMapping->CachedSizeX-1,TextureMapping->CachedSizeY-1);
				}
				else
				{
					// The last row, left corner
					(*TempShadowMapData)(0,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugShadowSample : (*ShadowMapData)(0,CopyY-1) * 2.0f - (*ShadowMapData)(1,CopyY-1);
					// The rest of the row, short of the right corner
					for (uint32 TempX = 0; TempX < (uint32)(TextureMapping->CachedSizeX); TempX++)
					{
						(*TempShadowMapData)(TempX+1,CopyY) = (*ShadowMapData)(TempX,CopyY-1);
					}
					// The right corner
					(*TempShadowMapData)(TrueSizeX-1,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugShadowSample : (*ShadowMapData)(TextureMapping->CachedSizeX-1,CopyY-1) * 2.0f - (*ShadowMapData)(TextureMapping->CachedSizeX-2,CopyY-1);
				}
			}

			// Copy it back in
			ShadowMaps.Add(Key, TempShadowMapData);
			delete ShadowMapData;
		}


		FSignedDistanceFieldShadowSample DebugDistanceShadowSample;
		DebugDistanceShadowSample.bIsMapped = true;
		DebugDistanceShadowSample.Distance = .5f;
		for (TMap<const FLight*, FSignedDistanceFieldShadowMapData2D*>::TIterator It(SignedDistanceFieldShadowMaps); It; ++It)
		{
			const FLight* Key = It.Key();
			FSignedDistanceFieldShadowMapData2D* ShadowMapData = It.Value();
			FSignedDistanceFieldShadowMapData2D* TempShadowMapData = new FSignedDistanceFieldShadowMapData2D(TrueSizeX, TrueSizeY);

			// Expand it
			for (uint32 CopyY = 0; CopyY < TrueSizeY; CopyY++)
			{
				if (CopyY == 0)
				{
					// The first row, left corner
					(*TempShadowMapData)(0,0) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugDistanceShadowSample : (*ShadowMapData)(0,0);
					// The rest of the row, short of the right corner
					for (uint32 TempX = 0; TempX < (uint32)(TextureMapping->CachedSizeX); TempX++)
					{
						// Extrapolate the padding texels, maintaining the same slope that the source data had, which is important for distance field shadows
						(*TempShadowMapData)(TempX+1,0) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugDistanceShadowSample : (*ShadowMapData)(TempX,0) * 2.0f - (*ShadowMapData)(TempX,1);
					}
					// The right corner
					(*TempShadowMapData)(TrueSizeX-1,0) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugDistanceShadowSample : (*ShadowMapData)(TextureMapping->CachedSizeX-1,0);
				}
				else if (CopyY == TrueSizeY - 1)
				{
					// The last row, left corner
					(*TempShadowMapData)(0,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugDistanceShadowSample : (*ShadowMapData)(0,TextureMapping->CachedSizeY-1);
					// The rest of the row, short of the right corner
					for (uint32 TempX = 0; TempX < (uint32)(TextureMapping->CachedSizeX); TempX++)
					{
						(*TempShadowMapData)(TempX+1,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugDistanceShadowSample : (*ShadowMapData)(TempX,TextureMapping->CachedSizeY-1) * 2.0f - (*ShadowMapData)(TempX,TextureMapping->CachedSizeY-2);
					}
					// The right corner
					(*TempShadowMapData)(TrueSizeX-1,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugDistanceShadowSample : (*ShadowMapData)(TextureMapping->CachedSizeX-1,TextureMapping->CachedSizeY-1);
				}
				else
				{
					// The last row, left corner
					(*TempShadowMapData)(0,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugDistanceShadowSample : (*ShadowMapData)(0,CopyY-1) * 2.0f - (*ShadowMapData)(1,CopyY-1);
					// The rest of the row, short of the right corner
					for (uint32 TempX = 0; TempX < (uint32)(TextureMapping->CachedSizeX); TempX++)
					{
						(*TempShadowMapData)(TempX+1,CopyY) = (*ShadowMapData)(TempX,CopyY-1);
					}
					// The right corner
					(*TempShadowMapData)(TrueSizeX-1,CopyY) = FStaticLightingMapping::s_bShowLightmapBorders ? DebugDistanceShadowSample : (*ShadowMapData)(TextureMapping->CachedSizeX-1,CopyY-1) * 2.0f - (*ShadowMapData)(TextureMapping->CachedSizeX-2,CopyY-1);
				}
			}

			// Copy it back in
			SignedDistanceFieldShadowMaps.Add(Key, TempShadowMapData);
			delete ShadowMapData;
		}
	}
	else
	{
		PaddedLightMapData = LightMapData;
	}
}

/** Rasterizes Mesh into TexelToCornersMap */
void FStaticLightingSystem::CalculateTexelCorners(const FStaticLightingMesh* Mesh, FTexelToCornersMap& TexelToCornersMap, int32 UVIndex, bool bDebugThisMapping) const
{
	static const FVector2f CornerOffsets[NumTexelCorners] = 
	{
		FVector2f(0, 0),
		FVector2f(-1, 0),
		FVector2f(0, -1),
		FVector2f(-1, -1)
	};

	// Rasterize each triangle of the mesh
	for (int32 TriangleIndex = 0; TriangleIndex < Mesh->NumTriangles; TriangleIndex++)
	{
		// Query the mesh for the triangle's vertices.
		FStaticLightingVertex V0;
		FStaticLightingVertex V1;
		FStaticLightingVertex V2;
		int32 TriangleElement;
		Mesh->GetTriangle(TriangleIndex, V0, V1, V2, TriangleElement);

		// Rasterize each triangle offset by the corner offsets
		for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
		{
			FTriangleRasterizer<FTexelCornerRasterPolicy> TexelCornerRasterizer(FTexelCornerRasterPolicy(
				Scene,
				TexelToCornersMap,
				CornerIndex,
				bDebugThisMapping
				));

			TexelCornerRasterizer.DrawTriangle(
				V0,
				V1,
				V2,
				V0.TextureCoordinates[UVIndex] * FVector2f(TexelToCornersMap.GetSizeX(), TexelToCornersMap.GetSizeY()) + CornerOffsets[CornerIndex],
				V1.TextureCoordinates[UVIndex] * FVector2f(TexelToCornersMap.GetSizeX(), TexelToCornersMap.GetSizeY()) + CornerOffsets[CornerIndex],
				V2.TextureCoordinates[UVIndex] * FVector2f(TexelToCornersMap.GetSizeX(), TexelToCornersMap.GetSizeY()) + CornerOffsets[CornerIndex],
				false
				);
		}
	}
}

/** Rasterizes Mesh into TexelToCornersMap, with extra parameters like which material index to rasterize and UV scale and bias. */
void FStaticLightingSystem::CalculateTexelCorners(
	const TArray<int32>& TriangleIndices, 
	const TArray<FStaticLightingVertex>& Vertices, 
	FTexelToCornersMap& TexelToCornersMap, 
	const TArray<int32>& ElementIndices,
	int32 MaterialIndex,
	int32 UVIndex, 
	bool bDebugThisMapping, 
	FVector2f UVBias, 
	FVector2f UVScale) const
{
	static const FVector2f CornerOffsets[NumTexelCorners] = 
	{
		FVector2f(0, 0),
		FVector2f(-1, 0),
		FVector2f(0, -1),
		FVector2f(-1, -1)
	};

	// Rasterize each triangle of the mesh
	for (int32 TriangleIndex = 0; TriangleIndex < TriangleIndices.Num(); TriangleIndex++)
	{
		if (ElementIndices[TriangleIndices[TriangleIndex]] == MaterialIndex)
		{
			const FStaticLightingVertex& V0 = Vertices[TriangleIndices[TriangleIndex] * 3 + 0];
			const FStaticLightingVertex& V1 = Vertices[TriangleIndices[TriangleIndex] * 3 + 1];
			const FStaticLightingVertex& V2 = Vertices[TriangleIndices[TriangleIndex] * 3 + 2];

			// Rasterize each triangle offset by the corner offsets
			for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
			{
				FTriangleRasterizer<FTexelCornerRasterPolicy> TexelCornerRasterizer(FTexelCornerRasterPolicy(
					Scene,
					TexelToCornersMap,
					CornerIndex,
					bDebugThisMapping
					));

				TexelCornerRasterizer.DrawTriangle(
					V0,
					V1,
					V2,
					UVScale * (UVBias + V0.TextureCoordinates[UVIndex]) * FVector2f(TexelToCornersMap.GetSizeX(), TexelToCornersMap.GetSizeY()) + CornerOffsets[CornerIndex],
					UVScale * (UVBias + V1.TextureCoordinates[UVIndex]) * FVector2f(TexelToCornersMap.GetSizeX(), TexelToCornersMap.GetSizeY()) + CornerOffsets[CornerIndex],
					UVScale * (UVBias + V2.TextureCoordinates[UVIndex]) * FVector2f(TexelToCornersMap.GetSizeX(), TexelToCornersMap.GetSizeY()) + CornerOffsets[CornerIndex],
					false
					);
			}
		}
	}
}

FLinearColor FStaticLightingMapping::GetCachedRadiosity(int32 RadiosityBufferIndex, int32 SurfaceCacheIndex) const
{
	return RadiositySurfaceCache[RadiosityBufferIndex][SurfaceCacheIndex];
}

FLinearColor FStaticLightingTextureMapping::GetSurfaceCacheLighting(const FMinimalStaticLightingVertex& Vertex) const
{
	checkSlow(SurfaceCacheSizeX > 0 && SurfaceCacheSizeY > 0);
	// Clamping is necessary since the UV's may be outside the [0, 1) range
	const int32 SurfaceCacheX = FMath::Clamp(FMath::TruncToInt(Vertex.TextureCoordinates[1].X * SurfaceCacheSizeX), 0, SurfaceCacheSizeX - 1);
	const int32 SurfaceCacheY = FMath::Clamp(FMath::TruncToInt(Vertex.TextureCoordinates[1].Y * SurfaceCacheSizeY), 0, SurfaceCacheSizeY - 1);
	const int32 SurfaceCacheIndex = SurfaceCacheY * SurfaceCacheSizeX + SurfaceCacheX;

	FLinearColor Lighting = SurfaceCacheLighting[SurfaceCacheIndex];
	return Lighting;
}

int32 FStaticLightingTextureMapping::GetSurfaceCacheIndex(const struct FMinimalStaticLightingVertex& Vertex) const
{
	checkSlow(SurfaceCacheSizeX > 0 && SurfaceCacheSizeY > 0);
	// Clamping is necessary since the UV's may be outside the [0, 1) range
	const int32 SurfaceCacheX = FMath::Clamp(FMath::TruncToInt(Vertex.TextureCoordinates[1].X * SurfaceCacheSizeX), 0, SurfaceCacheSizeX - 1);
	const int32 SurfaceCacheY = FMath::Clamp(FMath::TruncToInt(Vertex.TextureCoordinates[1].Y * SurfaceCacheSizeY), 0, SurfaceCacheSizeY - 1);
	return SurfaceCacheY * SurfaceCacheSizeX + SurfaceCacheX;
}

}
