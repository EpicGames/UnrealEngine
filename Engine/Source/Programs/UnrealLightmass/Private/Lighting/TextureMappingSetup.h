// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LightingMesh.h"
#include "MeshExport.h"
#include "Raster.h"

namespace Lightmass
{

/** A map from a texel to the world-space surface point which maps the texel. */
struct FTexelToVertex
{
	FVector4f WorldPosition;
	FVector4f WorldTangentX;
	FVector4f WorldTangentY;
	FVector4f WorldTangentZ;
	FVector4f TriangleNormal;

	/** Weight used when combining super sampled attributes and determining if the texel has been mapped. */
	float TotalSampleWeight;

	/** Tracks the max sample weight encountered. */
	float MaxSampleWeight;

	/** World space radius of the texel. */
	float TexelRadius;

	/** Final gathering sample radius, derived from texel radius by adjustment */
	float SampleRadius;

	/** Whether this texel was determined to be intersecting another surface. */
	uint32 bIntersectingSurface : 1;

	uint16 ElementIndex;

	/** Texture coordinates */
	FVector2f TextureCoordinates[MAX_TEXCOORDS];

	/** Create a static lighting vertex to represent the texel. */
	inline FStaticLightingVertex GetVertex() const
	{
		FStaticLightingVertex Vertex;
		Vertex.WorldPosition = WorldPosition;
		Vertex.WorldTangentX = WorldTangentX;
		Vertex.WorldTangentY = WorldTangentY;
		Vertex.WorldTangentZ = WorldTangentZ;
		for( int32 CurCoordIndex = 0; CurCoordIndex < MAX_TEXCOORDS; ++CurCoordIndex )
		{
			Vertex.TextureCoordinates[ CurCoordIndex ] = TextureCoordinates[ CurCoordIndex ];
		}
		return Vertex;
	}

	inline FFullStaticLightingVertex GetFullVertex() const
	{
		FFullStaticLightingVertex Vertex;
		(FStaticLightingVertex&)Vertex = GetVertex();
		Vertex.TriangleNormal = TriangleNormal;
		Vertex.GenerateTriangleTangents();
		return Vertex;
	}
};

/** A map from light-map texels to the world-space surface points which map the texels. */
class FTexelToVertexMap
{
public:

	/** Initialization constructor. */
	FTexelToVertexMap(int32 InSizeX,int32 InSizeY):
		SizeX(InSizeX),
		SizeY(InSizeY)
	{
		// Clear the map to zero.
		Data.AddZeroed(SizeX * SizeY);
	}

	// Accessors.
	FTexelToVertex& operator()(int32 X,int32 Y)
	{
		const uint32 TexelIndex = Y * SizeX + X;
		return Data[TexelIndex];
	}
	const FTexelToVertex& operator()(int32 X,int32 Y) const
	{
		const int32 TexelIndex = Y * SizeX + X;
		return Data[TexelIndex];
	}

	int32 GetSizeX() const { return SizeX; }
	int32 GetSizeY() const { return SizeY; }
	SIZE_T GetAllocatedSize() const { return Data.GetAllocatedSize(); }

private:

	/** The mapping data. */
	TArray<FTexelToVertex> Data;

	/** The width of the mapping data. */
	int32 SizeX;

	/** The height of the mapping data. */
	int32 SizeY;
};

struct FStaticLightingInterpolant
{
	FStaticLightingVertex Vertex;
	uint16 ElementIndex;

	FStaticLightingInterpolant() {}

	FStaticLightingInterpolant(const FStaticLightingVertex& InVertex, uint16 InElementIndex) :
		Vertex(InVertex),
		ElementIndex(InElementIndex)
	{}

	// Operators used for linear combinations of static lighting interpolants.
	friend FStaticLightingInterpolant operator+(const FStaticLightingInterpolant& A,const FStaticLightingInterpolant& B)
	{
		FStaticLightingInterpolant Result;
		Result.Vertex = A.Vertex + B.Vertex; 
		Result.ElementIndex = A.ElementIndex;
		return Result;
	}

	friend FStaticLightingInterpolant operator-(const FStaticLightingInterpolant& A,const FStaticLightingInterpolant& B)
	{
		FStaticLightingInterpolant Result;
		Result.Vertex = A.Vertex - B.Vertex; 
		Result.ElementIndex = A.ElementIndex;
		return Result;
	}

	friend FStaticLightingInterpolant operator*(const FStaticLightingInterpolant& A,float B)
	{
		FStaticLightingInterpolant Result;
		Result.Vertex = A.Vertex * B; 
		Result.ElementIndex = A.ElementIndex;
		return Result;
	}

	friend FStaticLightingInterpolant operator/(const FStaticLightingInterpolant& A,float B)
	{
		FStaticLightingInterpolant Result;
		Result.Vertex = A.Vertex / B; 
		Result.ElementIndex = A.ElementIndex;
		return Result;
	}
};

/** Used to map static lighting texels to vertices. */
class FStaticLightingRasterPolicy
{
public:

	typedef FStaticLightingInterpolant InterpolantType;

	/** Initialization constructor. */
	FStaticLightingRasterPolicy(
		const FScene& InScene,
		FTexelToVertexMap& InTexelToVertexMap,
		float InSampleWeight,
		const FVector4f& InTriangleNormal,
		bool bInDebugThisMapping,
		bool bInUseMaxWeight
		) :
		Scene(InScene),
		TexelToVertexMap(InTexelToVertexMap),
		SampleWeight(InSampleWeight),
		TriangleNormal(InTriangleNormal),
		bDebugThisMapping(bInDebugThisMapping),
		bUseMaxWeight(bInUseMaxWeight)
	{}

protected:

	// FTriangleRasterizer policy interface.

	int32 GetMinX() const { return 0; }
	int32 GetMaxX() const { return TexelToVertexMap.GetSizeX() - 1; }
	int32 GetMinY() const { return 0; }
	int32 GetMaxY() const { return TexelToVertexMap.GetSizeY() - 1; }

	void ProcessPixel(int32 X,int32 Y,const InterpolantType& Interpolant,bool BackFacing);

private:

	const FScene& Scene;

	/** The texel to vertex map which is being rasterized to. */
	FTexelToVertexMap& TexelToVertexMap;

	/** The weight of the current sample. */
	const float SampleWeight;
	const FVector4f TriangleNormal;
	const bool bDebugThisMapping;
	const bool bUseMaxWeight;
};

}
