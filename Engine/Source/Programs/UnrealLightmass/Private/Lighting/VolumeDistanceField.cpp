// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"
#include "LightingSystem.h"
#include "MonteCarlo.h"

namespace Lightmass
{

/** Prepares for multithreaded generation of VolumeDistanceField. */
void FStaticLightingSystem::BeginCalculateVolumeDistanceField()
{
	DistanceFieldVolumeBounds = Scene.ImportanceBoundingBox;
	if (DistanceFieldVolumeBounds.GetVolume() < KINDA_SMALL_NUMBER)
	{
		DistanceFieldVolumeBounds = AggregateMesh->GetBounds();
	}

	FBox3f UnclampedDistanceFieldVolumeBounds = DistanceFieldVolumeBounds;
	FVector4f DoubleExtent = UnclampedDistanceFieldVolumeBounds.GetExtent() * 2;
	DoubleExtent.X = DoubleExtent.X - FMath::Fmod(DoubleExtent.X, (FVector4f::FReal)VolumeDistanceFieldSettings.VoxelSize) + VolumeDistanceFieldSettings.VoxelSize;
	DoubleExtent.Y = DoubleExtent.Y - FMath::Fmod(DoubleExtent.Y, (FVector4f::FReal)VolumeDistanceFieldSettings.VoxelSize) + VolumeDistanceFieldSettings.VoxelSize;
	DoubleExtent.Z = DoubleExtent.Z - FMath::Fmod(DoubleExtent.Z, (FVector4f::FReal)VolumeDistanceFieldSettings.VoxelSize) + VolumeDistanceFieldSettings.VoxelSize;
	// Round the max up to the next step boundary
	UnclampedDistanceFieldVolumeBounds.Max = UnclampedDistanceFieldVolumeBounds.Min + DoubleExtent;

	const FVector4f VolumeSizes = UnclampedDistanceFieldVolumeBounds.GetExtent() * 2.0f / VolumeDistanceFieldSettings.VoxelSize;
	VolumeSizeX = FMath::TruncToFloat(VolumeSizes.X + DELTA);
	VolumeSizeY = FMath::TruncToFloat(VolumeSizes.Y + DELTA);
	VolumeSizeZ = FMath::TruncToFloat(VolumeSizes.Z + DELTA);

	// Use a float to avoid 32 bit integer overflow with large volumes
	const float NumVoxels = VolumeSizeX * VolumeSizeY * VolumeSizeZ;

	if (NumVoxels > VolumeDistanceFieldSettings.MaxVoxels)
	{
		const int32 OldSizeX = VolumeSizeX;
		const int32 OldSizeY = VolumeSizeY;
		const int32 OldSizeZ = VolumeSizeZ;
		const float SingleDimensionScale = FMath::Pow(NumVoxels / VolumeDistanceFieldSettings.MaxVoxels, 1.0f / 3.0f);
		DistanceFieldVoxelSize = VolumeDistanceFieldSettings.VoxelSize * SingleDimensionScale;

		DoubleExtent = DistanceFieldVolumeBounds.GetExtent() * 2;
		DoubleExtent.X = DoubleExtent.X - FMath::Fmod(DoubleExtent.X, (FVector4f::FReal)DistanceFieldVoxelSize) + DistanceFieldVoxelSize;
		DoubleExtent.Y = DoubleExtent.Y - FMath::Fmod(DoubleExtent.Y, (FVector4f::FReal)DistanceFieldVoxelSize) + DistanceFieldVoxelSize;
		DoubleExtent.Z = DoubleExtent.Z - FMath::Fmod(DoubleExtent.Z, (FVector4f::FReal)DistanceFieldVoxelSize) + DistanceFieldVoxelSize;
		// Round the max up to the next step boundary with the clamped voxel size
		DistanceFieldVolumeBounds.Max = DistanceFieldVolumeBounds.Min + DoubleExtent;

		const FVector4f ClampedVolumeSizes = DistanceFieldVolumeBounds.GetExtent() * 2.0f / DistanceFieldVoxelSize;
		VolumeSizeX = FMath::TruncToFloat(ClampedVolumeSizes.X + DELTA);
		VolumeSizeY = FMath::TruncToFloat(ClampedVolumeSizes.Y + DELTA);
		VolumeSizeZ = FMath::TruncToFloat(ClampedVolumeSizes.Z + DELTA);
		
		LogSolverMessage(FString::Printf(TEXT("CalculateVolumeDistanceField %ux%ux%u, clamped to %ux%ux%u"), OldSizeX, OldSizeY, OldSizeZ, VolumeSizeX, VolumeSizeY, VolumeSizeZ));
	}
	else 
	{
		DistanceFieldVolumeBounds = UnclampedDistanceFieldVolumeBounds;
		DistanceFieldVoxelSize = VolumeDistanceFieldSettings.VoxelSize;
		LogSolverMessage(FString::Printf(TEXT("CalculateVolumeDistanceField %ux%ux%u"), VolumeSizeX, VolumeSizeY, VolumeSizeZ));
	}

	VolumeDistanceField.Empty(VolumeSizeX * VolumeSizeY * VolumeSizeZ);
	VolumeDistanceField.AddZeroed(VolumeSizeX * VolumeSizeY * VolumeSizeZ);

	FPlatformAtomics::InterlockedExchange(&NumOutstandingVolumeDataLayers, VolumeSizeZ);
}

/** Generates a single z layer of VolumeDistanceField. */
void FStaticLightingSystem::CalculateVolumeDistanceFieldWorkRange(int32 ZIndex)
{
	const double StartTime = FPlatformTime::Seconds();
	FStaticLightingMappingContext MappingContext(NULL, *this);

	TArray<FVector4f> SampleDirections;
	TArray<FVector2f> SampleDirectionUniforms;
	const int32 NumThetaSteps = FMath::TruncToInt(FMath::Sqrt(VolumeDistanceFieldSettings.NumVoxelDistanceSamples / (2.0f * (float)PI)));
	const int32 NumPhiSteps = FMath::TruncToInt(NumThetaSteps * (float)PI);
	FLMRandomStream RandomStream(0);
	GenerateStratifiedUniformHemisphereSamples(NumThetaSteps, NumPhiSteps, RandomStream, SampleDirections, SampleDirectionUniforms);
	TArray<FVector4f> OtherHemisphereSamples;
	TArray<FVector2f> OtherSampleDirectionUniforms;
	GenerateStratifiedUniformHemisphereSamples(NumThetaSteps, NumPhiSteps, RandomStream, OtherHemisphereSamples, OtherSampleDirectionUniforms);

	for (int32 i = 0; i < OtherHemisphereSamples.Num(); i++)
	{
		FVector4f Sample = OtherHemisphereSamples[i];
		Sample.Z *= -1;
		SampleDirections.Add(Sample);
	}

	const FVector4f CellExtents = FVector4f(DistanceFieldVoxelSize / 2, DistanceFieldVoxelSize / 2, DistanceFieldVoxelSize / 2);
	for (int32 YIndex = 0; YIndex < VolumeSizeY; YIndex++)
	{
		for (int32 XIndex = 0; XIndex < VolumeSizeX; XIndex++)
		{
			const FVector4f VoxelPosition = FVector4f(XIndex, YIndex, ZIndex) * DistanceFieldVoxelSize + DistanceFieldVolumeBounds.Min + CellExtents;
			const int32 Index = ZIndex * VolumeSizeY * VolumeSizeX + YIndex * VolumeSizeX + XIndex;
	
			float MinDistance[2];
			MinDistance[0] = FLT_MAX;
			MinDistance[1] = FLT_MAX;

			int32 Hit[2];
			Hit[0] = 0;
			Hit[1] = 0;

			int32 HitFront[2];
			HitFront[0] = 0;
			HitFront[1] = 0;

			// Generate two distance fields
			// The first is for mostly horizontal triangles, the second is for mostly vertical triangles
			// Keeping them separate allows reconstructing a cleaner surface,
			// Otherwise there would be holes in the surface where an unclosed wall mesh intersects an unclosed ground mesh
			for (int32 i = 0; i < 2; i++)
			{
				for (int32 SampleIndex = 0; SampleIndex < SampleDirections.Num(); SampleIndex++)
				{
					const float ExtentDistance = DistanceFieldVolumeBounds.GetExtent().GetMax() * 2.0f;
					FLightRay Ray(
						VoxelPosition,
						VoxelPosition + SampleDirections[SampleIndex] * VolumeDistanceFieldSettings.VolumeMaxDistance,
						NULL,
						NULL
						);

					// Trace rays in all directions to find the closest solid surface
					FLightRayIntersection Intersection;
					AggregateMesh->IntersectLightRay(Ray, true, false, false, MappingContext.RayCache, Intersection);

					if (Intersection.bIntersects)
					{
						if ((i == 0 && FMath::Abs(Intersection.IntersectionVertex.WorldTangentZ.Z) >= .707f) || (i == 1 && FMath::Abs(Intersection.IntersectionVertex.WorldTangentZ.Z) < .707f))
						{
							Hit[i]++;
							if (Dot3(Ray.Direction, Intersection.IntersectionVertex.WorldTangentZ) < 0)
							{
								HitFront[i]++;
							}

							const float CurrentDistance = (VoxelPosition - Intersection.IntersectionVertex.WorldPosition).Size3();
							if (CurrentDistance < MinDistance[i])
							{
								MinDistance[i] = CurrentDistance;
							}
						}
					}
				}
				// Consider this voxel 'outside' an object if more than 75% of the rays hit front faces
				MinDistance[i] *= (Hit[i] == 0 || HitFront[i] > Hit[i] * .75f) ? 1 : -1;
			}
			
			// Create a mask storing where an intersection can possibly take place
			// This allows the reconstruction to ignore areas where large positive and negative distances come together,
			// Which is caused by unclosed surfaces.
			const uint8 Mask0 = FMath::Abs(MinDistance[0]) < DistanceFieldVoxelSize * 2 ? 255 : 0; 
			// 0 will be -MaxDistance, .5 will be 0, 1 will be +MaxDistance
			const float NormalizedDistance0 = FMath::Clamp(MinDistance[0] / VolumeDistanceFieldSettings.VolumeMaxDistance + .5f, 0.0f, 1.0f);

			const uint8 Mask1 = FMath::Abs(MinDistance[1]) < DistanceFieldVoxelSize * 2 ? 255 : 0; 
			const float NormalizedDistance1 = FMath::Clamp(MinDistance[1] / VolumeDistanceFieldSettings.VolumeMaxDistance + .5f, 0.0f, 1.0f);

			const FColor FinalValue(
				FMath::Clamp<uint8>(FMath::TruncToInt(NormalizedDistance0 * 255), 0, 255), 
				FMath::Clamp<uint8>(FMath::TruncToInt(NormalizedDistance1 * 255), 0, 255), 
				Mask0,
				Mask1
				);

			VolumeDistanceField[Index] = FinalValue;
		}
	}
	MappingContext.Stats.VolumeDistanceFieldThreadTime = FPlatformTime::Seconds() - StartTime;
}

}
