// Copyright Epic Games, Inc. All Rights Reserved.

#include "LMCore.h"
#include "Misc/Guid.h"
#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformTime.h"
#include "UnrealLightmass.h"
#include "Misc/Paths.h"
#include "Templates/UniquePtr.h"

namespace Lightmass
{

/*-----------------------------------------------------------------------------
	Logging functionality
-----------------------------------------------------------------------------*/

FLightmassLog::FLightmassLog()
{
	// Create a Guid for this run.
	FGuid Guid = FGuid::NewGuid();

	// get the app name to base the log name off of
	FString ExeName = FPlatformProcess::ExecutableName();
	ExeName = ExeName.Replace(TEXT("\\"), TEXT("/"));
	// Extract filename part and add "-[guid].log"
	int32 ExeNameLen = ExeName.Len();
	int32 PathSeparatorPos = ExeNameLen - 1;
	while ( PathSeparatorPos >= 0 && ExeName[PathSeparatorPos] != TEXT('/') )
	{
		PathSeparatorPos--;
	}
#if PLATFORM_MAC
	Filename = FPaths::Combine( FPlatformProcess::UserLogsDir(), *FString( ExeNameLen - PathSeparatorPos, *ExeName + PathSeparatorPos + 1 ) );
#else
	Filename = FString( ExeNameLen - PathSeparatorPos - 5, *ExeName + PathSeparatorPos + 1 );
#endif
	Filename += TEXT("_");
	Filename += FPlatformProcess::ComputerName();
	Filename += TEXT("_");
	Filename += Guid.ToString();
	Filename += TEXT(".log");

	// open the file for writing
	File = IFileManager::Get().CreateFileWriter(*Filename);

	// mark the file to be unicode
	if (File != NULL)
	{
		uint16 UnicodeBOM = 0xfeff;
		(*File) << UnicodeBOM;
	}
	else
	{
		// print to the screen that we failed to open the file
		printf("\nFailed to open the log file '%s' for writing\n\n", TCHAR_TO_UTF8(*Filename));
	}
}

FLightmassLog::~FLightmassLog()
{
	delete File;
}


void FLightmassLog::Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	// write it out to disk
	if (File != NULL)
	{
		File->Serialize((void*)V, FCString::Strlen(V) * sizeof(TCHAR));
		File->Serialize((void*)TEXT("\r\n"), 2 * sizeof(TCHAR));
	}

	// also print it to the screen and debugger output
#if PLATFORM_USE_LS_SPEC_FOR_WIDECHAR
	printf("%s\n", TCHAR_TO_UTF8(V));
#else
	wprintf(TEXT("%s\n"), V);
#endif
	fflush( stdout );

	// FUnixPlatformMisc::LowLevelOutputDebugString does a "fprintf(stderr..."
	//  Don't print these messages twice when running under debugger on Linux.
#if !PLATFORM_LINUX
	if( FPlatformMisc::IsDebuggerPresent() )
	{
		FPlatformMisc::LowLevelOutputDebugString( V );
		FPlatformMisc::LowLevelOutputDebugString( TEXT("\n") );
		fflush( stderr );
	}
#endif
}


void FLightmassLog::Flush()
{
	File->Flush();
}

static TUniquePtr<FLightmassLog>	GScopedLog;

FLightmassLog* FLightmassLog::Get()
{
	static FLightmassLog LogInstance;
	return &LogInstance;
}

/** CPU frequency for stats, only used for inner loop timing with rdtsc. */
double GCPUFrequency = 3000000000.0;

/** Number of CPU clock cycles per second (as counted by __rdtsc). */
double GSecondPerCPUCycle = 1.0 / 3000000000.0;

struct FInitCPUFrequency
{
	uint64 StartCPUTime;
	uint64 EndCPUTime;
	double StartTime;
	double EndTime;
};
static FInitCPUFrequency GInitCPUFrequency;

/** Start initializing CPU frequency (as counted by __rdtsc). */
void StartInitCPUFrequency()
{
	GInitCPUFrequency.StartTime		= FPlatformTime::Seconds();
	GInitCPUFrequency.StartCPUTime	= __rdtsc();
}

/** Finish initializing CPU frequency (as counted by __rdtsc), and set up CPUFrequency and CPUCyclesPerSecond. */
void FinishInitCPUFrequency()
{
	GInitCPUFrequency.EndTime		= FPlatformTime::Seconds();
	GInitCPUFrequency.EndCPUTime	= __rdtsc();
	double NumSeconds				= GInitCPUFrequency.EndTime - GInitCPUFrequency.StartTime;
	GCPUFrequency					= double(GInitCPUFrequency.EndCPUTime - GInitCPUFrequency.StartCPUTime) / NumSeconds;
	GSecondPerCPUCycle				= NumSeconds / double(GInitCPUFrequency.EndCPUTime - GInitCPUFrequency.StartCPUTime);
	UE_LOGF(LogLightmass, Log, "Measured CPU frequency: %.2f GHz", GCPUFrequency/1000000000.0);
}

}


