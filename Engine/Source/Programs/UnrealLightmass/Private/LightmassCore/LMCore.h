// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// recommended that these are included by all files (ie, leave in LMCore.h, and maybe include LMCore.h in LightmassPCH.h)
#include "LMHelpers.h"			// helper #defines, etc
#include "LMQueue.h"			// TQueue implementation
#include "LMMath.h"				// math functions and classes
#include "LMThreading.h"		// threading functionality
#include "LMStats.h"			// Stats

// these can be moved out and just included per .cpp file
#include "LMOctree.h"			// TOctree functionality
#include "LMkDOP.h"				// TkDOP functionality
#include "LMCollision.h"		// Collision functionality


