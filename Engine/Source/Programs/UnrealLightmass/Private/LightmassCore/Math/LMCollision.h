// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Lightmass
{

static bool ClipLineWithBox(const FBox3f& Box, const FVector4f& Start, const FVector4f& End, FVector4f& IntersectedStart, FVector4f& IntersectedEnd)
{
    IntersectedStart = Start;
    IntersectedEnd = End;

    FVector4f Dir;
    float TEdgeOfBox,TLineLength;
    bool StartCulled,EndCulled;
   
    // Bound by neg X
    StartCulled = IntersectedStart.X < Box.Min.X;
    EndCulled = IntersectedEnd.X < Box.Min.X;
    if (StartCulled && EndCulled)
    {
        IntersectedStart = Start;
        IntersectedEnd = Start;
        return false;
    }
    else if (StartCulled)
    {
        check(IntersectedEnd.X > IntersectedStart.X); // div by 0 should be impossible by check above

        Dir = IntersectedStart - IntersectedEnd;
        TEdgeOfBox = Box.Min.X - IntersectedEnd.X;
        TLineLength = IntersectedStart.X - IntersectedEnd.X;
        IntersectedStart = IntersectedEnd + Dir*(TEdgeOfBox/TLineLength);
    }
    else if (EndCulled)
    {
        check(IntersectedStart.X > IntersectedEnd.X); // div by 0 should be impossible by check above

        Dir = IntersectedEnd - IntersectedStart;
        TEdgeOfBox = Box.Min.X - IntersectedStart.X;
        TLineLength = IntersectedEnd.X - IntersectedStart.X;
        IntersectedEnd = IntersectedStart + Dir*(TEdgeOfBox/TLineLength);
    }

    // Bound by pos X
    StartCulled = IntersectedStart.X > Box.Max.X;
    EndCulled = IntersectedEnd.X > Box.Max.X;
    if (StartCulled && EndCulled)
    {
        IntersectedStart = Start;
        IntersectedEnd = Start;
        return false;
    }
    else if (StartCulled)
    {
        check(IntersectedEnd.X < IntersectedStart.X); // div by 0 should be impossible by check above

        Dir = IntersectedStart - IntersectedEnd;
        TEdgeOfBox = Box.Max.X - IntersectedEnd.X;
        TLineLength = IntersectedStart.X - IntersectedEnd.X;
        IntersectedStart = IntersectedEnd + Dir*(TEdgeOfBox/TLineLength);
    }
    else if (EndCulled)
    {
        check(IntersectedStart.X < IntersectedEnd.X); // div by 0 should be impossible by check above

        Dir = IntersectedEnd - IntersectedStart;
        TEdgeOfBox = Box.Max.X - IntersectedStart.X;
        TLineLength = IntersectedEnd.X - IntersectedStart.X;
        IntersectedEnd = IntersectedStart + Dir*(TEdgeOfBox/TLineLength);
    }

    // Bound by neg Y
    StartCulled = IntersectedStart.Y < Box.Min.Y;
    EndCulled = IntersectedEnd.Y < Box.Min.Y;
    if (StartCulled && EndCulled)
    {
        IntersectedStart = Start;
        IntersectedEnd = Start;
        return false;
    }
    else if (StartCulled)
    {
        check(IntersectedEnd.Y > IntersectedStart.Y); // div by 0 should be impossible by check above

        Dir = IntersectedStart - IntersectedEnd;
        TEdgeOfBox = Box.Min.Y - IntersectedEnd.Y;
        TLineLength = IntersectedStart.Y - IntersectedEnd.Y;
        IntersectedStart = IntersectedEnd + Dir*(TEdgeOfBox/TLineLength);
    }
    else if (EndCulled)
    {
        check(IntersectedStart.Y > IntersectedEnd.Y); // div by 0 should be impossible by check above

        Dir = IntersectedEnd - IntersectedStart;
        TEdgeOfBox = Box.Min.Y - IntersectedStart.Y;
        TLineLength = IntersectedEnd.Y - IntersectedStart.Y;
        IntersectedEnd = IntersectedStart + Dir*(TEdgeOfBox/TLineLength);
    }

    // Bound by pos Y
    StartCulled = IntersectedStart.Y > Box.Max.Y;
    EndCulled = IntersectedEnd.Y > Box.Max.Y;
    if (StartCulled && EndCulled)
    {
        IntersectedStart = Start;
        IntersectedEnd = Start;
        return false;
    }
    else if (StartCulled)
    {
        check(IntersectedEnd.Y < IntersectedStart.Y); // div by 0 should be impossible by check above

        Dir = IntersectedStart - IntersectedEnd;
        TEdgeOfBox = Box.Max.Y - IntersectedEnd.Y;
        TLineLength = IntersectedStart.Y - IntersectedEnd.Y;
        IntersectedStart = IntersectedEnd + Dir*(TEdgeOfBox/TLineLength);
    }
    else if (EndCulled)
    {
        check(IntersectedStart.Y < IntersectedEnd.Y); // div by 0 should be impossible by check above

        Dir = IntersectedEnd - IntersectedStart;
        TEdgeOfBox = Box.Max.Y - IntersectedStart.Y;
        TLineLength = IntersectedEnd.Y - IntersectedStart.Y;
        IntersectedEnd = IntersectedStart + Dir*(TEdgeOfBox/TLineLength);
    }

    // Bound by neg Z
    StartCulled = IntersectedStart.Z < Box.Min.Z;
    EndCulled = IntersectedEnd.Z < Box.Min.Z;
    if (StartCulled && EndCulled)
    {
        IntersectedStart = Start;
        IntersectedEnd = Start;
        return false;
    }
    else if (StartCulled)
    {
        check(IntersectedEnd.Z > IntersectedStart.Z); // div by 0 should be impossible by check above

        Dir = IntersectedStart - IntersectedEnd;
        TEdgeOfBox = Box.Min.Z - IntersectedEnd.Z;
        TLineLength = IntersectedStart.Z - IntersectedEnd.Z;
        IntersectedStart = IntersectedEnd + Dir*(TEdgeOfBox/TLineLength);
    }
    else if (EndCulled)
    {
        check(IntersectedStart.Z > IntersectedEnd.Z); // div by 0 should be impossible by check above

        Dir = IntersectedEnd - IntersectedStart;
        TEdgeOfBox = Box.Min.Z - IntersectedStart.Z;
        TLineLength = IntersectedEnd.Z - IntersectedStart.Z;
        IntersectedEnd = IntersectedStart + Dir*(TEdgeOfBox/TLineLength);
    }

    // Bound by pos Z
    StartCulled = IntersectedStart.Z > Box.Max.Z;
    EndCulled = IntersectedEnd.Z > Box.Max.Z;
    if (StartCulled && EndCulled)
    {
        IntersectedStart = Start;
        IntersectedEnd = Start;
        return false;
    }
    else if (StartCulled)
    {
        check(IntersectedEnd.Z < IntersectedStart.Z); // div by 0 should be impossible by check above

        Dir = IntersectedStart - IntersectedEnd;
        TEdgeOfBox = Box.Max.Z - IntersectedEnd.Z;
        TLineLength = IntersectedStart.Z - IntersectedEnd.Z;
        IntersectedStart = IntersectedEnd + Dir*(TEdgeOfBox/TLineLength);
    }
    else if (EndCulled)
    {
        check(IntersectedStart.Z < IntersectedEnd.Z); // div by 0 should be impossible by check above

        Dir = IntersectedEnd - IntersectedStart;
        TEdgeOfBox = Box.Max.Z - IntersectedStart.Z;
        TLineLength = IntersectedEnd.Z - IntersectedStart.Z;
        IntersectedEnd = IntersectedStart + Dir*(TEdgeOfBox/TLineLength);
    }
    return true;
}

} // namespace Lightmass