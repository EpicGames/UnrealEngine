// Copyright Epic Games, Inc. All Rights Reserved.

#include "LMMath.h"
#include "LMCore.h"

namespace Lightmass
{


/** Converts a linear space XYZ color to linear space RGB. */
FLinearColor FLinearColorUtils::XYZToLinearRGB(const FLinearColor& InColor)
{
	FLinearColor SourceXYZ(InColor);
	// Inverse of the transform in FLinearColor::LinearRGBToXYZ()
	const FMatrix44f XYZToRGB(
		FVector3f(3.2404548f, -0.9692664f, 0.0556434f),
		FVector3f(-1.5371389f, 1.8760109f, -0.2040259f),
		FVector3f(-0.4985315f, 0.0415561f, 1.0572252f),
		FVector3f(0,			  0,		  0)); 

	if (FMath::IsNearlyEqual(InColor.R, 0.0f, (float)SMALL_NUMBER) && FMath::IsNearlyEqual(InColor.B, 0.0f, (float)SMALL_NUMBER))
	{
		SourceXYZ.G = 0.0f;
	}
	const FVector4f LinearRGB = XYZToRGB.TransformVector(FVector3f(SourceXYZ.R, SourceXYZ.G, SourceXYZ.B));
	return FLinearColor(FMath::Max(LinearRGB.X, 0.0f), FMath::Max(LinearRGB.Y, 0.0f), FMath::Max(LinearRGB.Z, 0.0f));
}

/** Converts an XYZ color to xyzY, where xy and z are chrominance measures and Y is the brightness. */
FLinearColor FLinearColorUtils::XYZToxyzY(const FLinearColor& InColor)
{
	const float InvTotal = 1.0f / FMath::Max(InColor.R + InColor.G + InColor.B, (float)SMALL_NUMBER);
	return FLinearColor(InColor.R * InvTotal, InColor.G * InvTotal, InColor.B * InvTotal, InColor.G);
}

/** Converts an xyzY color to XYZ. */
FLinearColor FLinearColorUtils::xyzYToXYZ(const FLinearColor& InColor)
{
	const float yInverse = 1.0f / FMath::Max(InColor.G, (float)SMALL_NUMBER);
	return FLinearColor(InColor.R * InColor.A * yInverse, InColor.A, InColor.B * InColor.A * yInverse);
}

/** Converts a linear space RGB color to an HSV color */
FLinearColor FLinearColorUtils::LinearRGBToHSV(const FLinearColor& InColor)
{
	const float RGBMin = FMath::Min3(InColor.R, InColor.G, InColor.B);
	const float RGBMax = FMath::Max3(InColor.R, InColor.G, InColor.B);
	const float RGBRange = RGBMax - RGBMin;

	const float Hue = (RGBMax == RGBMin ? 0.0f :
					   RGBMax == InColor.R    ? FMath::Fmod((((InColor.G - InColor.B) / RGBRange) * 60.0f) + 360.0f, 360.0f) :
					   RGBMax == InColor.G    ?             (((InColor.B - InColor.R) / RGBRange) * 60.0f) + 120.0f :
					   RGBMax == InColor.B    ?             (((InColor.R - InColor.G) / RGBRange) * 60.0f) + 240.0f :
					   0.0f);
	
	const float Saturation = (RGBMax == 0.0f ? 0.0f : RGBRange / RGBMax);
	const float Value = RGBMax;

	// In the new color, R = H, G = S, B = V, A = 1.0
	return FLinearColor(Hue, Saturation, Value);
}

/** Converts an HSV color to a linear space RGB color */
FLinearColor FLinearColorUtils::HSVToLinearRGB(const FLinearColor& InColor)
{
	// In this color, R = H, G = S, B = V
	const float Hue = InColor.R;
	const float Saturation = InColor.G;
	const float Value = InColor.B;

	const float HDiv60 = Hue / 60.0f;
	const float HDiv60_Floor = floorf(HDiv60);
	const float HDiv60_Fraction = HDiv60 - HDiv60_Floor;

	const float RGBValues[4] = {
		Value,
		Value * (1.0f - Saturation),
		Value * (1.0f - (HDiv60_Fraction * Saturation)),
		Value * (1.0f - ((1.0f - HDiv60_Fraction) * Saturation)),
	};
	const uint32 RGBSwizzle[6][3] = {
		{0, 3, 1},
		{2, 0, 1},
		{1, 0, 3},
		{1, 2, 0},
		{3, 1, 0},
		{0, 1, 2},
	};
	const uint32 SwizzleIndex = ((uint32)HDiv60_Floor) % 6;

	return FLinearColor(RGBValues[RGBSwizzle[SwizzleIndex][0]],
						RGBValues[RGBSwizzle[SwizzleIndex][1]],
						RGBValues[RGBSwizzle[SwizzleIndex][2]]);
}

}

namespace Lightmass
{

/** Converts a linear space RGB color to linear space XYZ. */
FLinearColor FLinearColorUtils::LinearRGBToXYZ(const FLinearColor& InColor)
{
	// RGB to XYZ linear transformation used by sRGB
	//http://www.w3.org/Graphics/Color/sRGB
	const FMatrix44f RGBToXYZ(
		FVector3f(0.4124564f, 0.2126729f, 0.0193339f),
		FVector3f(0.3575761f, 0.7151522f, 0.1191920f),
		FVector3f(0.1804375f, 0.0721750f, 0.9503041f),
		FVector3f(0,			 0,			 0)); 

	const FVector4f ResultVector = RGBToXYZ.TransformVector(FVector3f(InColor.R, InColor.G, InColor.B));
	return FLinearColor(ResultVector.X, ResultVector.Y, ResultVector.Z);
}

FLinearColor FLinearColorUtils::AdjustSaturation(const FLinearColor& InColor, float SaturationFactor)
{
	// Convert to HSV space for the saturation adjustment
	FLinearColor HSVColor = LinearRGBToHSV(InColor);

	// Clamp the range to what's expected 
	SaturationFactor = FMath::Clamp(SaturationFactor, 0.0f, 2.0f);

	if (SaturationFactor < 1.0f)
	{
		HSVColor.G = FMath::Lerp(0.0f, HSVColor.G, SaturationFactor);
	}
	else
	{
		HSVColor.G = FMath::Lerp(HSVColor.G, 1.0f, SaturationFactor - 1.0f);
	}

	// Convert back to linear RGB
	return HSVColor.HSVToLinearRGB();
}

bool GetBarycentricWeights(
	const FVector4f& Position0,
	const FVector4f& Position1,
	const FVector4f& Position2,
	const FVector4f& InterpolatePosition,
	float Tolerance,
	FVector4f& BarycentricWeights
	)
{
	BarycentricWeights = FVector4f(0,0,0);
	FVector4f TriangleNormal = (Position0 - Position1) ^ (Position2 - Position0);
	float ParallelogramArea = TriangleNormal.Size3();
	FVector4f UnitTriangleNormal = TriangleNormal / ParallelogramArea;
	float PlaneDistance = Dot3(UnitTriangleNormal, (InterpolatePosition - Position0));

	// Only continue if the position to interpolate to is in the plane of the triangle (within some error)
	if (FMath::Abs(PlaneDistance) < Tolerance)
	{
		// Move the position to interpolate to into the plane of the triangle along the normal, 
		// Otherwise there will be error in our barycentric coordinates
		FVector4f AdjustedInterpolatePosition = InterpolatePosition - UnitTriangleNormal * PlaneDistance;

		FVector4f NormalU = (AdjustedInterpolatePosition - Position1) ^ (Position2 - AdjustedInterpolatePosition);
		// Signed area, if negative then InterpolatePosition is not in the triangle
		float ParallelogramAreaU = NormalU.Size3() * (Dot3(NormalU, TriangleNormal) > 0.0f ? 1.0f : -1.0f);
		float BaryCentricU = ParallelogramAreaU / ParallelogramArea;

		FVector4f NormalV = (AdjustedInterpolatePosition - Position2) ^ (Position0 - AdjustedInterpolatePosition);
		float ParallelogramAreaV = NormalV.Size3() * (Dot3(NormalV, TriangleNormal) > 0.0f ? 1.0f : -1.0f);
		float BaryCentricV = ParallelogramAreaV / ParallelogramArea;

		float BaryCentricW = 1.0f - BaryCentricU - BaryCentricV;
		if (BaryCentricU > -Tolerance && BaryCentricV > -Tolerance && BaryCentricW > -Tolerance)
		{
			BarycentricWeights = FVector4f(BaryCentricU, BaryCentricV, BaryCentricW);
			return true;
		}
	}
	return false;
}


}

