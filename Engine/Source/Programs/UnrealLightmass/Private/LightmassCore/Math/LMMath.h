// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define USE_SSE2_FOR_MERSENNE_TWISTER 1
#if USE_SSE2_FOR_MERSENNE_TWISTER
	#include <emmintrin.h>
#endif

namespace Lightmass
{

// always use SSE for now
#include "LMMathSSE.h"

bool GetBarycentricWeights(
	const FVector4f& Position0,
	const FVector4f& Position1,
	const FVector4f& Position2,
	const FVector4f& InterpolatePosition,
	float Tolerance,
	FVector4f& BarycentricWeights
	);


/** Types used by SFMT */
typedef unsigned int uint32_t;
#if PLATFORM_WINDOWS
typedef unsigned __int64 uint64_t;
#endif

/** Mersenne Exponent. The period of the sequence 
 *  is a multiple of 2^MEXP-1.
 * #define MEXP 19937 */
static const uint64_t MEXP = 19937;
/** SFMT generator has an internal state array of 128-bit integers,
 * and N is its size. */
static const uint64_t N = (MEXP / 128 + 1);
/** N32 is the size of internal state array when regarded as an array
 * of 32-bit integers.*/
static const uint64_t N32 = (N * 4);
/** N64 is the size of internal state array when regarded as an array
 * of 64-bit integers.*/
static const uint64_t N64 = (N * 2);

/*------------------------------------------------------
  128-bit SIMD data type for SSE2 or standard C
  ------------------------------------------------------*/

#if USE_SSE2_FOR_MERSENNE_TWISTER

/** 128-bit data structure */
union W128_T {
	__m128i si;
	uint32_t u[4];
};
/** 128-bit data type */
typedef union W128_T w128_t;

#else

/** 128-bit data structure */
struct W128_T {
	uint32_t u[4];
};
/** 128-bit data type */
typedef struct W128_T w128_t;

#endif

/** Value returned by appTrunc if the converted result is larger than the maximum signed 32bit integer. */
static const int32 appTruncErrorCode = 0x80000000;

/**
 * Converts float to int via truncation.
 */
inline int32 appTrunc( float F )
{
	return LmVectorTruncate( LmVectorSetFloat1( F ) );
}

/** Thread-safe Random Number Generator which wraps the SIMD-oriented Fast Mersenne Twister (SFMT). */
class FLMRandomStream
{
public:

	FLMRandomStream(int32 InSeed) :
		initialized(0)
	{
		psfmt32 = &sfmt[0].u[0];
		psfmt64 = (uint64_t *)&sfmt[0].u[0];
		init_gen_rand(InSeed);
	}

	/** 
	 * Generates a uniformly distributed pseudo-random float in the range [0,1).
	 * This is implemented with the Mersenne Twister and has excellent precision and distribution properties
	 */
	inline float GetFraction()
	{
		float NewFraction;
		do 
		{
			NewFraction = genrand_res53();
			// The comment for genrand_res53 says it returns a real number in the range [0,1), but in practice it sometimes returns 1,
			// Possibly a result of rounding during the double -> float conversion
		} while (NewFraction >= 1.0f - DELTA);
		return NewFraction;
	}

private:

	/** 
	 * @file SFMT.h 
	 *
	 * @brief SIMD oriented Fast Mersenne Twister(SFMT) pseudorandom
	 * number generator
	 *
	 * @author Mutsuo Saito (Hiroshima University)
	 * @author Makoto Matsumoto (Hiroshima University)
	 *
	 * Copyright (C) 2006, 2007 Mutsuo Saito, Makoto Matsumoto and Hiroshima
	 * University. All rights reserved.
	 *
	 * The new BSD License is applied to this software.
	 * see LICENSE.txt
	 *
	 * @note We assume that your system has inttypes.h.  If your system
	 * doesn't have inttypes.h, you have to typedef uint32_t and uint64_t,
	 * and you have to define PRIu64 and PRIx64 in this file as follows:
	 * @verbatim
	 typedef unsigned int uint32_t
	 typedef unsigned long long uint64_t  
	 #define PRIu64 "llu"
	 #define PRIx64 "llx"
	@endverbatim
	 * uint32_t must be exactly 32-bit unsigned integer type (no more, no
	 * less), and uint64_t must be exactly 64-bit unsigned integer type.
	 * PRIu64 and PRIx64 are used for printf function to print 64-bit
	 * unsigned int and 64-bit unsigned int in hexadecimal format.
	 */

	/** the 128-bit internal state array */
	w128_t sfmt[N];
	/** the 32bit integer pointer to the 128-bit internal state array */
	uint32_t *psfmt32;
#if !defined(BIG_ENDIAN64) || defined(ONLY64)
	/** the 64bit integer pointer to the 128-bit internal state array */
	uint64_t *psfmt64;
#endif
	/** index counter to the 32-bit internal state array */
	int idx;
	/** a flag: it is 0 if and only if the internal state is not yet
	* initialized. */
	int initialized;

	uint32_t gen_rand32(void);
	uint64_t gen_rand64(void);
	void fill_array32(uint32_t *array, int size);
	void fill_array64(uint64_t *array, int size);
	void init_gen_rand(uint32_t seed);
	void init_by_array(uint32_t *init_key, int key_length);
	const char * get_idstring(void);
	int get_min_array_size32(void);
	int get_min_array_size64(void);
	void gen_rand_all(void);
	void gen_rand_array(w128_t *array, int size);
	void period_certification(void);

	/* These real versions are due to Isaku Wada */
	/** generates a random number on [0,1]-real-interval */
	inline double to_real1(uint32_t v)
	{
		return v * (1.0/4294967295.0); 
		/* divided by 2^32-1 */ 
	}

	/** generates a random number on [0,1]-real-interval */
	inline double genrand_real1(void)
	{
		return to_real1(gen_rand32());
	}

	/** generates a random number on [0,1)-real-interval */
	inline double to_real2(uint32_t v)
	{
		return v * (1.0/4294967296.0); 
		/* divided by 2^32 */
	}

	/** generates a random number on [0,1)-real-interval */
	inline double genrand_real2(void)
	{
		return to_real2(gen_rand32());
	}

	/** generates a random number on (0,1)-real-interval */
	inline double to_real3(uint32_t v)
	{
		return (((double)v) + 0.5)*(1.0/4294967296.0); 
		/* divided by 2^32 */
	}

	/** generates a random number on (0,1)-real-interval */
	inline double genrand_real3(void)
	{
		return to_real3(gen_rand32());
	}
	/** These real versions are due to Isaku Wada */

	/** generates a random number on [0,1) with 53-bit resolution*/
	inline double to_res53(uint64_t v) 
	{ 
		return v * (1.0/18446744073709551616.0L);
	}

	/** generates a random number on [0,1) with 53-bit resolution from two
	 * 32 bit integers */
	inline double to_res53_mix(uint32_t x, uint32_t y) 
	{ 
		return to_res53(x | ((uint64_t)y << 32));
	}

	/** generates a random number on [0,1) with 53-bit resolution
	 */
	inline double genrand_res53(void) 
	{ 
		return to_res53(gen_rand64());
	} 

	/** generates a random number on [0,1) with 53-bit resolution
		using 32bit integer.
	 */
	inline double genrand_res53_mix(void) 
	{ 
		uint32_t x, y;

		x = gen_rand32();
		y = gen_rand32();
		return to_res53_mix(x, y);
	} 
};

/*-----------------------------------------------------------------------------
 FFloat32
-----------------------------------------------------------------------------*/
/**
 * 32 bit float components
 */
class FFloat32
{
public:
	union
	{
		struct
		{
//__INTEL_BYTE_ORDER__
			uint32	Mantissa : 23;
			uint32	Exponent : 8;
			uint32	Sign : 1;			
		} Components;

		float	FloatValue;
	};

	FFloat32( float InValue=0.0f )
		:	FloatValue(InValue)
	{}
};

/*-----------------------------------------------------------------------------
 FFloat16
-----------------------------------------------------------------------------*/
/**
 * 16 bit float components and conversion
 *
 *
 * IEEE float 16
 * Represented by 10-bit mantissa M, 5-bit exponent E, and 1-bit sign S
 *
 * Specials:
 *	E=0, M=0		== 0.0
 *	E=0, M!=0		== Denormalized value (M / 2^10) * 2^-14
 *	0<E<31, M=any	== (1 + M / 2^10) * 2^(E-15)
 *	E=31, M=0		== Infinity
 *	E=31, M!=0		== NAN
 */
class FFloat16
{
public:
	union
	{
		struct
		{
//__INTEL_BYTE_ORDER__
			uint16	Mantissa : 10;
			uint16	Exponent : 5;
			uint16	Sign : 1;
		} Components;

		uint16	Encoded;
	};

	/** Default constructor */
	FFloat16( ) :
		Encoded(0)
	{
	}

	/** Copy constructor. */
	FFloat16( const FFloat16& FP16Value )
	{
		Encoded = FP16Value.Encoded;
	}

	/** Conversion constructor. Convert from Fp32 to Fp16. */
	FFloat16( float FP32Value )
	{
		Set( FP32Value );
	}	

	/** Assignment operator. Convert from Fp32 to Fp16. */
	FFloat16& operator=( float FP32Value )
	{
		Set( FP32Value );
		return *this;
	}

	/** Assignment operator. Copy Fp16 value. */
	FFloat16& operator=( const FFloat16& FP16Value )
	{
		Encoded = FP16Value.Encoded;
		return *this;
	}

	/** Convert from Fp16 to Fp32. */
	operator float() const
	{
		return GetFloat();
	}

	/** Convert from Fp32 to Fp16. */
	void Set( float FP32Value )
	{
		FFloat32 FP32(FP32Value);

		// Copy sign-bit
		Components.Sign = FP32.Components.Sign;

		// Check for zero, denormal or too small value.
		if ( FP32.Components.Exponent <= 112 )			// Too small exponent? (0+127-15)
		{
			// Set to 0.
			Components.Exponent = 0;
			Components.Mantissa = 0;
		}
		// Check for INF or NaN, or too high value
		else if ( FP32.Components.Exponent >= 143 )		// Too large exponent? (31+127-15)
		{
			// Set to 65504.0 (max value)
			Components.Exponent = 30;
			Components.Mantissa = 1023;
		}
		// Handle normal number.
		else
		{
			Components.Exponent = int32(FP32.Components.Exponent) - 127 + 15;
			Components.Mantissa = uint16(FP32.Components.Mantissa >> 13);
		}
	}

	/** Convert from Fp16 to Fp32. */
	FORCEINLINE float GetFloat() const
	{
		FFloat32	Result;

		Result.Components.Sign = Components.Sign;
		if (Components.Exponent == 0)
		{
			// Zero or denormal. Just clamp to zero...
			Result.Components.Exponent = 0;
			Result.Components.Mantissa = 0;
		}
		else if (Components.Exponent == 31)		// 2^5 - 1
		{
			// Infinity or NaN. Set to 65504.0
			Result.Components.Exponent = 142;
			Result.Components.Mantissa = 8380416;
		}
		else
		{
			// Normal number.
			Result.Components.Exponent = int32(Components.Exponent) - 15 + 127; // Stored exponents are biased by half their range.
			Result.Components.Mantissa = uint32(Components.Mantissa) << 13;
		}

		return Result.FloatValue;
	}
};

struct FLinearColorUtils
{
	/** Converts a linear space RGB color to linear space XYZ. */
	static FLinearColor LinearRGBToXYZ(const FLinearColor& InColor);

	/** Converts a linear space XYZ color to linear space RGB. */
	static FLinearColor XYZToLinearRGB(const FLinearColor& InColor);

	/** Converts an XYZ color to xyzY, where xy and z are chrominance measures and Y is the brightness. */
	static FLinearColor XYZToxyzY(const FLinearColor& InColor);

	/** Converts an xyzY color to XYZ. */
	static FLinearColor xyzYToXYZ(const FLinearColor& InColor);

	/** Converts a linear space RGB color to an HSV color */
	static FLinearColor LinearRGBToHSV(const FLinearColor& InColor);

	/** Converts an HSV color to a linear space RGB color */
	static FLinearColor HSVToLinearRGB(const FLinearColor& InColor);

	/**
	 * Returns a color with adjusted saturation levels, with valid input from 0.0 to 2.0
	 * 0.0 produces a fully desaturated color
	 * 1.0 produces no change to the saturation
	 * 2.0 produces a fully saturated color
	 *
	 * @param	SaturationFactor	Saturation factor in range [0..2]
	 * @return	Desaturated color
	 */
	static FLinearColor AdjustSaturation(const FLinearColor& InColor, float SaturationFactor);

	static bool AreFloatsValid(const FLinearColor& InColor)
	{
		return FMath::IsFinite(InColor.R) && FMath::IsFinite(InColor.G) && FMath::IsFinite(InColor.B) && FMath::IsFinite(InColor.A)
			&& !FMath::IsNaN(InColor.R) && !FMath::IsNaN(InColor.G) && !FMath::IsNaN(InColor.B) && !FMath::IsNaN(InColor.A);
	}

};

struct FVectorUtils
{
	/** @return >0: point is in front of the plane, <0: behind, =0: on the plane */
	FORCEINLINE static float PlaneDot( const FVector4f &V, const FVector4f &P )
	{
		return V.X*P.X + V.Y*P.Y + V.Z*P.Z - V.W;
	}
};


/**
 * Counts the number of trailing zeros in the bit representation of the value,
 * counting from least-significant bit to most.
 *
 * @param Value the value to determine the number of leading zeros for
 * @return the number of zeros before the first "on" bit
 */
#if PLATFORM_WINDOWS
#pragma intrinsic( _BitScanForward )
FORCEINLINE uint32 appCountTrailingZeros(uint32 Value)
{
	if (Value == 0)
	{
		return 32;
	}
	uint32 BitIndex;	// 0-based, where the LSB is 0 and MSB is 31
	_BitScanForward( (::DWORD *)&BitIndex, Value );	// Scans from LSB to MSB
	return BitIndex;
}
#else // PLATFORM_WINDOWS
FORCEINLINE uint32 appCountTrailingZeros(uint32 Value)
{
	if (Value == 0)
	{
		return 32;
	}
	return __builtin_ffs(Value) - 1;
}
#endif // PLATFORM_WINDOWS

/** Converts spherical coordinates on the unit sphere into a cartesian unit length vector. */
FORCEINLINE FVector4f SphericalToUnitCartesian(const FVector2f& InHemispherical)
{
	const float SinTheta = FMath::Sin(InHemispherical.X);
	return FVector4f(FMath::Cos(InHemispherical.Y) * SinTheta, FMath::Sin(InHemispherical.Y) * SinTheta, FMath::Cos(InHemispherical.X));
}

}
