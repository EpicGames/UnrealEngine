// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/*=============================================================================
 *	Helpers:
 *============================================================================*/

/**
 *	float4 vector register type, where the first float (X) is stored in the lowest 32 bits, and so on.
 */
typedef __m128	LmVectorRegister;

/**
 * @param A0	Selects which component (0-3) from 'A' into 1st slot in the result
 * @param A1	Selects which component (0-3) from 'A' into 2nd slot in the result
 * @param B2	Selects which component (0-3) from 'B' into 3rd slot in the result
 * @param B3	Selects which component (0-3) from 'B' into 4th slot in the result
 */
#define LMSHUFFLEMASK(A0,A1,B2,B3) ( (A0) | ((A1)<<2) | ((B2)<<4) | ((B3)<<6) )

/**
 * Returns a bitwise equivalent vector based on 4 DWORDs.
 *
 * @param X		1st uint32 component
 * @param Y		2nd uint32 component
 * @param Z		3rd uint32 component
 * @param W		4th uint32 component
 * @return		Bitwise equivalent vector with 4 floats
 */
FORCEINLINE LmVectorRegister LmMakeVectorRegister( uint32 X, uint32 Y, uint32 Z, uint32 W )
{
	union { LmVectorRegister v; uint32 f[4]; } Tmp;
	Tmp.f[0] = X;
	Tmp.f[1] = Y;
	Tmp.f[2] = Z;
	Tmp.f[3] = W;
	return Tmp.v;
}

/**
 * Returns a vector based on 4 FLOATs.
 *
 * @param X		1st float component
 * @param Y		2nd float component
 * @param Z		3rd float component
 * @param W		4th float component
 * @return		Vector of the 4 FLOATs
 */
FORCEINLINE LmVectorRegister LmMakeVectorRegister( float X, float Y, float Z, float W )
{
	return _mm_setr_ps( X, Y, Z, W );
}

FORCEINLINE LmVectorRegister LmMakeVectorRegister( double X, double Y, double Z, double W )
{
	return _mm_setr_ps((float)X, (float)Y, (float)Z, (float)W);
}

/** Vector that represents (1,1,1,1) */
static const LmVectorRegister LM_SSE_ONE = LmMakeVectorRegister( 1.0f, 1.0f, 1.0f, 1.0f );

/** Bitmask to AND out the XYZ components in a vector */
static const LmVectorRegister LM_SSE_XYZ_MASK = LmMakeVectorRegister( (uint32)0xffffffff, (uint32)0xffffffff, (uint32)0xffffffff, (uint32)0x00000000 );

/** Bitmask to AND out the sign bit of each components in a vector */
#define LM_SIGN_BIT (~(1 << 31))
static const LmVectorRegister LM_SSE_SIGN_MASK = LmMakeVectorRegister( (uint32)LM_SIGN_BIT, (uint32)LM_SIGN_BIT, (uint32)LM_SIGN_BIT, (uint32)LM_SIGN_BIT );


/*=============================================================================
 *	Intrinsics:
 *============================================================================*/

/**
 * Returns a vector with all zeros.
 *
 * @return		LmVectorRegister(0.0f, 0.0f, 0.0f, 0.0f)
 */
#define LmVectorZero()					_mm_setzero_ps()

/**
 * Returns a vector with all ones.
 *
 * @return		LmVectorRegister(1.0f, 1.0f, 1.0f, 1.0f)
 */
#define LmVectorOne()						LM_SSE_ONE

/**
 * Returns an component from a vector.
 *
 * @param Vec				Vector register
 * @param ComponentIndex	Which component to get, X=0, Y=1, Z=2, W=3
 * @return					The component as a float
 */
#define LmVectorGetComponent( Vec, ComponentIndex )	(((float*) &(Vec))[ComponentIndex])

/**
 * Loads 4 FLOATs from unaligned memory.
 *
 * @param Ptr	Unaligned memory pointer to the 4 FLOATs
 * @return		LmVectorRegister(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
 */
#define LmVectorLoad( Ptr )				_mm_loadu_ps( (float*)(Ptr) )

/**
 * Loads 3 FLOATs from unaligned memory and leaves W undefined.
 *
 * @param Ptr	Unaligned memory pointer to the 3 FLOATs
 * @return		LmVectorRegister(Ptr[0], Ptr[1], Ptr[2], undefined)
 */
#define LmVectorLoadFloat3( Ptr )			LmMakeVectorRegister( ((const float*)(Ptr))[0], ((const float*)(Ptr))[1], ((const float*)(Ptr))[2], 0.0f )

/**
 * Loads 3 FLOATs from unaligned memory and sets W=0.
 *
 * @param Ptr	Unaligned memory pointer to the 3 FLOATs
 * @return		LmVectorRegister(Ptr[0], Ptr[1], Ptr[2], 0.0f)
 */
#define LmVectorLoadFloat3_W0( Ptr )		LmMakeVectorRegister( ((const float*)(Ptr))[0], ((const float*)(Ptr))[1], ((const float*)(Ptr))[2], 0.0f )

/**
 * Loads 3 FLOATs from unaligned memory and sets W=1.
 *
 * @param Ptr	Unaligned memory pointer to the 3 FLOATs
 * @return		LmVectorRegister(Ptr[0], Ptr[1], Ptr[2], 1.0f)
 */
#define LmVectorLoadFloat3_W1( Ptr )		LmMakeVectorRegister( ((const float*)(Ptr))[0], ((const float*)(Ptr))[1], ((const float*)(Ptr))[2], 1.0f )

/**
 * Loads 4 FLOATs from aligned memory.
 *
 * @param Ptr	Aligned memory pointer to the 4 FLOATs
 * @return		LmVectorRegister(Ptr[0], Ptr[1], Ptr[2], Ptr[3])
 */
#define LmVectorLoadAligned( Ptr )		_mm_load_ps( (float*)(Ptr) )

/**
 * Loads 1 float from unaligned memory and replicates it to all 4 components.
 *
 * @param Ptr	Unaligned memory pointer to the float
 * @return		LmVectorRegister(Ptr[0], Ptr[0], Ptr[0], Ptr[0])
 */
#define LmVectorLoadFloat1( Ptr )			_mm_load1_ps( (float*)(Ptr) )

/**
 * Propagates passed in float to all registers
 *
 * @param F		Float to set
 * @return		LmVectorRegister(F,F,F,F)
 */
#define LmVectorSetFloat1( F )	_mm_set1_ps( F )

/**
 * Creates a vector out of three FLOATs and leaves W undefined.
 *
 * @param X		1st float component
 * @param Y		2nd float component
 * @param Z		3rd float component
 * @return		LmVectorRegister(X, Y, Z, undefined)
 */
#define LmVectorSetFloat3( X, Y, Z )		LmMakeVectorRegister( X, Y, Z, 0.0f )

/**
 * Creates a vector out of four FLOATs.
 *
 * @param X		1st float component
 * @param Y		2nd float component
 * @param Z		3rd float component
 * @param W		4th float component
 * @return		LmVectorRegister(X, Y, Z, W)
 */
#define LmVectorSet( X, Y, Z, W )			LmMakeVectorRegister( X, Y, Z, W )

/**
 * Stores a vector to aligned memory.
 *
 * @param Vec	Vector to store
 * @param Ptr	Aligned memory pointer
 */
#define LmVectorStoreAligned( Vec, Ptr )	_mm_store_ps( (float*)(Ptr), Vec )

/**
 * Stores a vector to memory (aligned or unaligned).
 *
 * @param Vec	Vector to store
 * @param Ptr	Memory pointer
 */
#define LmVectorStore( Vec, Ptr )			_mm_storeu_ps( (float*)(Ptr), Vec )

/**
 * Stores the XYZ components of a vector to unaligned memory.
 *
 * @param Vec	Vector to store XYZ
 * @param Ptr	Unaligned memory pointer
 */
FORCEINLINE void LmVectorStoreFloat3( const LmVectorRegister& Vec, void* Ptr )
{
	union { LmVectorRegister v; float f[4]; } Tmp;
	Tmp.v = Vec;
	float* FloatPtr = (float*)(Ptr);
	FloatPtr[0] = Tmp.f[0];
	FloatPtr[1] = Tmp.f[1];
	FloatPtr[2] = Tmp.f[2];
}

/**
 * Stores the X component of a vector to unaligned memory.
 *
 * @param Vec	Vector to store X
 * @param Ptr	Unaligned memory pointer
 */
#define LmVectorStoreFloat1( Vec, Ptr )	_mm_store_ss((float*)(Ptr), Vec)

/**
 * Truncates the X component of a vector and returns it as an integer.
 *
 * @param	Vec		Vector to truncate X component
 * @return	(int32) Truncate( Vec.X )
 */
#define LmVectorTruncate( Vec ) _mm_cvttss_si32( Vec )

/**
 * Multiplexes two vectors using upper two floating point values. 
 * 
 * @param	Vec1	Source vector
 * @param	Vec2	Source vector
 * @param	LmVectorRegister( Vec1.z, Vec1.w, Vec2.z, Vec2.w )
 */
#define LmVectorMultiplexHigh( Vec1, Vec2 )	_mm_movehl_ps( Vec1, Vec2 )

/**
 * Multiplexes two vectors using lower two floating point values. 
 * 
 * @param	Vec1	Source vector
 * @param	Vec2	Source vector
 * @param	LmVectorRegister( Vec1.x, Vec1.y, Vec2.x, Vec2.y )
 */
#define LmVectorMultiplexLow( Vec1, Vec2 )	_mm_movelh_ps( Vec1, Vec2 )

/**
 * Replicates one component into all four components and returns the new vector.
 *
 * @param Vec				Source vector
 * @param ComponentIndex	Index (0-3) of the component to replicate
 * @return					LmVectorRegister( Vec[ComponentIndex], Vec[ComponentIndex], Vec[ComponentIndex], Vec[ComponentIndex] )
 */
#define LmVectorReplicate( Vec, ComponentIndex )	_mm_shuffle_ps( Vec, Vec, LMSHUFFLEMASK(ComponentIndex,ComponentIndex,ComponentIndex,ComponentIndex) )

/**
 * Returns the absolute value (component-wise).
 *
 * @param Vec			Source vector
 * @return				LmVectorRegister( abs(Vec.x), abs(Vec.y), abs(Vec.z), abs(Vec.w) )
 */
#define LmVectorAbs( Vec )				_mm_and_ps(Vec, LM_SSE_SIGN_MASK)

/**
 * Returns the negated value (component-wise).
 *
 * @param Vec			Source vector
 * @return				LmVectorRegister( -Vec.x, -Vec.y, -Vec.z, -Vec.w )
 */
#define LmVectorNegate( Vec )				_mm_sub_ps(_mm_setzero_ps(),Vec)

/**
 * Adds two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		LmVectorRegister( Vec1.x+Vec2.x, Vec1.y+Vec2.y, Vec1.z+Vec2.z, Vec1.w+Vec2.w )
 */
#define LmVectorAdd( Vec1, Vec2 )			_mm_add_ps( Vec1, Vec2 )

/**
 * Subtracts a vector from another (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		LmVectorRegister( Vec1.x-Vec2.x, Vec1.y-Vec2.y, Vec1.z-Vec2.z, Vec1.w-Vec2.w )
 */
#define LmVectorSubtract( Vec1, Vec2 )	_mm_sub_ps( Vec1, Vec2 )

/**
 * Multiplies two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		LmVectorRegister( Vec1.x*Vec2.x, Vec1.y*Vec2.y, Vec1.z*Vec2.z, Vec1.w*Vec2.w )
 */
#define LmVectorMultiply( Vec1, Vec2 )	_mm_mul_ps( Vec1, Vec2 )

/**
 * Divides two vectors (component-wise) and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		LmVectorRegister( Vec1.x/Vec2.x, Vec1.y/Vec2.y, Vec1.z/Vec2.z, Vec1.w/Vec2.w )
 */
#define LmVectorDivide( Vec1, Vec2 )		_mm_div_ps( Vec1, Vec2 )

/**
 * Multiplies two vectors (component-wise), adds in the third vector and returns the result.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @param Vec3	3rd vector
 * @return		LmVectorRegister( Vec1.x*Vec2.x + Vec3.x, Vec1.y*Vec2.y + Vec3.y, Vec1.z*Vec2.z + Vec3.z, Vec1.w*Vec2.w + Vec3.w )
 */
#define LmVectorMultiplyAdd( Vec1, Vec2, Vec3 )	_mm_add_ps( _mm_mul_ps(Vec1, Vec2), Vec3 )

/**
 * Calculates the dot3 product of two vectors and returns a vector with the result in all 4 components.
 * Only really efficient on Xbox 360.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot3(Vec1.xyz, Vec2.xyz), LmVectorRegister( d, d, d, d )
 */
FORCEINLINE LmVectorRegister LmVectorDot3( const LmVectorRegister& Vec1, const LmVectorRegister& Vec2 )
{
	LmVectorRegister Temp = LmVectorMultiply( Vec1, Vec2 );
	return LmVectorAdd( LmVectorReplicate(Temp,0), LmVectorAdd( LmVectorReplicate(Temp,1), LmVectorReplicate(Temp,2) ) );
}

/**
 * Calculates the dot4 product of two vectors and returns a vector with the result in all 4 components.
 * Only really efficient on Xbox 360.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		d = dot4(Vec1.xyzw, Vec2.xyzw), LmVectorRegister( d, d, d, d )
 */
FORCEINLINE LmVectorRegister LmVectorDot4( const LmVectorRegister& Vec1, const LmVectorRegister& Vec2 )
{
	LmVectorRegister Temp1, Temp2;
	Temp1 = LmVectorMultiply( Vec1, Vec2 );
	Temp2 = _mm_shuffle_ps( Temp1, Temp1, LMSHUFFLEMASK(2,3,0,1) );	// (Z,W,X,Y).
	Temp1 = LmVectorAdd( Temp1, Temp2 );								// (X*X + Z*Z, Y*Y + W*W, Z*Z + X*X, W*W + Y*Y)
	Temp2 = _mm_shuffle_ps( Temp1, Temp1, LMSHUFFLEMASK(1,2,3,0) );	// Rotate left 4 bytes (Y,Z,W,X).
	return LmVectorAdd( Temp1, Temp2 );								// (X*X + Z*Z + Y*Y + W*W, Y*Y + W*W + Z*Z + X*X, Z*Z + X*X + W*W + Y*Y, W*W + Y*Y + X*X + Z*Z)
}

/**
 * Calculates the cross product of two vectors (XYZ components). W is set to 0.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		cross(Vec1.xyz, Vec2.xyz). W is set to 0.
 */
FORCEINLINE LmVectorRegister LmVectorCross( const LmVectorRegister& Vec1, const LmVectorRegister& Vec2 )
{
	LmVectorRegister A_YZXW = _mm_shuffle_ps( Vec1, Vec1, LMSHUFFLEMASK(1,2,0,3) );
	LmVectorRegister B_ZXYW = _mm_shuffle_ps( Vec2, Vec2, LMSHUFFLEMASK(2,0,1,3) );
	LmVectorRegister A_ZXYW = _mm_shuffle_ps( Vec1, Vec1, LMSHUFFLEMASK(2,0,1,3) );
	LmVectorRegister B_YZXW = _mm_shuffle_ps( Vec2, Vec2, LMSHUFFLEMASK(1,2,0,3) );
	return LmVectorSubtract( LmVectorMultiply(A_YZXW,B_ZXYW), LmVectorMultiply(A_ZXYW, B_YZXW) );
}

/**
 * Calculates x raised to the power of y (component-wise).
 *
 * @param Base		Base vector
 * @param Exponent	Exponent vector
 * @return			LmVectorRegister( Base.x^Exponent.x, Base.y^Exponent.y, Base.z^Exponent.z, Base.w^Exponent.w )
 */
FORCEINLINE LmVectorRegister LmVectorPow( const LmVectorRegister& Base, const LmVectorRegister& Exponent )
{
	//@TODO: Optimize
	union { LmVectorRegister v; float f[4]; } B, E;
	B.v = Base;
	E.v = Exponent;
	return _mm_setr_ps( powf(B.f[0], E.f[0]), powf(B.f[1], E.f[1]), powf(B.f[2], E.f[2]), powf(B.f[3], E.f[3]) );
}

/**
 * Multiplies two 4x4 matrices.
 *
 * @param Result	Pointer to where the result should be stored
 * @param Matrix1	Pointer to the first matrix
 * @param Matrix2	Pointer to the second matrix
 */
FORCEINLINE void LmVectorMatrixMultiply( void *Result, const void* Matrix1, const void* Matrix2 )
{
	const LmVectorRegister *A	= (const LmVectorRegister *) Matrix1;
	const LmVectorRegister *B	= (const LmVectorRegister *) Matrix2;
	LmVectorRegister *R		= (LmVectorRegister *) Result;
	LmVectorRegister Temp, R0, R1, R2, R3;

	// First row of result (Matrix1[0] * Matrix2).
	Temp	= LmVectorMultiply( LmVectorReplicate( A[0], 0 ), B[0] );
	Temp	= LmVectorMultiplyAdd( LmVectorReplicate( A[0], 1 ), B[1], Temp );
	Temp	= LmVectorMultiplyAdd( LmVectorReplicate( A[0], 2 ), B[2], Temp );
	R0		= LmVectorMultiplyAdd( LmVectorReplicate( A[0], 3 ), B[3], Temp );

	// Second row of result (Matrix1[1] * Matrix2).
	Temp	= LmVectorMultiply( LmVectorReplicate( A[1], 0 ), B[0] );
	Temp	= LmVectorMultiplyAdd( LmVectorReplicate( A[1], 1 ), B[1], Temp );
	Temp	= LmVectorMultiplyAdd( LmVectorReplicate( A[1], 2 ), B[2], Temp );
	R1		= LmVectorMultiplyAdd( LmVectorReplicate( A[1], 3 ), B[3], Temp );

	// Third row of result (Matrix1[2] * Matrix2).
	Temp	= LmVectorMultiply( LmVectorReplicate( A[2], 0 ), B[0] );
	Temp	= LmVectorMultiplyAdd( LmVectorReplicate( A[2], 1 ), B[1], Temp );
	Temp	= LmVectorMultiplyAdd( LmVectorReplicate( A[2], 2 ), B[2], Temp );
	R2		= LmVectorMultiplyAdd( LmVectorReplicate( A[2], 3 ), B[3], Temp );

	// Fourth row of result (Matrix1[3] * Matrix2).
	Temp	= LmVectorMultiply( LmVectorReplicate( A[3], 0 ), B[0] );
	Temp	= LmVectorMultiplyAdd( LmVectorReplicate( A[3], 1 ), B[1], Temp );
	Temp	= LmVectorMultiplyAdd( LmVectorReplicate( A[3], 2 ), B[2], Temp );
	R3		= LmVectorMultiplyAdd( LmVectorReplicate( A[3], 3 ), B[3], Temp );

	// Store result
	R[0] = R0;
	R[1] = R1;
	R[2] = R2;
	R[3] = R3;
}

/**
 * Calculate the inverse of an FMatrix44f.
 *
 * @param DstMatrix		FMatrix44f pointer to where the result should be stored
 * @param SrcMatrix		FMatrix44f pointer to the Matrix to be inversed
 */
#define LmVectorMatrixInverse( DstMatrix, SrcMatrix ) D3DXMatrixInverse( (D3DXMATRIX*) DstMatrix, NULL, (const D3DXMATRIX*)SrcMatrix )

/**
 * Returns the minimum values of two vectors (component-wise).
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		LmVectorRegister( min(Vec1.x,Vec2.x), min(Vec1.y,Vec2.y), min(Vec1.z,Vec2.z), min(Vec1.w,Vec2.w) )
 */
#define LmVectorMin( Vec1, Vec2 )			_mm_min_ps( Vec1, Vec2 )

/**
 * Returns the maximum values of two vectors (component-wise).
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		LmVectorRegister( max(Vec1.x,Vec2.x), max(Vec1.y,Vec2.y), max(Vec1.z,Vec2.z), max(Vec1.w,Vec2.w) )
 */
#define LmVectorMax( Vec1, Vec2 )			_mm_max_ps( Vec1, Vec2 )

/**
 * Returns the minimum values of the first component of two vectors and passes through
 * others from first vectors.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		LmVectorRegister( min(Vec1.x,Vec2.x), Vec1.y, Vec1.z, Vec1.w )
 */
#define LmVectorMinFloat1( Vec1, Vec2 )	_mm_min_ss( Vec1, Vec2 )

/**
 * Returns the maximum values of the first component of two vectors and passes through
 * others from first vectors.
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		LmVectorRegister( max(Vec1.x,Vec2.x), Vec1.y, Vec1.z, Vec1.w )
 */
#define LmVectorMaxFloat1( Vec1, Vec2 )	_mm_max_ss( Vec1, Vec2 )

/**
 * Compares first component of Vec1 to first component of Vec2 and returns
 * true if Vec1 >= Vec2.
 *
 * @param	Vec1	Vector to compare
 * @param	Vec2	Vector to compare
 * @return (Vec1.x >= Vec2.x) ? true : false
 */
#define LmVectorCompareGEFloat1( Vec1, Vec2 )	_mm_comige_ss( Vec1, Vec2 )

/**
 * Compares Vec1 to Vec2 and returns 0xFFFFFFFF per component if it was GE
 * or 0 if it wasn't.
 *
 * @param	Vec1	Vector to compare
 * @param	Vec2	Vector to compare
 * @return  LmVectorRegister( Vec1.x >= Vec2.x ? 0xFFFFFFFF : 0, Vec1.y >= Vec2.y ? 0xFFFFFFFF : 0, Vec1.z >= Vec2.z ? 0xFFFFFFFF : 0, Vec1.w >= Vec2.w ? 0xFFFFFFFF : 0 )
 */
#define LmVectorCompareGE( Vec1, Vec2 )	_mm_cmpge_ps( Vec1, Vec2 )

/**
 * Compares Vec1 to Vec2 and returns 0xFFFFFFFF per component if it was GT
 * or 0 if it wasn't.
 *
 * @param	Vec1	Vector to compare
 * @param	Vec2	Vector to compare
 * @return  LmVectorRegister( Vec1.x > Vec2.x ? 0xFFFFFFFF : 0, Vec1.y > Vec2.y ? 0xFFFFFFFF : 0, Vec1.z > Vec2.z ? 0xFFFFFFFF : 0, Vec1.w > Vec2.w ? 0xFFFFFFFF : 0 )
 */
#define LmVectorCompareGT( Vec1, Vec2 )	_mm_cmpgt_ps( Vec1, Vec2 )

/**
 * Returns the bitwise AND.
 *
 * @param	Vec1	Vector to AND
 * @param	Vec2	Vector to AND
 * @return	bitwise per component AND operation.
 */
#define LmVectorBitwiseAND( Vec1, Vec2 )	_mm_and_ps( (Vec1), (Vec2) )

/**
 * Returns the bitwise OR.
 *
 * @param	Vec1	Vector to OR
 * @param	Vec2	Vector to OR
 * @return	bitwise per component OR operation.
 */
#define LmVectorBitwiseOR( Vec1, Vec2 )	_mm_or_ps( (Vec1), (Vec2) )

/**
 * Returns the bitwise XOR.
 *
 * @param	Vec1	Vector to XOR
 * @param	Vec2	Vector to XOR
 * @return	bitwise per component XOR operation.
 */
#define LmVectorBitwiseXOR( Vec1, Vec2 )	_mm_xor_ps( (Vec1), (Vec2) )

/**
 * Returns the bitwise AND NOT.
 *
 * @param	Vec1	Vector to NOT AND
 * @param	Vec2	Vector to AND
 * @return	LmVectorRegister( ~Vec1.x & Vec2.x, ~Vec1.y & Vec2.y, ~Vec1.z & Vec2.z, ~Vec1.w & Vec2.w )
 */
#define LmVectorBitwiseANDNOT( Vec1, Vec2 )	_mm_andnot_ps( (Vec1), (Vec2) )

/**
 * Selects the passed in vectors based on mask.
 *
 * @param	Vec1	Vector to choose from
 * @param	Vec2	Vector to choose from
 * @param	Mask	Mask to use for chosing. Can be thought of as "alpha"
 * @return	LmVectorRegister( Mask.X == 0xFFFFFFFF ? Vec2.X : Vec1.X, Mask.Y == 0xFFFFFFFF ? Vec2.Y : Vec1.Y, Mask.Z == 0xFFFFFFFF ? Vec2.Z : Vec1.Z, Mask.W == 0xFFFFFFFF ? Vec2.W : Vec1.W );
 */
FORCEINLINE LmVectorRegister LmVectorSelect( const LmVectorRegister& Vec1, const LmVectorRegister& Vec2, const LmVectorRegister& Mask )
{
	return _mm_or_ps( _mm_and_ps( Vec2, Mask ), _mm_andnot_ps( Mask, Vec1 ) );
}

/**
 * Swizzles the 4 components of a vector and returns the result.
 *
 * @param Vec		Source vector
 * @param X			Index for which component to use for X (literal 0-3)
 * @param Y			Index for which component to use for Y (literal 0-3)
 * @param Z			Index for which component to use for Z (literal 0-3)
 * @param W			Index for which component to use for W (literal 0-3)
 * @return			The swizzled vector
 */
#define LmVectorSwizzle( Vec, X, Y, Z, W )	_mm_shuffle_ps( Vec, Vec, LMSHUFFLEMASK(X,Y,Z,W) )

/**
 * Loads 4 BYTEs from unaligned memory and converts them into 4 FLOATs.
 * IMPORTANT: You need to call VectorResetFloatRegisters() before using scalar FLOATs after you've used this intrinsic!
 *
 * @param Ptr			Unaligned memory pointer to the 4 BYTEs.
 * @return				LmVectorRegister( float(Ptr[0]), float(Ptr[1]), float(Ptr[2]), float(Ptr[3]) )
 */
//#define VectorLoadByte4( Ptr )			_mm_cvtpu8_ps( *((__m64*)Ptr) )

/**
 * Returns non-zero if any component in Vec1 is greater than the corresponding component in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x > Vec2.x) || (Vec1.y > Vec2.y) || (Vec1.z > Vec2.z) || (Vec1.w > Vec2.w)
 */
#define LmVectorAnyGreaterThan( Vec1, Vec2 )		_mm_movemask_ps( _mm_cmpgt_ps(Vec1, Vec2) )

/**
 * Returns non-zero if all components in Vec1 are greater than the corresponding components in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x > Vec2.x) && (Vec1.y > Vec2.y) && (Vec1.z > Vec2.z) && (Vec1.w > Vec2.w)
 */
#define LmVectorAllGreaterThan( Vec1, Vec2 )		(!_mm_movemask_ps( _mm_cmple_ps(Vec1, Vec2) ))

/**
 * Returns non-zero if any component in Vec1 is less than the corresponding component in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x < Vec2.x) || (Vec1.y < Vec2.y) || (Vec1.z < Vec2.z) || (Vec1.w < Vec2.w)
 */
#define LmVectorAnyLessThan( Vec1, Vec2 )		_mm_movemask_ps( _mm_cmplt_ps(Vec1, Vec2) )

/**
 * Returns non-zero if all components in Vec1 are less than the corresponding components in Vec2, otherwise 0.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Non-zero integer if (Vec1.x < Vec2.x) && (Vec1.y < Vec2.y) && (Vec1.z < Vec2.z) && (Vec1.w < Vec2.w)
 */
#define LmVectorAllLessThan( Vec1, Vec2 )		(!_mm_movemask_ps( _mm_cmpge_ps(Vec1, Vec2) ))

/**
 * These functions return a vector mask to indicate which components pass the comparison.
 * Each component is 0xffffffff if it passes, 0x00000000 if it fails.
 *
 * @param Vec1			1st source vector
 * @param Vec2			2nd source vector
 * @return				Vector with a mask for each component.
 */
#define LmVectorMask_LT( Vec1, Vec2 )			_mm_cmplt_ps(Vec1, Vec2)
#define LmVectorMask_LE( Vec1, Vec2 )			_mm_cmple_ps(Vec1, Vec2)
#define LmVectorMask_GT( Vec1, Vec2 )			_mm_cmpgt_ps(Vec1, Vec2)
#define LmVectorMask_GE( Vec1, Vec2 )			_mm_cmpge_ps(Vec1, Vec2)
#define LmVectorMask_EQ( Vec1, Vec2 )			_mm_cmpeq_ps(Vec1, Vec2)
#define LmVectorMask_NE( Vec1, Vec2 )			_mm_cmpneq_ps(Vec1, Vec2)

/**
 * Shifts the 4 signed or unsigned 32-bit integers right by input amount while shifting in zeros.
 *
 * @param Vec1			1st source vector
 * @param Count			Number of bits to shift
 * @return				Shifted vector
 */
#define LmVectorShiftRight( Vec1, Count )		_mm_castsi128_ps(_mm_srli_epi32(_mm_castps_si128(Vec1), Count))

/**
 * Returns an integer bit-mask (0x00 - 0x0f) based on the sign-bit for each component in a vector.
 *
 * @param VecMask		Vector
 * @return				Bit 0 = sign(VecMask.x), Bit 1 = sign(VecMask.y), Bit 2 = sign(VecMask.z), Bit 3 = sign(VecMask.w)
 */
#define LmVectorMaskBits( VecMask )			_mm_movemask_ps( VecMask )

/**
 * Returns the control register.
 *
 * @return			The uint32 control register
 */
#define LmVectorGetControlRegister()		_mm_getcsr()

/**
 * Sets the control register.
 *
 * @param ControlStatus		The uint32 control status value to set
 */
#define	LmVectorSetControlRegister(ControlStatus) _mm_setcsr( ControlStatus )

/**
 * Control status bit to round all floating point math results towards zero.
 */
#define LM_VECTOR_ROUND_TOWARD_ZERO		_MM_ROUND_TOWARD_ZERO

// To be continued...


