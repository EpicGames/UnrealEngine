// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LMMath.h"
#include "LMMathSSE.h"
#include "UnrealLightmass.h"
#include "Misc/MemStack.h"

namespace Lightmass
{

/** A concise iteration over the children of an octree node. */
#define FOREACH_OCTREE_CHILD_NODE(ChildRef) \
	for(FOctreeChildNodeRef ChildRef(0);!ChildRef.IsNULL();ChildRef.Advance())

/** An unquantized bounding box. */
class FBoxCenterAndExtent
{
public:
	FVector4f Center;
	FVector4f Extent;

	/** Default constructor. */
	FBoxCenterAndExtent() {}

	/** Initialization constructor. */
	FBoxCenterAndExtent(const FVector4f& InCenter,const FVector4f& InExtent)
	:	Center(InCenter)
	,	Extent(InExtent)
	{}

	/** FBox3f conversion constructor. */
	FBoxCenterAndExtent(const FBox3f& Box)
	{
		FVector3f BoxCenter;
		FVector3f BoxExtent;
		Box.GetCenterAndExtents(BoxCenter, BoxExtent);
		Center = BoxCenter;
		Extent = BoxExtent;
		Center.W = Extent.W = 0;
	}

	/** FBoxSphereBounds3f conversion constructor. */
	FBoxCenterAndExtent(const FBoxSphereBounds3f& BoxSphere)
	{
		Center = BoxSphere.Origin;
		Extent = BoxSphere.BoxExtent;
		Center.W = Extent.W = 0;
	}

	/** Converts to a FBox3f. */
	FORCEINLINE FBox3f GetBox() const
	{
		return FBox3f(Center - Extent,Center + Extent);
	}

	/**
	 * Determines whether two boxes intersect.
	 * @return true if the boxes intersect, or false.
	 */
	friend bool Intersect(const FBoxCenterAndExtent& A,const FBoxCenterAndExtent& B)
	{
		// Note: The W component of the bounds center positions is unreliable here, and must not be used in the intersection
		FVector4f CenterDifference = A.Center - B.Center;
		CenterDifference.X = FMath::Abs(CenterDifference.X);
		CenterDifference.Y = FMath::Abs(CenterDifference.Y);
		CenterDifference.Z = FMath::Abs(CenterDifference.Z);

		const FVector4f CompositeExtent = A.Extent + B.Extent;

		return CenterDifference.X < CompositeExtent.X && CenterDifference.Y < CompositeExtent.Y && CenterDifference.Z < CompositeExtent.Z;
	}
};

/** A reference to a child of an octree node. */
class FOctreeChildNodeRef
{
public:

	union
	{
		struct
		{
			uint32 X : 1;
			uint32 Y : 1;
			uint32 Z : 1;
			uint32 bNULL : 1;
		};
		uint32 Index : 3;
	};

	/** Initialization constructor. */
	FOctreeChildNodeRef(int32 InX,int32 InY,int32 InZ)
	:	X(InX)
	,	Y(InY)
	,	Z(InZ)
	,	bNULL(false)
	{}

	/** Initialized the reference with a child index. */
	FOctreeChildNodeRef(int32 InIndex = 0)
	{
		bNULL = false;
		Index = InIndex;
	}

	/** Advances the reference to the next child node.  If this was the last node remain, sets bInvalid=true. */
	FORCEINLINE void Advance()
	{
		if(Index < 7)
		{
			++Index;
		}
		else
		{
			bNULL = true;
		}
	}

	/** @return true if the reference isn't set. */
	FORCEINLINE bool IsNULL() const
	{
		return bNULL;
	}
};

/** A subset of an octree node's children that intersect a bounding box. */
class FOctreeChildNodeSubset
{
public:

	union
	{
		struct 
		{
			uint32 bPositiveX : 1;
			uint32 bPositiveY : 1;
			uint32 bPositiveZ : 1;
			uint32 bNegativeX : 1;
			uint32 bNegativeY : 1;
			uint32 bNegativeZ : 1;
		};

		struct
		{
			/** Only the bits for the children on the positive side of the splits. */
			uint32 PositiveChildBits : 3;

			/** Only the bits for the children on the negative side of the splits. */
			uint32 NegativeChildBits : 3;
		};

		/** All the bits corresponding to the child bits. */
		uint32 ChildBits : 6;

		/** All the bits used to store the subset. */
		uint32 AllBits;
	};

	/** Initializes the subset to be empty. */
	FOctreeChildNodeSubset()
	:	AllBits(0)
	{}

	/** Initializes the subset to contain a single node. */
	FOctreeChildNodeSubset(FOctreeChildNodeRef ChildRef)
	:	AllBits(0)
	{
		// The positive child bits correspond to the child index, and the negative to the NOT of the child index.
		PositiveChildBits = ChildRef.Index;
		NegativeChildBits = ~ChildRef.Index;
	}

	/** Determines whether the subset contains a specific node. */
	bool Contains(FOctreeChildNodeRef ChildRef) const;
};

/** The context of an octree node, derived from the traversal of the tree. */
class FOctreeNodeContext
{
public:

	/** The bounds of the node. */
	FBoxCenterAndExtent Bounds;

	/** The extent of the node's children. */
	float ChildExtent;

	/** The offset of the childrens' centers from the center of this node. */
	float ChildCenterOffset;

	/** The node bounds are expanded by their extent divided by LoosenessDenominator. */
	int32 LoosenessDenominator;

	/** Default constructor. */
	FOctreeNodeContext()
	{}

	/** Initialization constructor. */
	FOctreeNodeContext(const FBoxCenterAndExtent& InBounds, int32 InLoosenessDenominator)
	:	Bounds(InBounds)
	,	LoosenessDenominator(InLoosenessDenominator)
	{
		// A child node's tight extents are half its parent's extents, and its loose extents are expanded by 1/LoosenessDenominator.
		const float TightChildExtent = Bounds.Extent.X * 0.5f;
		const float LooseChildExtent = TightChildExtent * (1.0f + 1.0f / (float)LoosenessDenominator);

		ChildExtent = LooseChildExtent;
		ChildCenterOffset = Bounds.Extent.X - LooseChildExtent;
	}

	/** Child node initialization constructor. */
	FORCEINLINE FOctreeNodeContext GetChildContext(FOctreeChildNodeRef ChildRef) const
	{
		return FOctreeNodeContext(FBoxCenterAndExtent(
				FVector4f(
					Bounds.Center.X + ChildCenterOffset * (-1.0f + 2 * ChildRef.X),
					Bounds.Center.Y + ChildCenterOffset * (-1.0f + 2 * ChildRef.Y),
					Bounds.Center.Z + ChildCenterOffset * (-1.0f + 2 * ChildRef.Z),
					0
					),
				FVector4f(
					ChildExtent,
					ChildExtent,
					ChildExtent,
					0
					)
				), LoosenessDenominator);
	}
	
	/**
	 * Determines which of the octree node's children intersect with a bounding box.
	 * @param BoundingBox - The bounding box to check for intersection with.
	 * @return A subset of the children's nodes that intersect the bounding box.
	 */
	FOctreeChildNodeSubset GetIntersectingChildren(const FBoxCenterAndExtent& BoundingBox) const;

	/**
	 * Determines which of the octree node's children contain the whole bounding box, if any.
	 * @param BoundingBox - The bounding box to check for intersection with.
	 * @return The octree's node that the bounding box is farthest from the outside edge of, or an invalid node ref if it's not contained
	 *			by any of the children.
	 */
	FOctreeChildNodeRef GetContainingChild(const FBoxCenterAndExtent& BoundingBox) const;
};

/** An octree. */
template<typename ElementType,typename OctreeSemantics>
class TOctree
{
public:

	typedef TArray<ElementType, typename OctreeSemantics::ElementAllocator > ElementArrayType;
	/** Note: Care must be taken to not modify any part of the element which would change it's location in the octree using this iterator. */
	typedef typename ElementArrayType::TIterator ElementIt;
	typedef typename ElementArrayType::TConstIterator ElementConstIt;

	/** A node in the octree. */
	class FNode
	{
	public:

		friend class TOctree;

		/** Initialization constructor. */
		FNode(const FNode* InParent)
		:	Parent(InParent)
		,	bIsLeaf(true)
		{
			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				Children[ChildRef.Index] = NULL;
			}
		}

		/** Destructor. */
		~FNode()
		{
			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if (HasChild(ChildRef.Index))
				{
					Children[ChildRef.Index]->~FNode();
				}
			}
		}

		// Accessors.
		FORCEINLINE ElementIt GetElementIt() const { return ElementIt(Elements); }
		FORCEINLINE ElementConstIt GetConstElementIt() const { return ElementConstIt(Elements); }
		FORCEINLINE bool IsLeaf() const { return bIsLeaf; }
		FORCEINLINE bool HasChild(FOctreeChildNodeRef ChildRef) const
		{
			return Children[ChildRef.Index] != NULL;
		}
		FORCEINLINE FNode* GetChild(FOctreeChildNodeRef ChildRef) const
		{
			return Children[ChildRef.Index];
		}
		FORCEINLINE int32 GetElementCount() const
		{
			return Elements.Num();
		}

	private:

		/** The elements in this node. */
		mutable ElementArrayType Elements;

		/** The parent of this node. */
		const FNode* Parent;

		/** The children of the node. */
		mutable FNode* Children[8];

		/** true if the meshes should be added directly to the node, rather than subdividing when possible. */
		mutable uint32 bIsLeaf : 1;
	};

	/** A reference to an octree node and its context. */
	class FNodeReference
	{
	public:

		FNode* Node;
		FOctreeNodeContext Context;

		/** Default constructor. */
		FNodeReference():
			Node(NULL),
			Context()
		{}

		/** Initialization constructor. */
		FNodeReference(FNode* InNode,const FOctreeNodeContext& InContext):
			Node(InNode),
			Context(InContext)
		{}
	};	

	/** A reference to an octree node, its context, and a read lock. */
	class FConstNodeReference
	{
	public:

		const FNode* Node;
		FOctreeNodeContext Context;

		/** Default constructor. */
		FConstNodeReference():
			Node(NULL),
			Context()
		{}

		/** Initialization constructor. */
		FConstNodeReference(const FNode* InNode,const FOctreeNodeContext& InContext):
			Node(InNode),
			Context(InContext)
		{}
	};	

	/** The default iterator allocator gives the stack enough inline space to contain a path and its siblings from root to leaf. */
	typedef TInlineAllocator<7 * (14 - 1) + 8> DefaultStackAllocator;

	/** An octree node iterator. */
	template<typename StackAllocator = DefaultStackAllocator>
	class TConstIterator
	{
	public:

		/** Pushes a child of the current node onto the stack of nodes to visit. */
		void PushChild(FOctreeChildNodeRef ChildRef)
		{
			NodeStack.Add(
				FConstNodeReference(
					CurrentNode.Node->GetChild(ChildRef),
					CurrentNode.Context.GetChildContext(ChildRef)
					));
		}

		/** Iterates to the next node. */
		void Advance()
		{
			if(NodeStack.Num())
			{
				CurrentNode = NodeStack[NodeStack.Num() - 1];
				NodeStack.RemoveAt(NodeStack.Num() - 1);
			}
			else
			{
				CurrentNode = FConstNodeReference();
			}
		}

		/** Checks if there are any nodes left to iterate over. */
		bool HasPendingNodes() const
		{
			return CurrentNode.Node != NULL;
		}

		/** Starts iterating at the root of an octree. */
		TConstIterator(const TOctree& Tree)
		:	CurrentNode(FConstNodeReference(&Tree.RootNode,Tree.RootNodeContext))
		{}

		/** Starts iterating at a particular node of an octree. */
		TConstIterator(const FNode& Node,const FOctreeNodeContext& Context):
			CurrentNode(FConstNodeReference(&Node,Context))
		{}

		// Accessors.
		const FNode& GetCurrentNode() const
		{
			return *CurrentNode.Node;
		}
		const FOctreeNodeContext& GetCurrentContext() const
		{
			return CurrentNode.Context;
		}

	private:

		/** The node that is currently being visited. */
		FConstNodeReference CurrentNode;
	
		/** The nodes which are pending iteration. */
		TArray<FConstNodeReference,StackAllocator> NodeStack;
	};

	
	/** An octree node iterator. */
	template<typename StackAllocator = DefaultStackAllocator>
	class TIterator
	{
	public:

		/** Pushes a child of the current node onto the stack of nodes to visit. */
		void PushChild(FOctreeChildNodeRef ChildRef)
		{
			NodeStack.Add(
				FNodeReference(
					CurrentNode.Node->GetChild(ChildRef),
					CurrentNode.Context.GetChildContext(ChildRef)
					));
		}

		/** Iterates to the next node. */
		void Advance()
		{
			if(NodeStack.Num())
			{
				CurrentNode = NodeStack[NodeStack.Num() - 1];
				NodeStack.RemoveAt(NodeStack.Num() - 1);
			}
			else
			{
				CurrentNode = FNodeReference();
			}
		}

		/** Checks if there are any nodes left to iterate over. */
		bool HasPendingNodes() const
		{
			return CurrentNode.Node != NULL;
		}

		/** Starts iterating at the root of an octree. */
		TIterator(TOctree& Tree)
		:	CurrentNode(FNodeReference(&Tree.RootNode,Tree.RootNodeContext))
		{}

		/** Starts iterating at a particular node of an octree. */
		TIterator(FNode& Node,const FOctreeNodeContext& Context):
			CurrentNode(FNodeReference(&Node,Context))
		{}

		// Accessors.
		FNode& GetCurrentNode()
		{
			return *CurrentNode.Node;
		}
		const FOctreeNodeContext& GetCurrentContext() const
		{
			return CurrentNode.Context;
		}

	private:

		/** The node that is currently being visited. */
		FNodeReference CurrentNode;
	
		/** The nodes which are pending iteration. */
		TArray<FNodeReference,StackAllocator> NodeStack;
	};

	/** Iterates over the elements in the octree that intersect a bounding box. */
	template<typename StackAllocator = DefaultStackAllocator>
	class TConstElementBoxIterator
	{
	public:

		/** Iterates to the next element. */
		void Advance()
		{
			++ElementIt;
			AdvanceToNextIntersectingElement();
		}

		/** Checks if there are any elements left to iterate over. */
		bool HasPendingElements() const
		{
			return NodeIt.HasPendingNodes();
		}

		/** Initialization constructor. */
		TConstElementBoxIterator(const TOctree& Tree,const FBoxCenterAndExtent& InBoundingBox)
		:	IteratorBounds(InBoundingBox)
		,	NodeIt(Tree)
		,	ElementIt(Tree.RootNode.GetConstElementIt())
		{
			ProcessChildren();
			AdvanceToNextIntersectingElement();
		}

		// Accessors.
		const ElementType& GetCurrentElement() const
		{
			return *ElementIt;
		}

	private:

		/** The bounding box to check for intersection with. */
		FBoxCenterAndExtent IteratorBounds;

		/** The octree node iterator. */
		TConstIterator<StackAllocator> NodeIt;

		/** The element iterator for the current node. */
		ElementConstIt ElementIt;

		/** Processes the children of the current node. */
		void ProcessChildren()
		{
			// Add the child nodes that intersect the bounding box to the node iterator's stack.
			const FNode& CurrentNode = NodeIt.GetCurrentNode();
			const FOctreeNodeContext& Context = NodeIt.GetCurrentContext();
			const FOctreeChildNodeSubset IntersectingChildSubset = Context.GetIntersectingChildren(IteratorBounds);
			FOREACH_OCTREE_CHILD_NODE(ChildRef)
			{
				if(IntersectingChildSubset.Contains(ChildRef) && CurrentNode.HasChild(ChildRef))
				{
					NodeIt.PushChild(ChildRef);
				}
			}
		}

		/** Advances the iterator to the next intersecting primitive, starting at a primitive in the current node. */
		void AdvanceToNextIntersectingElement()
		{
			// Keep trying elements until we find one that intersects or run out of elements to try.
			while(NodeIt.HasPendingNodes())
			{
				// Check if we've advanced past the elements in the current node.
				if(ElementIt)
				{
					// Check if the current element intersects the bounding box.
					if(Intersect(OctreeSemantics::GetBoundingBox(*ElementIt),IteratorBounds))
					{
						// If it intersects, break out of the advancement loop.
						break;
					}
					else
					{
						// If it doesn't intersect, skip to the next element.
						++ElementIt;
					}
				}
				else
				{
					// Advance to the next node.
					NodeIt.Advance();
					if(NodeIt.HasPendingNodes())
					{
						ProcessChildren();

						// The element iterator can't be assigned to, but it can be replaced by Move.
						Move(ElementIt,NodeIt.GetCurrentNode().GetConstElementIt());
					}
				}
			};
		}
	};

	/**
	 * Adds an element to the octree.
	 * @param Element - The element to add.
	 * @return An identifier for the element in the octree.
	 */
	void AddElement(typename TTypeTraits<ElementType>::ConstInitType Element);

	/** Writes stats for the octree to the log. */
	void DumpStats(bool bDetailed) const;

	void GetMemoryUsage(size_t& OutSizeBytes) const;

	/** Initialization constructor. */
	TOctree(const FVector4f& InOrigin, float InExtent);

	void Destroy()
	{
		RootNode.~FNode();
		new (&RootNode) FNode(NULL);
	}

private:
	/** Lightmass octree is grow-only. Use a stack allocator to coalesce allocations for better allocation performance. */
	FMemStackBase NodeAllocator;

	/** The octree's root node. */
	FNode RootNode;

	/** The octree's root node's context. */
	FOctreeNodeContext RootNodeContext;

	/** The extent of a leaf at the maximum allowed depth of the tree. */
	float MinLeafExtent;

	/** Adds an element to a node or its children. */
	void AddElementToNode(
		typename TTypeTraits<ElementType>::ConstInitType Element,
		const FNode& InNode,
		const FOctreeNodeContext& InContext
		);
};


FORCEINLINE bool FOctreeChildNodeSubset::Contains(FOctreeChildNodeRef ChildRef) const
{
	// This subset contains the child if it has all the bits set that are set for the subset containing only the child node.
	const FOctreeChildNodeSubset ChildSubset(ChildRef);
	return (ChildBits & ChildSubset.ChildBits) == ChildSubset.ChildBits;
}

FORCEINLINE FOctreeChildNodeSubset FOctreeNodeContext::GetIntersectingChildren(const FBoxCenterAndExtent& QueryBounds) const
{
	FOctreeChildNodeSubset Result;

	// Load the query bounding box values as VectorRegisters.
	const LmVectorRegister QueryBoundsCenter = LmVectorLoadAligned(&QueryBounds.Center);
	const LmVectorRegister QueryBoundsExtent = LmVectorLoadAligned(&QueryBounds.Extent);
	const LmVectorRegister QueryBoundsMax = LmVectorAdd(QueryBoundsCenter,QueryBoundsExtent);
	const LmVectorRegister QueryBoundsMin = LmVectorSubtract(QueryBoundsCenter,QueryBoundsExtent);

	// Compute the bounds of the node's children.
	const LmVectorRegister BoundsCenter = LmVectorLoadAligned(&Bounds.Center);
	const LmVectorRegister BoundsExtent = LmVectorLoadAligned(&Bounds.Extent);
	const LmVectorRegister PositiveChildBoundsMin = LmVectorSubtract(
		LmVectorAdd(BoundsCenter,LmVectorLoadFloat1(&ChildCenterOffset)),
		LmVectorLoadFloat1(&ChildExtent)
		);
	const LmVectorRegister NegativeChildBoundsMax = LmVectorAdd(
		LmVectorSubtract(BoundsCenter,LmVectorLoadFloat1(&ChildCenterOffset)),
		LmVectorLoadFloat1(&ChildExtent)
		);

	// Intersect the query bounds with the node's children's bounds.
	Result.bPositiveX = LmVectorAnyGreaterThan(LmVectorReplicate(QueryBoundsMax,0),LmVectorReplicate(PositiveChildBoundsMin,0)) != false;
	Result.bPositiveY = LmVectorAnyGreaterThan(LmVectorReplicate(QueryBoundsMax,1),LmVectorReplicate(PositiveChildBoundsMin,1)) != false;
	Result.bPositiveZ = LmVectorAnyGreaterThan(LmVectorReplicate(QueryBoundsMax,2),LmVectorReplicate(PositiveChildBoundsMin,2)) != false;
	Result.bNegativeX = LmVectorAnyGreaterThan(LmVectorReplicate(QueryBoundsMin,0),LmVectorReplicate(NegativeChildBoundsMax,0)) == false;
	Result.bNegativeY = LmVectorAnyGreaterThan(LmVectorReplicate(QueryBoundsMin,1),LmVectorReplicate(NegativeChildBoundsMax,1)) == false;
	Result.bNegativeZ = LmVectorAnyGreaterThan(LmVectorReplicate(QueryBoundsMin,2),LmVectorReplicate(NegativeChildBoundsMax,2)) == false;
	return Result;
}

FORCEINLINE FOctreeChildNodeRef FOctreeNodeContext::GetContainingChild(const FBoxCenterAndExtent& QueryBounds) const
{
	FOctreeChildNodeRef Result;

	// Load the query bounding box values as VectorRegisters.
	const LmVectorRegister QueryBoundsCenter = LmVectorLoadAligned(&QueryBounds.Center);
	const LmVectorRegister QueryBoundsExtent = LmVectorLoadAligned(&QueryBounds.Extent);

	// Compute the bounds of the node's children.
	const LmVectorRegister BoundsCenter = LmVectorLoadAligned(&Bounds.Center);
	const LmVectorRegister ChildCenterOffsetVector = LmVectorLoadFloat1(&ChildCenterOffset);
	const LmVectorRegister NegativeCenterDifference = LmVectorSubtract(QueryBoundsCenter,LmVectorSubtract(BoundsCenter,ChildCenterOffsetVector));
	const LmVectorRegister PositiveCenterDifference = LmVectorSubtract(LmVectorAdd(BoundsCenter,ChildCenterOffsetVector),QueryBoundsCenter);

	// If the query bounds isn't entirely inside the bounding box of the child it's closest to, it's not contained by any of the child nodes.
	const LmVectorRegister MinDifference = LmVectorMin(PositiveCenterDifference,NegativeCenterDifference);
	if(LmVectorAnyGreaterThan(LmVectorAdd(QueryBoundsExtent,MinDifference),LmVectorLoadFloat1(&ChildExtent)))
	{
		Result.bNULL = true;
	}
	else
	{
		// Return the child node that the query is closest to as the containing child.
		Result.X = LmVectorAnyGreaterThan(LmVectorReplicate(QueryBoundsCenter,0),LmVectorReplicate(BoundsCenter,0)) != false;
		Result.Y = LmVectorAnyGreaterThan(LmVectorReplicate(QueryBoundsCenter,1),LmVectorReplicate(BoundsCenter,1)) != false;
		Result.Z = LmVectorAnyGreaterThan(LmVectorReplicate(QueryBoundsCenter,2),LmVectorReplicate(BoundsCenter,2)) != false;
	}

	return Result;
}

template<typename ElementType,typename OctreeSemantics>
void TOctree<ElementType,OctreeSemantics>::AddElement(typename TTypeTraits<ElementType>::ConstInitType Element)
{
	AddElementToNode(Element,RootNode,RootNodeContext);
}
															
template<typename ElementType,typename OctreeSemantics>
void TOctree<ElementType,OctreeSemantics>::AddElementToNode(
	typename TTypeTraits<ElementType>::ConstInitType Element,
	const FNode& InNode,
	const FOctreeNodeContext& InContext
	)
{
	const FBoxCenterAndExtent ElementBounds = OctreeSemantics::GetBoundingBox(Element);

	for(TConstIterator<TInlineAllocator<1> > NodeIt(InNode,InContext);NodeIt.HasPendingNodes();NodeIt.Advance())
	{
		const FNode& Node = NodeIt.GetCurrentNode();
		const FOctreeNodeContext& Context = NodeIt.GetCurrentContext();

		bool bAddElementToThisNode = false;

		if(Node.IsLeaf())
		{
			// If this is a leaf, check if adding this element would turn it into a node by overflowing its element list.
			if(Node.Elements.Num() + 1 > OctreeSemantics::MaxElementsPerLeaf && Context.Bounds.Extent.X > MinLeafExtent)
			{
				// Copy the leaf's elements, remove them from the leaf, and turn it into a node.
				ElementArrayType ChildElements;
				Exchange(ChildElements,Node.Elements);

				// Allow elements to be added to children of this node.
				Node.bIsLeaf = false;

				// Re-add all of the node's child elements, potentially creating children of this node for them.
				for(ElementConstIt ElementIt(ChildElements);ElementIt;++ElementIt)
				{
					AddElementToNode(*ElementIt,Node,Context);
				}

				// Add the element to this node.
				AddElementToNode(Element,Node,Context);
				return;
			}
			else
			{
				// If the leaf has room for the new element, simply add it to the list.
				bAddElementToThisNode = true;
			}
		}
		else
		{
			// If this isn't a leaf, find a child that entirely contains the element.
			const FOctreeChildNodeRef ChildRef = Context.GetContainingChild(ElementBounds);	
			if(ChildRef.IsNULL())
			{
				// If none of the children completely contain the element, add it to this node directly.
				bAddElementToThisNode = true;
			}
			else
			{
				// Create the child node if it hasn't been created yet.
				if(!Node.Children[ChildRef.Index])
				{
					uint8* NodeAddr = NodeAllocator.PushBytes(sizeof(typename TOctree<ElementType, OctreeSemantics>::FNode), alignof(typename TOctree<ElementType, OctreeSemantics>::FNode));
					Node.Children[ChildRef.Index] = new (NodeAddr) typename TOctree<ElementType,OctreeSemantics>::FNode(&Node);
				}

				// Push the node onto the stack to visit.
				NodeIt.PushChild(ChildRef);
			}
		}

		if(bAddElementToThisNode)
		{
			// Add the element to this node.
			new(Node.Elements) ElementType(Element);
			
			return;
		}
	}

	// Failed to find an octree node for element.
	check(0);
}

template<typename ElementType,typename OctreeSemantics>
void TOctree<ElementType,OctreeSemantics>::DumpStats(bool bDetailed) const
{
	int32 NumNodes = 0;
	int32 NumLeaves = 0;
	int32 NumElements = 0;
	int32 NumLeafElements = 0;
	int32 MaxElementsPerNode = 0;
	uint64 NodeBytes = 0;
	TArray<int32> NodeElementDistribution;

	for(TConstIterator<> NodeIt(*this);NodeIt.HasPendingNodes();NodeIt.Advance())
	{
		const FNode& CurrentNode = NodeIt.GetCurrentNode();
		const int32 CurrentNodeElementCount = CurrentNode.GetElementCount();

		NumNodes++;
		NodeBytes += sizeof(FNode) 
			// Add the size of indirect elements
			+ CurrentNode.Elements.GetAllocatedSize();
		if(CurrentNode.IsLeaf())
		{
			NumLeaves++;
			NumLeafElements += CurrentNodeElementCount;
		}

		NumElements += CurrentNodeElementCount;
		MaxElementsPerNode = FMath::Max(MaxElementsPerNode,CurrentNodeElementCount);

		if (bDetailed)
		{
			if( CurrentNodeElementCount >= NodeElementDistribution.Num() )
			{
				NodeElementDistribution.AddZeroed( CurrentNodeElementCount - NodeElementDistribution.Num() + 1 );
			}
			NodeElementDistribution[CurrentNodeElementCount]++;
		}

		FOREACH_OCTREE_CHILD_NODE(ChildRef)
		{
			if(CurrentNode.HasChild(ChildRef))
			{
				NodeIt.PushChild(ChildRef);
			}
		}
	}

	if (NumElements > 0)
	{
		UE_LOGF(LogLightmass, Log, "Octree overview:");
		UE_LOGF(LogLightmass, Log, "\t%zu bytes per node", sizeof(FNode));
		UE_LOGF(LogLightmass, Log, "\t%i nodes, for %.1f Mb", NumNodes, NodeBytes / 1048576.0f);
		UE_LOGF(LogLightmass, Log, "\t%i leaves", NumLeaves);
		UE_LOGF(LogLightmass, Log, "\t%i elements", NumElements);
		UE_LOGF(LogLightmass, Log, "\t%.1f%% elements in leaves", 100.0f * NumLeafElements / (float)NumElements);
		UE_LOGF(LogLightmass, Log, "\t%.1f avg elements per node, %i max elements per node", NumElements / (float)NumNodes, MaxElementsPerNode);
		if (bDetailed)
		{
			UE_LOGF(LogLightmass, Log, "Octree node element distribution:");
			for( int32 i=0; i<NodeElementDistribution.Num(); i++ )
			{
				if( NodeElementDistribution[i] > 0 )
				{
					UE_LOGF(LogLightmass, Log, "\tElements: %3i, Nodes: %3i",i,NodeElementDistribution[i]);
				}
			}
		}
	}
}

template<typename ElementType,typename OctreeSemantics>
void TOctree<ElementType,OctreeSemantics>::GetMemoryUsage(size_t& OutSizeBytes) const
{
	size_t SizeBytes = 0;

	for (TConstIterator<> NodeIt(*this);NodeIt.HasPendingNodes();NodeIt.Advance())
	{
		const FNode& CurrentNode = NodeIt.GetCurrentNode();

		SizeBytes += sizeof(FNode) + CurrentNode.Elements.GetAllocatedSize();
		
		FOREACH_OCTREE_CHILD_NODE(ChildRef)
		{
			if(CurrentNode.HasChild(ChildRef))
			{
				NodeIt.PushChild(ChildRef);
			}
		}
	}

	OutSizeBytes = SizeBytes;
}

template<typename ElementType,typename OctreeSemantics>
TOctree<ElementType,OctreeSemantics>::TOctree(const FVector4f& InOrigin,float InExtent)
:	RootNode(NULL)
,	RootNodeContext(FBoxCenterAndExtent(InOrigin,FVector4f(InExtent,InExtent,InExtent,0)), OctreeSemantics::LoosenessDenominator)
,	MinLeafExtent(InExtent * FMath::Pow((1.0f + 1.0f / (float)OctreeSemantics::LoosenessDenominator) / 2.0f, (float)OctreeSemantics::MaxNodeDepth))
{
}


} // namespace
