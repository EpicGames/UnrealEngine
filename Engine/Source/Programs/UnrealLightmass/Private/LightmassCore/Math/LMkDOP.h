// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
// #include "Collision.h"
#include "LMMath.h"
#include "LMMathSSE.h"
#include "LMStats.h"
#include "Templates/UnrealTemplate.h"
#include "UnrealLightmass.h"

namespace Lightmass
{
	
// Indicates how many "k / 2" there are in the k-DOP. 3 == AABB == 6 DOP. The code relies on this being 3.
#define NUM_PLANES	3
// Change to 'x' if you want kdop traversal stats at a heavy (usually around 6x) perf hit
#define SLOW_KDOP_STATS(...) 
// Whether to cache Centroid, LocalNormal and such for build triangles. Peak memory vs. minor build time trade-off.
#define CACHE_BUILD_TEMPORARIES 0
// The default number of triangles to store in each leaf
#define DEFAULT_MAX_TRIS_PER_LEAF 4

struct FHitResult
{
	/** Normal vector in coordinate system of the returner. Zero==none.	*/
	FVector4f	Normal;
	/** Time until hit.													*/
	float		Time;
	/** Primitive data item which was hit, INDEX_NONE=none				*/
	int32		Item;		

	FHitResult()
	: Normal	(0,0,0,0)
	, Time		(1.0f)
	, Item		(INDEX_NONE)
	{
	}
};

/**
 * Stores XYZ from 4 vectors in one Struct Of Arrays.
 */
struct FVector3SOA
{
	/** X = (v0.x, v1.x, v2.x, v3.x) */
	LmVectorRegister	X;
	/** Y = (v0.y, v1.y, v2.y, v3.y) */
	LmVectorRegister	Y;
	/** Z = (v0.z, v1.z, v2.z, v3.z) */
	LmVectorRegister	Z;
};

/**
 * Stores XYZW from 4 vectors in one Struct Of Arrays.
 */
struct FVector4SOA
{
	/** X = (v0.x, v1.x, v2.x, v3.x) */
	LmVectorRegister	X;
	/** Y = (v0.y, v1.y, v2.y, v3.y) */
	LmVectorRegister	Y;
	/** Z = (v0.z, v1.z, v2.z, v3.z) */
	LmVectorRegister	Z;
	/** W = (v0.w, v1.w, v2.w, v3.w) */
	LmVectorRegister	W;
};

/**
 * Stores 4 triangles in one struct (Struct Of Arrays).
 */
struct FTriangleSOA
{
	/** 3 vertex positions for each of the 4 triangles. */
	FVector3SOA	Positions[3];
	/** Triangle normal (including -W for a plane) for each of the 4 triangles. */
	FVector4SOA	Normals;
	/** All bits are 1 for two sided triangles, and 0 otherwise. */
	LmVectorRegister	TwoSidedMask;
	/** All bits are 1 for static and opaque triangles, and 0 otherwise. */
	LmVectorRegister	StaticAndOpaqueMask;
	LmVectorRegister	MeshIndices;
	LmVectorRegister	LODIndices;
	LmVectorRegister	HLODRange;
	/** A 32-bit payload value for each of the 4 triangles. */
	uint32		Payload[4];
};

/** Tracks information about a traversal through the kDOP tree. */
template<typename KDOP_IDX_TYPE>
struct TTraversalHistory
{
	/** 
	 * Number of parent nodes to track information for. 
	 * This has been tweaked for optimal boolean shadow ray performance.
	 */
	enum { NodeHistoryLength = 1 };
	KDOP_IDX_TYPE Nodes[NodeHistoryLength];

	TTraversalHistory()
	{
		for (int32 NodeIndex = 0; NodeIndex < NodeHistoryLength; NodeIndex++)
		{
			// Initialize the history to invalid index
			Nodes[NodeIndex] = 0xFFFFFFFF;
		}
	}

	TTraversalHistory AddNode(KDOP_IDX_TYPE NewNodeIndex)
	{
		TTraversalHistory NewHistory;
		for (int32 NodeIndex = NodeHistoryLength - 1; NodeIndex > 0; NodeIndex--)
		{
			// Move all the indices toward the end of the array one element
			NewHistory.Nodes[NodeIndex] = Nodes[NodeIndex - 1];
		}
		// Insert the new node at the beginning of the array
		NewHistory.Nodes[0] = NewNodeIndex;
		return NewHistory;
	}

	KDOP_IDX_TYPE GetOldestNode() const
	{
		// Accesses the oldest node that this traversal history is tracking
		return Nodes[NodeHistoryLength - 1];
	}
};

class FRangeChecklessHeapAllocator : public FHeapAllocator
{
public:

	/** Don't want to lose performance on range checks in performance-critical ray tracing code. */
	enum { RequireRangeCheck = false };
};

/**
 * Line vs triangle intersection test.
 *
 * @param Start		Start of the line segment
 * @param End		End of the line segment
 * @param Dir		Direction of the line segment (not normalized, just End-Start)
 * @param V0		First vertex of the triangle
 * @param V1		Second vertex of the triangle
 * @param V2		Third vertex of the triangle
 * @param Normal	Triangle normal (including +W for the plane equation)
 * @param IntersectionTime	[in/out] Best intersection time so far (0..1), as in: IntersectionPoint = Start + IntersectionTime * Dir.
 * @return			true if the line intersected the triangle
 */
FORCEINLINE bool appLineCheckTriangle(const FVector4f& Start, const FVector4f& End, const FVector4f& Dir, const FVector4f& V0, const FVector4f& V1, const FVector4f& V2, const FVector4f& Normal, float& IntersectionTime)
{
	const float StartDist = FVectorUtils::PlaneDot(Normal, Start);
	const float EndDist = FVectorUtils::PlaneDot(Normal, End);

	// Check if the line is completely on one side of the triangle, or if it's co-planar.
#if 1
	if ((StartDist == EndDist) || (StartDist < -0.001f && EndDist < -0.001f) || (StartDist > 0.001f && EndDist > 0.001f))
#else
	if ( (StartDist * EndDist) > -0.0001f )
#endif
	{
		return false;
	}

	// Figure out when it will hit the triangle
	float Time = -StartDist / (EndDist - StartDist);

	// If this triangle is not closer than the previous hit, reject it
	if (Time < 0.f || Time >= IntersectionTime)
	{
		return false;
	}

	// Calculate the line's point of intersection with the node's plane
	const FVector4f& Intersection = Start + Dir * Time;
	const FVector4f* Verts[3] = 
	{ 
		&V0, &V1, &V2
	};

	// Check if the point of intersection is inside the triangle's edges.
	for( int32 SideIndex = 0; SideIndex < 3; SideIndex++ )
	{
		const FVector4f& SideDirection = Normal ^ (*Verts[(SideIndex + 1) % 3] - *Verts[SideIndex]);
		const float SideW = Dot3(SideDirection, *Verts[SideIndex]);
		const float DotW = Dot3(SideDirection, Intersection);
		if ((DotW - SideW) >= 0.001f)
		{
			return false;
		}
	}
	IntersectionTime = Time;
	return true;
}

/** ( -0.0001f, -0.0001f, -0.0001f, -0.0001f ) */
static const LmVectorRegister GSmallNegativeNumber = { -0.0001f, -0.0001f, -0.0001f, -0.0001f };
//extern const LmVectorRegister GSmallNegativeNumber;

/** ( 0.0001f, 0.0001f, 0.0001f, 0.0001f ) */
static const LmVectorRegister GSmallNumber = { 0.0001f, 0.0001f, 0.0001f, 0.0001f };
//extern const LmVectorRegister GSmallNumber;

static const LmVectorRegister GZeroVectorRegister = { 0, 0, 0, 0 };
static const LmVectorRegister GFullMaskVectorRegister = LmMakeVectorRegister((uint32)-1, (uint32)-1, (uint32)-1, (uint32)-1);

static const LmVectorRegister IndexNoneVectorRegister = LmMakeVectorRegister((uint32)INDEX_NONE, (uint32)INDEX_NONE, (uint32)INDEX_NONE, (uint32)INDEX_NONE);
static const LmVectorRegister VectorNegativeOne = LmMakeVectorRegister( -1.0f, -1.0f, -1.0f, -1.0f );

// LOD masks
static const LmVectorRegister HLODTreeIndexMask = LmMakeVectorRegister((uint32)0xFFFF0000, (uint32)0xFFFF0000, (uint32)0xFFFF0000, (uint32)0xFFFF0000);
static const LmVectorRegister HLODRangeStartIndexMask = LmMakeVectorRegister((uint32)0xFFFF, (uint32)0xFFFF, (uint32)0xFFFF, (uint32)0xFFFF);
static const LmVectorRegister HLODRangeEndIndexMask = LmMakeVectorRegister((uint32)0xFFFF0000, (uint32)0xFFFF0000, (uint32)0xFFFF0000, (uint32)0xFFFF0000);
static const LmVectorRegister HLODTreeIndexNoneRegister = LmMakeVectorRegister(0xFFFF0000 & (uint32)INDEX_NONE, 0xFFFF0000 & (uint32)INDEX_NONE, 0xFFFF0000 & (uint32)INDEX_NONE, 0xFFFF0000 & (uint32)INDEX_NONE);

static const LmVectorRegister LODIndexMask = LmMakeVectorRegister((uint32)0xFFFF, (uint32)0xFFFF, (uint32)0xFFFF, (uint32)0xFFFF);

/**
 * Line vs triangle intersection test. Tests 1 line against 4 triangles at once.
 *
 * @param Start		Start of the line segment
 * @param End		End of the line segment
 * @param Dir		Direction of the line segment (not normalized, just End-Start)
 * @param Triangle4	Four triangles
 * @param IntersectionTime	[in/out] Best intersection time so far (0..1), as in: IntersectionPoint = Start + IntersectionTime * Dir.
 * @return			Index (0-3) to specify which of the 4 triangles the line intersected, or -1 if none was found.
 */
FORCEINLINE int32 appLineCheckTriangleSOA(const FVector3SOA& Start, const FVector3SOA& End, const FVector3SOA& Dir, LmVectorRegister MeshIndex, LmVectorRegister LODIndices, LmVectorRegister HLODRange, const FTriangleSOA& Triangle4, bool bStaticAndOpaqueOnly, bool bTwoSidedCollision, bool bFlipSidedness, float& InOutIntersectionTime)
{
	LmVectorRegister TriangleMask;

	LmVectorRegister StartDist;
	StartDist = LmVectorMultiplyAdd( Triangle4.Normals.X, Start.X, Triangle4.Normals.W );
	StartDist = LmVectorMultiplyAdd( Triangle4.Normals.Y, Start.Y, StartDist );
	StartDist = LmVectorMultiplyAdd( Triangle4.Normals.Z, Start.Z, StartDist );

	LmVectorRegister EndDist;
	EndDist = LmVectorMultiplyAdd( Triangle4.Normals.X, End.X, Triangle4.Normals.W );
	EndDist = LmVectorMultiplyAdd( Triangle4.Normals.Y, End.Y, EndDist );
	EndDist = LmVectorMultiplyAdd( Triangle4.Normals.Z, End.Z, EndDist );

	// Are both end-points of the line on the same side of the triangle (or parallel to the triangle plane)?
	TriangleMask = LmVectorMask_LE( LmVectorMultiply( StartDist, EndDist ), GSmallNegativeNumber );
	if ( LmVectorMaskBits(TriangleMask) == 0 )
	{
		return -1;
	}

	if (bStaticAndOpaqueOnly)
	{
		// Only allow collision with opaque triangles if bStaticAndOpaqueOnly is true
		TriangleMask = LmVectorBitwiseAND(TriangleMask, Triangle4.StaticAndOpaqueMask);
	}

	// Backface culling
	if (!bTwoSidedCollision)
	{
		LmVectorRegister XMultiply = LmVectorMultiply(Dir.X, Triangle4.Normals.X);
		LmVectorRegister YMultiply = LmVectorMultiply(Dir.Y, Triangle4.Normals.Y);
		LmVectorRegister ZMultiply = LmVectorMultiply(Dir.Z, Triangle4.Normals.Z);
		// Dot product between the triangle normals and the ray direction
		LmVectorRegister OriginalDots = LmVectorAdd(XMultiply, LmVectorAdd(YMultiply, ZMultiply));
		// Flip the dot product if bFlipSidedness is true
		LmVectorRegister ModifiedDots = LmVectorMultiply(OriginalDots, (bFlipSidedness ? VectorNegativeOne : LmVectorOne()));
		// Reject backface hits of non-two sided triangles
		TriangleMask = LmVectorBitwiseAND(TriangleMask, LmVectorBitwiseOR(LmVectorMask_GE(ModifiedDots, LmVectorZero()), Triangle4.TwoSidedMask));

		if ( LmVectorMaskBits(TriangleMask) == 0 )
		{
			return -1;
		}
	}

	// Figure out when it will hit the triangle
	LmVectorRegister Time = LmVectorDivide( StartDist, LmVectorSubtract(StartDist, EndDist) );

	// If this triangle is not closer than the previous hit, reject it
	LmVectorRegister IntersectionTime = LmVectorLoadFloat1( &InOutIntersectionTime );
	TriangleMask = LmVectorBitwiseAND( TriangleMask, LmVectorMask_GE( Time, LmVectorZero() ) );
	TriangleMask = LmVectorBitwiseAND( TriangleMask, LmVectorMask_LT( Time, IntersectionTime ) );
	if ( LmVectorMaskBits(TriangleMask) == 0 )
	{
		return -1;
	}

	// Calculate the line's point of intersection with the node's plane
	const LmVectorRegister IntersectionX = LmVectorMultiplyAdd( Dir.X, Time, Start.X );
	const LmVectorRegister IntersectionY = LmVectorMultiplyAdd( Dir.Y, Time, Start.Y );
	const LmVectorRegister IntersectionZ = LmVectorMultiplyAdd( Dir.Z, Time, Start.Z );

	// Check if the point of intersection is inside the triangle's edges.
	for( int32 SideIndex = 0; SideIndex < 3; SideIndex++ )
	{
		const LmVectorRegister EdgeX = LmVectorSubtract( Triangle4.Positions[(SideIndex + 1) % 3].X, Triangle4.Positions[SideIndex].X );
		const LmVectorRegister EdgeY = LmVectorSubtract( Triangle4.Positions[(SideIndex + 1) % 3].Y, Triangle4.Positions[SideIndex].Y );
		const LmVectorRegister EdgeZ = LmVectorSubtract( Triangle4.Positions[(SideIndex + 1) % 3].Z, Triangle4.Positions[SideIndex].Z );
		const LmVectorRegister SideDirectionX = LmVectorSubtract( LmVectorMultiply( Triangle4.Normals.Y, EdgeZ ), LmVectorMultiply(Triangle4.Normals.Z, EdgeY) );
		const LmVectorRegister SideDirectionY = LmVectorSubtract( LmVectorMultiply( Triangle4.Normals.Z, EdgeX ), LmVectorMultiply(Triangle4.Normals.X, EdgeZ) );
		const LmVectorRegister SideDirectionZ = LmVectorSubtract( LmVectorMultiply( Triangle4.Normals.X, EdgeY ), LmVectorMultiply(Triangle4.Normals.Y, EdgeX) );
		LmVectorRegister SideW;
		SideW = LmVectorMultiply( SideDirectionX, Triangle4.Positions[SideIndex].X );
		SideW = LmVectorMultiplyAdd( SideDirectionY, Triangle4.Positions[SideIndex].Y, SideW );
		SideW = LmVectorMultiplyAdd( SideDirectionZ, Triangle4.Positions[SideIndex].Z, SideW );
		LmVectorRegister DotW;
		DotW = LmVectorMultiply( SideDirectionX, IntersectionX );
		DotW = LmVectorMultiplyAdd( SideDirectionY, IntersectionY, DotW );
		DotW = LmVectorMultiplyAdd( SideDirectionZ, IntersectionZ, DotW );
		TriangleMask = LmVectorBitwiseAND( TriangleMask, LmVectorMask_LT( LmVectorSubtract(DotW, SideW), GSmallNumber ) );
		if ( LmVectorMaskBits(TriangleMask) == 0 )
		{
			return -1;
		}
	}

	// Unpack HLOD Data
	LmVectorRegister HLODTreeIndexRay = LmVectorShiftRight(LmVectorBitwiseAND(HLODTreeIndexMask, LODIndices), 16);
	LmVectorRegister HLODTreeIndexTri = LmVectorShiftRight(LmVectorBitwiseAND(HLODTreeIndexMask, Triangle4.LODIndices), 16);
	LmVectorRegister HLODRangeStartRay = LmVectorBitwiseAND(HLODRangeStartIndexMask, HLODRange);
	LmVectorRegister HLODRangeStartTri = LmVectorBitwiseAND(HLODRangeStartIndexMask, Triangle4.HLODRange);
	LmVectorRegister HLODRangeEndRay = LmVectorShiftRight(LmVectorBitwiseAND(HLODRangeEndIndexMask, HLODRange), 16);
	LmVectorRegister HLODRangeEndTri = LmVectorShiftRight(LmVectorBitwiseAND(HLODRangeEndIndexMask, Triangle4.HLODRange), 16);

	LmVectorRegister IsDifferentHLOD = LmVectorMask_NE(HLODTreeIndexRay, HLODTreeIndexTri);
	LmVectorRegister IsTriNotAHLOD = LmVectorBitwiseOR(LmVectorMask_EQ(HLODTreeIndexTri, HLODTreeIndexNoneRegister), LmVectorMask_EQ(HLODTreeIndexTri, GZeroVectorRegister));

	// Ignore children of this HLOD node
	LmVectorRegister IsRayOutsideRange = LmVectorBitwiseOR(LmVectorMask_LT(HLODRangeStartRay, HLODRangeStartTri), LmVectorMask_GT(HLODRangeStartRay, HLODRangeEndTri));
	LmVectorRegister IsTriOutsideRange = LmVectorBitwiseOR(LmVectorMask_LT(HLODRangeStartTri, HLODRangeStartRay), LmVectorMask_GT(HLODRangeStartTri, HLODRangeEndRay));
	LmVectorRegister IsRayLeafNode = LmVectorMask_EQ(HLODRangeStartRay, HLODRangeEndRay);
	LmVectorRegister IsTriLeafNode = LmVectorMask_EQ(HLODRangeStartTri, HLODRangeEndTri);

	LmVectorRegister HLODMask = LmVectorBitwiseAND(IsRayOutsideRange, IsTriOutsideRange);
	HLODMask = LmVectorBitwiseOR(HLODMask, LmVectorBitwiseAND(IsRayLeafNode, IsTriLeafNode));

	// Allow if exact same mesh
	LmVectorRegister SameHLODMesh = LmVectorMask_EQ(HLODRange, Triangle4.HLODRange);
	HLODMask = LmVectorBitwiseOR(HLODMask, SameHLODMesh);

	// Ignore interactions between unrelated HLODs and non-HLODs
	HLODMask = LmVectorBitwiseOR(HLODMask, IsDifferentHLOD);
	HLODMask = LmVectorBitwiseOR(HLODMask, IsTriNotAHLOD);

	LmVectorRegister IsHLODNonLeafAndOtherMesh = LmVectorBitwiseAND(LmVectorMask_NE(HLODRangeStartTri, HLODRangeEndTri), IsDifferentHLOD);
	HLODMask = LmVectorBitwiseAND(HLODMask, LmVectorBitwiseXOR(IsHLODNonLeafAndOtherMesh, GFullMaskVectorRegister));
	
	TriangleMask = LmVectorBitwiseAND(TriangleMask, HLODMask);
	if (LmVectorMaskBits(TriangleMask) == 0)
	{
		return -1;
	}

	// Only allow intersections with the base LOD of other meshes and the LOD that is initiating the trace of the current mesh
	LmVectorRegister LODIndexRay = LmVectorBitwiseAND(LODIndices, LODIndexMask);
	LmVectorRegister LODIndexTri = LmVectorBitwiseAND(Triangle4.LODIndices, LODIndexMask);

	// LOD0OfOtherMeshMask = MeshIndex != TriangleMesh && IndexTriangleLODIndex == 0
	const LmVectorRegister LOD0OfOtherMeshMask = LmVectorBitwiseAND(LmVectorMask_NE(MeshIndex, Triangle4.MeshIndices), LmVectorMask_EQ(LODIndexTri, GZeroVectorRegister));
	// MatchingLODfCurrentMeshMask = MeshIndex == TriangleMeshIndex && LODIndex == TriangleLODIndex
	const LmVectorRegister MatchingLODfCurrentMeshMask = LmVectorBitwiseAND(LmVectorMask_EQ(MeshIndex, Triangle4.MeshIndices), LmVectorMask_EQ(LODIndexRay, LODIndexTri));
	// TriangleMask = TriangleMask && (LOD0OfOtherMeshMask || MatchingLODOfCurrentMeshMask)
	TriangleMask = LmVectorBitwiseAND(TriangleMask, LmVectorBitwiseOR(LOD0OfOtherMeshMask, MatchingLODfCurrentMeshMask));
	if ( LmVectorMaskBits(TriangleMask) == 0 )
	{
		return -1;
	}

	// Set all non-hitting times to 1.0
	Time = Lightmass::LmVectorSelect( LmVectorOne(), Time, TriangleMask );

	// Get the best intersection time out of the 4 possibilities.
	LmVectorRegister BestTimes = LmVectorMin( Time, LmVectorSwizzle(Time, 2, 3, 0, 0) );
	BestTimes = LmVectorMin( BestTimes, LmVectorSwizzle(BestTimes, 1, 0, 0, 0) );
	IntersectionTime = LmVectorReplicate( BestTimes, 0 );

	// Get the triangle index that corresponds to the best time.
	// NOTE: This will pick the first triangle, in case there are multiple hits at the same spot.
	int32 SubIndex = LmVectorMaskBits( LmVectorMask_EQ( Time, IntersectionTime ) );
	SubIndex = appCountTrailingZeros( SubIndex );

	// Return results.
	LmVectorStoreFloat1( IntersectionTime, &InOutIntersectionTime );
	return SubIndex;
}

// This structure is used during the build process. It contains the triangle's
// centroid for calculating which plane it should be split or not with
template<typename KDOP_IDX_TYPE>
struct FkDOPBuildCollisionTriangle
{
	/**
	 * First vertex in the triangle
	 */
	FVector4f V0;
	/**
	 * Second vertex in the triangle
	 */
	FVector4f V1;
	/**
	 * Third vertex in the triangle
	 */
	FVector4f V2;

#if CACHE_BUILD_TEMPORARIES
private:
	/**
 	 * Centroid of the triangle used for determining which bounding volume to
  	 * place the triangle in
	 */
	FVector4f Centroid;

	/** Cached local normal used by line/ triangle collision code. */
	FVector4f LocalNormal;
public:
#endif

	/** Index of the mesh this triangle belongs to. */
	int32 MeshIndex;

	/** Index of the LOD this triangle belongs to. */
	uint32 LODIndices;

	/** Range of IDs for HLOD nodes children. */
	uint32 HLODRange;
	
	/** True if the triangle is two sided. */
	uint32 bTwoSided : 1;

	/** True if the triangle is opaque. */
	uint32 bStaticAndOpaque : 1;

	/** The material of this triangle */
	KDOP_IDX_TYPE MaterialIndex;

#if CACHE_BUILD_TEMPORARIES
	FORCEINLINE const FVector4f& GetCentroid() const
	{
		return Centroid;
	}
	FORCEINLINE const FVector4f& GetLocalNormal() const
	{
		return LocalNormal;
	}
#else
	inline FVector4f GetCentroid() const
	{
		return (V0 + V1 + V2) / 3.f;
	}
	inline FVector4f GetLocalNormal() const
	{
		FVector4f LocalNormal = ((V1 - V2) ^ (V0 - V2)).GetSafeNormal();
		LocalNormal.W = Dot3(V0, LocalNormal);
		return LocalNormal;
	}
#endif

	/**
	 * Sets the indices, material index, calculates the centroid using the
	 * specified triangle vertex positions
	 */
	FkDOPBuildCollisionTriangle(
		KDOP_IDX_TYPE InMaterialIndex,
		const FVector4f& vert0,const FVector4f& vert1,const FVector4f& vert2,
		int32 InMeshIndex, 
		uint32 InLODIndices,
		uint32 InHLODRange,
		bool bInTwoSided,
		bool bInStaticAndOpaque) :
		V0(vert0), V1(vert1), V2(vert2),
		MeshIndex(InMeshIndex),
		LODIndices(InLODIndices),
		HLODRange(InHLODRange),
		bTwoSided(bInTwoSided),
		bStaticAndOpaque(bInStaticAndOpaque),
		MaterialIndex(InMaterialIndex)
	{
#if CACHE_BUILD_TEMPORARIES
		// Calculate the centroid for the triangle
		Centroid = (V0 + V1 + V2) / 3.f;
		// Calculate the local normal used by line check code.
		LocalNormal = ((V1 - V2) ^ (V0 - V2)).GetSafeNormal();
		LocalNormal.W = Dot3(V0, LocalNormal);
#endif
	}
};

// Forward declarations
template <typename COLL_DATA_PROVIDER,typename KDOP_IDX_TYPE> struct TkDOPNode;
template <typename COLL_DATA_PROVIDER,typename KDOP_IDX_TYPE> struct TkDOPTree;
template <typename COLL_DATA_PROVIDER,typename KDOP_IDX_TYPE> struct TkDOPLineCollisionCheck;
template <typename COLL_DATA_PROVIDER,typename KDOP_IDX_TYPE> struct TkDOPBoxCollisionCheck;

/**
 * Holds the min/max planes that make up a set of 4 bounding volumes.
 */
struct FMultiBox
{
	/**
	 * Min planes for this set of bounding volumes. Array index is X/Y/Z.
	 */
	MS_ALIGN(16) FVector4f Min[3]  GCC_ALIGN(16);

	/** 
	 * Max planes for this set of bounding volumes. Array index is X/Y/Z.
	 */
	FVector4f Max[3];

	/**
	 * Sets the box at the passed in index to the passed in box.
	 *
	 * @param	BoundingVolumeIndex		Index of box to set
	 * @param	Box						Box to set
	 */
	void SetBox( int32 BoundingVolumeIndex, const FBox3f& Box )
	{
		Min[0].Component(BoundingVolumeIndex) = Box.Min.X;
		Min[1].Component(BoundingVolumeIndex) = Box.Min.Y;
		Min[2].Component(BoundingVolumeIndex) = Box.Min.Z;
		Max[0].Component(BoundingVolumeIndex) = Box.Max.X;
		Max[1].Component(BoundingVolumeIndex) = Box.Max.Y;
		Max[2].Component(BoundingVolumeIndex) = Box.Max.Z;
	}

	/**
	 * Returns the bounding volume at the passed in index.
	 *
	 * @param	BoundingVolumeIndex		Index of bounding volume to return
	 *
	 * @return Bounding volume at the passed in index
	 */
	FBox3f GetBox( int32 BoundingVolumeIndex )
	{
		FBox3f Box;
		Box.Min = FVector4f(Min[0][BoundingVolumeIndex],Min[1][BoundingVolumeIndex],Min[2][BoundingVolumeIndex],1);
		Box.Max = FVector4f(Max[0][BoundingVolumeIndex],Max[1][BoundingVolumeIndex],Max[2][BoundingVolumeIndex],1);
		return Box;
	}
};

#define kDOPArray TArray

namespace BoxTriangleIntersectionInternal
{

struct FTriangle
{
	FVector3f Vertices[3];
};

struct FOverlapInterval
{
	float Min;
	float Max;
};

FORCEINLINE FOverlapInterval GetInterval(const FTriangle& Triangle, const FVector3f& Vector)
{
	FOverlapInterval Result;
	Result.Min = Result.Max = FVector3f::DotProduct(Vector, Triangle.Vertices[0]);

	for (int32 i = 1; i < 3; ++i)
	{
		float Projection = FVector3f::DotProduct(Vector, Triangle.Vertices[i]);
		Result.Min = FMath::Min(Result.Min, Projection);
		Result.Max = FMath::Max(Result.Max, Projection);
	}

	return Result;
}

FORCEINLINE FOverlapInterval GetInterval(const FBox3f& Box, const FVector3f& Vector)
{
	FVector3f BoxVertices[8] =
	{
		FVector3f(Box.Min.X, Box.Max.Y, Box.Max.Z),
		FVector3f(Box.Min.X, Box.Max.Y, Box.Min.Z),
		FVector3f(Box.Min.X, Box.Min.Y, Box.Max.Z),
		FVector3f(Box.Min.X, Box.Min.Y, Box.Min.Z),
		FVector3f(Box.Max.X, Box.Max.Y, Box.Max.Z),
		FVector3f(Box.Max.X, Box.Max.Y, Box.Min.Z),
		FVector3f(Box.Max.X, Box.Min.Y, Box.Max.Z),
		FVector3f(Box.Max.X, Box.Min.Y, Box.Min.Z)
	};

	FOverlapInterval Result;
	Result.Min = Result.Max = FVector3f::DotProduct(Vector, BoxVertices[0]);

	for (int32 i = 1; i < UE_ARRAY_COUNT(BoxVertices); ++i)
	{
		float Projection = FVector3f::DotProduct(Vector, BoxVertices[i]);
		Result.Min = FMath::Min(Result.Min, Projection);
		Result.Max = FMath::Max(Result.Max, Projection);
	}

	return Result;
}

FORCEINLINE bool OverlapOnAxis(const FBox3f& Box, const FTriangle& Triangle, const FVector3f& Vector)
{
	FOverlapInterval A = GetInterval(Box, Vector);
	FOverlapInterval B = GetInterval(Triangle, Vector);
	return ((B.Min <= A.Max) && (A.Min <= B.Max));
}

FORCEINLINE bool IntersectTriangleAndAABB(const FTriangle& Triangle, const FBox3f& Box)
{
	FVector3f TriangleEdge0 = Triangle.Vertices[1] - Triangle.Vertices[0];
	FVector3f TriangleEdge1 = Triangle.Vertices[2] - Triangle.Vertices[1];
	FVector3f TriangleEdge2 = Triangle.Vertices[0] - Triangle.Vertices[2];

	FVector3f BoxNormal0(1.0f, 0.0f, 0.0f);
	FVector3f BoxNormal1(0.0f, 1.0f, 0.0f);
	FVector3f BoxNormal2(0.0f, 0.0f, 1.0f);

	FVector3f TestDirections[13] =
	{
		// Separating axes from the box normals
		BoxNormal0,
		BoxNormal1,
		BoxNormal2,
		// One separating axis for the triangle normal
		FVector3f::CrossProduct(TriangleEdge0, TriangleEdge1),
		// Separating axes for the triangle edges
		FVector3f::CrossProduct(BoxNormal0, TriangleEdge0),
		FVector3f::CrossProduct(BoxNormal0, TriangleEdge1),
		FVector3f::CrossProduct(BoxNormal0, TriangleEdge2),
		FVector3f::CrossProduct(BoxNormal1, TriangleEdge0),
		FVector3f::CrossProduct(BoxNormal1, TriangleEdge1),
		FVector3f::CrossProduct(BoxNormal1, TriangleEdge2),
		FVector3f::CrossProduct(BoxNormal2, TriangleEdge0),
		FVector3f::CrossProduct(BoxNormal2, TriangleEdge1),
		FVector3f::CrossProduct(BoxNormal2, TriangleEdge2)
	};

	for (int i = 0; i < UE_ARRAY_COUNT(TestDirections); ++i)
	{
		if (!OverlapOnAxis(Box, Triangle, TestDirections[i]))
		{
			// If we don't overlap on a single axis, the shapes do not intersect
			return false;
		}
	}

	return true;
}
}

/**
 * A node in the kDOP tree. The node contains the kDOP volume that encompasses
 * it's children and/or triangles
 */
template <typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE>
struct TkDOPNode
{
	/** Exposes data provider type to clients. */
	typedef COLL_DATA_PROVIDER							DataProviderType;

	/** Exposes node type to clients. */
	typedef TkDOPNode<DataProviderType,KDOP_IDX_TYPE>	NodeType;

	/** Set of bounding volumes for child nodes. */
	FMultiBox BoundingVolumes;

	// Note this isn't smaller since 4 byte alignment will take over anyway
	bool bIsLeaf;
	uint8 Occupancy;

	// Union of either child kDOP nodes or a list of enclosed triangles
	union
	{
		// This structure contains the left and right child kDOP indices
		// These index values correspond to the array in the FkDOPTree
		struct
		{
			KDOP_IDX_TYPE LeftNode;
			KDOP_IDX_TYPE RightNode;
		} n;
		// This structure contains the list of enclosed triangles
		// These index values correspond to the triangle information in the
		// FkDOPTree using the start and count as the means of delineating
		// which triangles are involved
		struct
		{
			KDOP_IDX_TYPE NumTriangles;
			KDOP_IDX_TYPE StartIndex;
		} t;
	};

	/**
	 * Inits the data to no child nodes and an inverted volume
	 */
	FORCEINLINE TkDOPNode()
	{
		n.LeftNode = ((KDOP_IDX_TYPE) -1);
        n.RightNode = ((KDOP_IDX_TYPE) -1);
	}

	/**
	 * The slab testing algorithm is based on the following papers. We chose to use the
	 * faster final hit determination, which means we'll get some false positives.
     *
	 * "A Cross-Platform Framework for Interactive Ray Tracing"
	 * Markus Geimer, Stefan Mueller
	 * http://www.uni-koblenz.de/~cg/publikationen/cp_raytrace.pdf
	 *
  	 * "Efficiency Issues for Ray Tracing"
	 * Brian Smits
	 * http://www.cs.utah.edu/~bes/papers/fastRT/paper-node10.html
	 *
	 * We don't need to handle +/- INF, but we have permission to use Thierry 
	 * Berger-Perrin trick available below.
	 *
     * http://www.flipcode.com/archives/SSE_RayBox_Intersection_Test.shtml
	 *
	 * @param	Check				Information about the ray to trace
	 * @param	HitTime	[out]	Time of hit
	 */
	FORCEINLINE void LineCheckBounds(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check, FVector4f& HitTime, int32 NodeHit[4] ) const
	{
#define PLAIN_C 0
#if PLAIN_C
		for( int32 BoxIndex=0; BoxIndex<4; BoxIndex++ )
		{
			// 0: Create constants
			FVector4f BoxMin( BoundingVolumes.Min[0][BoxIndex], BoundingVolumes.Min[1][BoxIndex], BoundingVolumes.Min[2][BoxIndex], 0 );
			FVector4f BoxMax( BoundingVolumes.Max[0][BoxIndex], BoundingVolumes.Max[1][BoxIndex], BoundingVolumes.Max[2][BoxIndex], 0 );

			// 1: Calculate slabs.
			FVector4f Slab1 = (BoxMin - Check.LocalStart) * Check.LocalOneOverDir;
			FVector4f Slab2 = (BoxMax - Check.LocalStart) * Check.LocalOneOverDir;

			// 2: Figure out per component min/ max
			FVector4f SlabMin = FVector4f( Lightmass::Min(Slab1.X, Slab2.X), Lightmass::Min(Slab1.Y, Slab2.Y), Lightmass::Min(Slab1.Z, Slab2.Z), Lightmass::Min(Slab1.W, Slab2.W) );
			FVector4f SlabMax = FVector4f( Lightmass::Max(Slab1.X, Slab2.X), Lightmass::Max(Slab1.Y, Slab2.Y), Lightmass::Max(Slab1.Z, Slab2.Z), Lightmass::Max(Slab1.W, Slab2.W) );

			// 3: Figure out global min/ max
			float MinTime = Max3( SlabMin.X, SlabMin.Y, SlabMin.Z );
			float MaxTime = Min3( SlabMax.X, SlabMax.Y, SlabMax.Z );

			// 4: Calculate hit time and determine whether there was a hit.
			HitTime[BoxIndex] = MinTime;			
			NodeHit[BoxIndex] = (MaxTime >= 0 && MaxTime >= MinTime && MinTime < Check.Result->Time) ? 0xFFFFFFFF : 0;
		}
#else
		// 0: load everything into registers
		const LmVectorRegister OriginX		= LmVectorSetFloat1( Check.LocalStart.X );
		const LmVectorRegister OriginY		= LmVectorSetFloat1( Check.LocalStart.Y );
		const LmVectorRegister OriginZ		= LmVectorSetFloat1( Check.LocalStart.Z );
		const LmVectorRegister InvDirX		= LmVectorSetFloat1( Check.LocalOneOverDir.X );
		const LmVectorRegister InvDirY		= LmVectorSetFloat1( Check.LocalOneOverDir.Y );
		const LmVectorRegister InvDirZ		= LmVectorSetFloat1( Check.LocalOneOverDir.Z );
		const LmVectorRegister CurrentHitTime	= LmVectorSetFloat1( Check.Result->Time );
		// Boxes are FVector2f so we need to unshuffle the data.
		const LmVectorRegister BoxMinX		= LmVectorLoadAligned( &BoundingVolumes.Min[0] );
		const LmVectorRegister BoxMinY		= LmVectorLoadAligned( &BoundingVolumes.Min[1] );
		const LmVectorRegister BoxMinZ		= LmVectorLoadAligned( &BoundingVolumes.Min[2] );
		const LmVectorRegister BoxMaxX		= LmVectorLoadAligned( &BoundingVolumes.Max[0] );
		const LmVectorRegister BoxMaxY		= LmVectorLoadAligned( &BoundingVolumes.Max[1] );
		const LmVectorRegister BoxMaxZ		= LmVectorLoadAligned( &BoundingVolumes.Max[2] );

		// 1: Calculate slabs.
		const LmVectorRegister BoxMinSlabX	= LmVectorMultiply( LmVectorSubtract( BoxMinX, OriginX ), InvDirX );
		const LmVectorRegister BoxMinSlabY	= LmVectorMultiply( LmVectorSubtract( BoxMinY, OriginY ), InvDirY );
		const LmVectorRegister BoxMinSlabZ	= LmVectorMultiply( LmVectorSubtract( BoxMinZ, OriginZ ), InvDirZ );		
		const LmVectorRegister BoxMaxSlabX	= LmVectorMultiply( LmVectorSubtract( BoxMaxX, OriginX ), InvDirX );
		const LmVectorRegister BoxMaxSlabY	= LmVectorMultiply( LmVectorSubtract( BoxMaxY, OriginY ), InvDirY );
		const LmVectorRegister BoxMaxSlabZ	= LmVectorMultiply( LmVectorSubtract( BoxMaxZ, OriginZ ), InvDirZ );

		// 2: Figure out per component min/ max
		const LmVectorRegister SlabMinX		= LmVectorMin( BoxMinSlabX, BoxMaxSlabX );
		const LmVectorRegister SlabMinY		= LmVectorMin( BoxMinSlabY, BoxMaxSlabY );
		const LmVectorRegister SlabMinZ		= LmVectorMin( BoxMinSlabZ, BoxMaxSlabZ );
		const LmVectorRegister SlabMaxX		= LmVectorMax( BoxMinSlabX, BoxMaxSlabX );
		const LmVectorRegister SlabMaxY		= LmVectorMax( BoxMinSlabY, BoxMaxSlabY );
		const LmVectorRegister SlabMaxZ		= LmVectorMax( BoxMinSlabZ, BoxMaxSlabZ );
		
		// 3: Figure out global min/ max
		const LmVectorRegister SlabMinXY		= LmVectorMax( SlabMinX , SlabMinY );
		const LmVectorRegister MinTime		= LmVectorMax( SlabMinXY, SlabMinZ );
		const LmVectorRegister SlabMaxXY		= LmVectorMin( SlabMaxX , SlabMaxY );
		const LmVectorRegister MaxTime		= LmVectorMin( SlabMaxXY, SlabMaxZ );

		// 4: Calculate hit time and determine whether there was a hit.		
		LmVectorStoreAligned( MinTime, &HitTime );
		const LmVectorRegister OutNodeHit		= LmVectorBitwiseAND( LmVectorCompareGE( MaxTime, LmVectorZero() ), LmVectorCompareGE( MaxTime, MinTime ) );
		const LmVectorRegister CloserNodeHit	= LmVectorBitwiseAND( OutNodeHit, LmVectorCompareGT( CurrentHitTime, MinTime ) );
		LmVectorStoreAligned( CloserNodeHit, (float*) NodeHit );
#endif
	}

	/** 
	 * Determines the line in the FkDOPLineCollisionCheck intersects this node. It
	 * also will check the child nodes if it is not a leaf, otherwise it will check
	 * against the triangle data.
	 *
	 * @param Check -- The aggregated line check data
	 */
	bool LineCheck(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check, TTraversalHistory<KDOP_IDX_TYPE> History) const
	{
		bool bHit = 0;
		// If this is a node, check the two child nodes and pick the closest one
		// to recursively check against and only check the second one if there is
		// not a hit or the hit returned is further out than the second node
		if (bIsLeaf == 0)
		{
			SLOW_KDOP_STATS(FPlatformAtomics::InterlockedIncrement((SSIZE_T*)&GKDOPParentNodesTraversed));
			// Check both left and right node at the same time.
			FVector4f NodeHitTime;
			MS_ALIGN(16) int32	NodeHit[4] GCC_ALIGN(16);
			LineCheckBounds( Check, NodeHitTime, NodeHit );

			// Left node was hit.
			if( NodeHit[0] )
			{
				// Left and Right node are hit.
				if( NodeHit[1] )
				{
					// Left node is closer than right node.
					if( NodeHitTime.X < NodeHitTime.Y )
					{
						bHit = Check.Nodes[n.LeftNode].LineCheckPreCalculated(Check,NodeHitTime,History.AddNode(n.LeftNode),NodeHit);
						// Only check right node if it could possibly be a closer hit
						if(	NodeHitTime.Y < Check.Result->Time
						// No need to check if we have a hit and don't care about closest
						&&	(!bHit || Check.bFindClosestIntersection))
						{
							bHit |= Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));
						}
					}
					// Right node is closer than left node.
					else
					{
						bHit = Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));
						// Only check left node if it could possibly be a closer hit
						if(	NodeHitTime.X < Check.Result->Time
						// No need to check if we have a hit and don't care about closest
						&&	(!bHit || Check.bFindClosestIntersection))
						{
							bHit |= Check.Nodes[n.LeftNode].LineCheckPreCalculated(Check,NodeHitTime,History.AddNode(n.LeftNode),NodeHit);
						}
					}
				}
				// Only left node was hit.
				else
				{
					bHit = Check.Nodes[n.LeftNode].LineCheckPreCalculated(Check,NodeHitTime,History.AddNode(n.LeftNode),NodeHit);
				}
			}
			// Left node was not hit.
			else
			{
				// Only right node was hit.
				if( NodeHit[1] )
				{			
					bHit = Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));					
				}
				// No node was hit.
				else
				{
					return false;
				}
			}
		}
		else
		{
			SLOW_KDOP_STATS(FPlatformAtomics::InterlockedIncrement((SSIZE_T*)&GKDOPLeafNodesTraversed));
			// This is a leaf, check the triangles for a hit
			bHit = LineCheckTriangles(Check,History);
		}
		return bHit;
	}

	/** 
	 * Determines the line in the FkDOPLineCollisionCheck intersects this node. It
	 * also will check the child nodes if it is not a leaf, otherwise it will check
	 * against the triangle data.
	 *
	 * @param Check -- The aggregated line check data
	 */
	bool LineCheckPreCalculated(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check, const FVector4f& NodeHitTime, TTraversalHistory<KDOP_IDX_TYPE> History, int32* NodeHit) const
	{
		bool bHit = 0;
		// If this is a node, check the two child nodes and pick the closest one
		// to recursively check against and only check the second one if there is
		// not a hit or the hit returned is further out than the second node
		if (bIsLeaf == 0)
		{
			SLOW_KDOP_STATS(FPlatformAtomics::InterlockedIncrement((SSIZE_T*)&GKDOPParentNodesTraversed));
			// Left node was hit.
			if( NodeHit[2] )
			{
				// Left and Right node are hit.
				if( NodeHit[3] )
				{
					// Left node is closer than right node.
					if( NodeHitTime.Z < NodeHitTime.W )
					{
						bHit = Check.Nodes[n.LeftNode].LineCheck(Check,History.AddNode(n.LeftNode));
						// Only check right node if it could possibly be a closer hit
						if(	NodeHitTime.W < Check.Result->Time
						// No need to check if we have a hit and don't care about closest
						&&	(!bHit || Check.bFindClosestIntersection))
						{
							bHit |= Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));
						}
					}
					// Right node is closer than left node.
					else
					{
						bHit = Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));
						// Only check left node if it could possibly be a closer hit
						if(	NodeHitTime.Z < Check.Result->Time
						// No need to check if we have a hit and don't care about closest
						&&	(!bHit || Check.bFindClosestIntersection))
						{
							bHit |= Check.Nodes[n.LeftNode].LineCheck(Check,History.AddNode(n.LeftNode));
						}
					}
				}
				// Only left node was hit.
				else
				{
					bHit = Check.Nodes[n.LeftNode].LineCheck(Check,History.AddNode(n.LeftNode));
				}
			}
			// Left node was not hit.
			else
			{
				// Only right node was hit.
				if( NodeHit[3] )
				{			
					bHit = Check.Nodes[n.RightNode].LineCheck(Check,History.AddNode(n.RightNode));					
				}
				// No node was hit.
				else
				{
					return false;
				}
			}
		}
		else
		{
			SLOW_KDOP_STATS(FPlatformAtomics::InterlockedIncrement((SSIZE_T*)&GKDOPLeafNodesTraversed));
			// This is a leaf, check the triangles for a hit
			bHit = LineCheckTriangles(Check,History);
		}
		return bHit;
	}

	/**
	 * Works through the list of triangles in this node checking each one for a
	 * collision.
	 *
	 * @param Check -- The aggregated line check data
	 */
	bool LineCheckTriangles(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check, const TTraversalHistory<KDOP_IDX_TYPE>& History) const
	{
		// Assume a miss
		bool bHit = false;
		for ( KDOP_IDX_TYPE SOAIndex = t.StartIndex; SOAIndex < (t.StartIndex + t.NumTriangles); SOAIndex++ )
		{
			const FTriangleSOA& TriangleSOA = Check.SOATriangles[SOAIndex];
			SLOW_KDOP_STATS(FPlatformAtomics::InterlockedAdd((SSIZE_T*)&GKDOPTrianglesTraversed, 4));
			SLOW_KDOP_STATS(FPlatformAtomics::InterlockedAdd((SSIZE_T*)&GKDOPTrianglesTraversedReal, Occupancy));
			int32 SubIndex = appLineCheckTriangleSOA( Check.StartSOA, Check.EndSOA, Check.DirSOA, Check.MeshIndexRegister, Check.LODIndicesRegister, Check.HLODRangeRegister, TriangleSOA, Check.bStaticAndOpaqueOnly, Check.bTwoSidedCollision, Check.bFlipSidedness, Check.Result->Time );
			if ( SubIndex >= 0 )
			{
				bHit = true;
				Check.LocalHitNormal.X = LmVectorGetComponent(TriangleSOA.Normals.X, SubIndex);
				Check.LocalHitNormal.Y = LmVectorGetComponent(TriangleSOA.Normals.Y, SubIndex);
				Check.LocalHitNormal.Z = LmVectorGetComponent(TriangleSOA.Normals.Z, SubIndex);
				Check.Result->Item = TriangleSOA.Payload[SubIndex];
				Check.HitNodeIndex = History.GetOldestNode();

				// Early out if we don't care about the closest intersection.
				if( !Check.bFindClosestIntersection )
				{
					break;
				}
			}
		}
		return bHit;
	}

	bool BoxCheckTriangles(TkDOPBoxCollisionCheck<COLL_DATA_PROVIDER, KDOP_IDX_TYPE>& Check) const
	{
		for (KDOP_IDX_TYPE SOAIndex = t.StartIndex; SOAIndex < (t.StartIndex + t.NumTriangles); SOAIndex++)
		{
			const FTriangleSOA& TriangleSOA = Check.SOATriangles[SOAIndex];
			for (int32 SubIndex = 0; SubIndex < 4; SubIndex++)
			{
				BoxTriangleIntersectionInternal::FTriangle Triangle;

				Triangle.Vertices[0].X = LmVectorGetComponent(TriangleSOA.Positions[0].X, SubIndex);
				Triangle.Vertices[0].Y = LmVectorGetComponent(TriangleSOA.Positions[0].Y, SubIndex);
				Triangle.Vertices[0].Z = LmVectorGetComponent(TriangleSOA.Positions[0].Z, SubIndex);

				Triangle.Vertices[1].X = LmVectorGetComponent(TriangleSOA.Positions[1].X, SubIndex);
				Triangle.Vertices[1].Y = LmVectorGetComponent(TriangleSOA.Positions[1].Y, SubIndex);
				Triangle.Vertices[1].Z = LmVectorGetComponent(TriangleSOA.Positions[1].Z, SubIndex);

				Triangle.Vertices[2].X = LmVectorGetComponent(TriangleSOA.Positions[2].X, SubIndex);
				Triangle.Vertices[2].Y = LmVectorGetComponent(TriangleSOA.Positions[2].Y, SubIndex);
				Triangle.Vertices[2].Z = LmVectorGetComponent(TriangleSOA.Positions[2].Z, SubIndex);

				if (BoxTriangleIntersectionInternal::IntersectTriangleAndAABB(Triangle, Check.Box))
				{
					Check.Item = TriangleSOA.Payload[SubIndex];
					return true;
				}
			}
		}

		return false;
	}

	void BoxCheckBounds(TkDOPBoxCollisionCheck<COLL_DATA_PROVIDER, KDOP_IDX_TYPE>& Check, bool NodeHit[2]) const
	{
		for (int32 BoxIndex = 0; BoxIndex < 2; BoxIndex++)
		{
			// 0: Create constants
			FVector3f BoxMin(BoundingVolumes.Min[0][BoxIndex], BoundingVolumes.Min[1][BoxIndex], BoundingVolumes.Min[2][BoxIndex]);
			FVector3f BoxMax(BoundingVolumes.Max[0][BoxIndex], BoundingVolumes.Max[1][BoxIndex], BoundingVolumes.Max[2][BoxIndex]);

			FBox3f ChildBox(BoxMin, BoxMax);

			NodeHit[BoxIndex] = Check.Box.Intersect(ChildBox);
		}
	}

	bool BoxCheck(TkDOPBoxCollisionCheck<COLL_DATA_PROVIDER, KDOP_IDX_TYPE>& Check) const
	{
		if (bIsLeaf == 0)
		{
			// Check both left and right node at the same time.
			bool NodeHit[2];
			BoxCheckBounds(Check, NodeHit);

			if (NodeHit[0])
			{
				if (Check.Nodes[n.LeftNode].BoxCheck(Check))
					return true;
			}

			if (NodeHit[1])
			{
				if (Check.Nodes[n.RightNode].BoxCheck(Check))
					return true;
			}

			return false;
		}
		else
		{
			return BoxCheckTriangles(Check);
		}
	}
};

/**
 * This is the tree of kDOPs that spatially divides the static mesh. It is
 * a binary tree of kDOP nodes.
 */
template<typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE>
struct TkDOPTree
{
	/** Exposes data provider type to clients. */
	typedef COLL_DATA_PROVIDER							DataProviderType;

	/** Exposes node type to clients. */
	typedef TkDOPNode<DataProviderType,KDOP_IDX_TYPE>	NodeType;

	/** Statistics --------------------------------------*/
	/** Number of triangles in the aggregate mesh kDOP. */
	int32 GKDOPTriangles = 0;
	/** Number of internal nodes in the aggregate mesh kDOP. */
	int32 GKDOPNodes = 0;
	/** Number of leaf nodes in the aggregate mesh kDOP. */
	int32 GKDOPNumLeaves = 0;
	/** Maximum number of triangles per leaf node during the splitting process */
	int32 GKDOPMaxTrisPerLeaf = DEFAULT_MAX_TRIS_PER_LEAF;
	/** Total number of kDOP internal nodes traversed when tracing rays. */
	volatile uint64 GKDOPParentNodesTraversed = 0;
	/** Total number of kDOP leaf nodes traversed when tracing rays. */
	volatile uint64 GKDOPLeafNodesTraversed = 0;
	/** Total number of kDOP triangles tested when tracing rays. */
	volatile uint64 GKDOPTrianglesTraversed = 0;
	volatile uint64 GKDOPTrianglesTraversedReal = 0;
	size_t kDOPPreallocatedMemory = 0u;
	float kDOPBuildTime = 0.0f;

	/** The list of nodes contained within this tree. Node 0 is always the root node. */
	kDOPArray<NodeType, FRangeChecklessHeapAllocator> Nodes;

	/** The list of collision triangles in this tree. */
	kDOPArray<FTriangleSOA, FRangeChecklessHeapAllocator> SOATriangles;

	/**
	 * Creates the root node and recursively splits the triangles into smaller
	 * volumes
	 *
	 * @param BuildTriangles -- The list of triangles to use for the build process
	 */
	void Build(TArray<FkDOPBuildCollisionTriangle<KDOP_IDX_TYPE> >& BuildTriangles)
	{
		kDOPBuildTime = 0;
		{
			FScopedRDTSCTimer kDOPBuildTimer(kDOPBuildTime);

			// Empty the current set of nodes and preallocate the memory so it doesn't
			// reallocate memory too much while we are recursively walking the tree
			// With near-perfect packing, we could easily size these to be
			// Nodes = (n / 2) + 1 and SOATriangles = (n / 4) + 1
			Nodes.Empty(BuildTriangles.Num() / 2);
			SOATriangles.Empty(BuildTriangles.Num() / 3);

			size_t NodesSize = (size_t)Nodes.GetTypeSize() * (size_t)Nodes.Max();
			size_t SOATrianglesSize = (size_t)SOATriangles.GetTypeSize() * (size_t)SOATriangles.Max();
			kDOPPreallocatedMemory = NodesSize + SOATrianglesSize;

			// Add the root node
			Nodes.AddZeroed();

			// Now tell that node to recursively subdivide the entire set of triangles
			SplitTriangleList(0,0,BuildTriangles.Num(),BuildTriangles);

			// Don't waste memory.
			Nodes.Shrink();
			SOATriangles.Shrink();
		}
	}

	
	/**
	 * Determines if the node is a leaf or not. If it is not a leaf, it subdivides
	 * the list of triangles again adding two child nodes and splitting them on
	 * the mean (splatter method). Otherwise it sets up the triangle information.
	 *
	 * @param Start -- The triangle index to start processing with
	 * @param NumTris -- The number of triangles to process
	 * @param BuildTriangles -- The list of triangles to use for the build process
	 * @param Nodes -- The list of nodes in this tree
	 * @return bounding box for this node
	 */
	FBox3f SplitTriangleList(KDOP_IDX_TYPE NodeIndex, int32 Start, int32 NumTris, TArray<FkDOPBuildCollisionTriangle<KDOP_IDX_TYPE> >& BuildTriangles)
	{
		// This local node pointer will have to be updated whenever a Nodes reallocation can occur
		NodeType* Node = &Nodes[NodeIndex];

		// Figure out if we are a leaf node or not
		if (NumTris > GKDOPMaxTrisPerLeaf)
		{
			// Still too many triangles, so continue subdividing the triangle list
			Node->bIsLeaf = 0;
			Node->Occupancy = 0;
			int32 BestPlane = -1;
			float BestMean = 0.f;
			float BestVariance = 0.f;

			// Determine how to split using the splatter algorithm
			{
				double Mean[NUM_PLANES] = { 0 };
				double Variance[NUM_PLANES] = { 0 };

				// Compute the mean for the triangle list
				for (int32 nTriangle = Start; nTriangle < Start + NumTris; nTriangle++)
				{
					// Project the centroid of the triangle against the plane
					// normals and accumulate to find the total projected
					// weighting
					FVector4f Centroid = BuildTriangles[nTriangle].GetCentroid();

					for (int32 nPlane = 0; nPlane < NUM_PLANES; nPlane++)
					{
						Mean[nPlane] += Centroid[nPlane];
					}
				}

				// Divide by the number of triangles to get the average
				for (int32 nPlane = 0; nPlane < NUM_PLANES; nPlane++)
				{
					Mean[nPlane] /= NumTris;
				}

				// Compute variance of the triangle list
				for (int32 nTriangle = Start; nTriangle < Start + NumTris; nTriangle++)
				{
					// Project the centroid again
					FVector4f Centroid = BuildTriangles[nTriangle].GetCentroid();

					// Now calculate the variance and accumulate it
					for (int32 nPlane = 0; nPlane < NUM_PLANES; nPlane++)
					{
						Variance[nPlane] += (Centroid[nPlane] - Mean[nPlane]) * (Centroid[nPlane] - Mean[nPlane]);
					}
				}

				// Determine which plane is the best to split on
				for (int32 nPlane = 0; nPlane < NUM_PLANES; nPlane++)
				{
					// Get the average variance
					Variance[nPlane] /= NumTris;
					if (Variance[nPlane] >= BestVariance)
					{
						BestPlane = nPlane;
						BestVariance = (float)Variance[nPlane];
						BestMean = (float)Mean[nPlane];
					}
				}
			}

			// Now that we have the plane to split on, work through the triangle
			// list placing them on the left or right of the splitting plane
			int32 Left = Start - 1;
			int32 Right = Start + NumTris;
			// Keep working through until the left index passes the right; we test that condition after each interior loop
			for (;;)
			{
				// Loop invariants: (1) Left < Right,
				// (2) All triangles <= Left belong on the left, all triangles >= Right belong on the right
				// (3) Left+1 is an untested triangle, Right-1 is an untested triangle
				float Dot;
				// Increment Left until it points to triangle that belongs on the right, or Right==Left
				for (++Left; Left < Right; ++Left)
				{
					Dot = BuildTriangles[Left].GetCentroid()[BestPlane];
					if (Dot < BestMean)
					{
						break;
					}
				}
				if (Left == Right)
				{
					break;
				}
				// Decrement Right until it points to triangle that belongs on the left, or Right==Left
				for (--Right; Left < Right; --Right)
				{
					Dot = BuildTriangles[Right].GetCentroid()[BestPlane];
					if (Dot >= BestMean)
					{
						break;
					}
				}
				if (Left == Right)
				{
					break;
				}
				// Left points to a triangle that belongs on the Right; Right points to a triangle that belongs on the Left. Swap them.
				Swap(BuildTriangles[Left], BuildTriangles[Right]);
			}
			// After loop array is partitioned and Left is the first index that belongs on the right side of the plane.
			// Check for wacky degenerate case where more than GKDOPMaxTrisPerLeaf
			// fall all in the same kDOP
			if (Left == Start + NumTris || Right == Start)
			{
				Left = Start + (NumTris / 2);
			}
			// Add the two child nodes
			KDOP_IDX_TYPE ChildIndex = Nodes.AddZeroed(2);
			// Nodes may have resized
			Node = &Nodes[NodeIndex];
			Node->n.LeftNode = ChildIndex;
			Node->n.RightNode = Node->n.LeftNode + 1;
			// Have the left node recursively subdivide its list and set bounding volume.
			FBox3f LeftBoundingVolume = SplitTriangleList(Node->n.LeftNode,Start,Left - Start,BuildTriangles);
			// Nodes may have resized
			Node = &Nodes[NodeIndex];
			Node->BoundingVolumes.SetBox(0,LeftBoundingVolume);
			// Set unused index 2,3 to child nodes of left node.
			Node->BoundingVolumes.SetBox(2,Nodes[Node->n.LeftNode].BoundingVolumes.GetBox(0));
			Node->BoundingVolumes.SetBox(3,Nodes[Node->n.LeftNode].BoundingVolumes.GetBox(1));

			// And now have the right node recursively subdivide its list and set bounding volume.			
			FBox3f RightBoundingVolume = SplitTriangleList(Node->n.RightNode,Left,Start + NumTris - Left,BuildTriangles);
			// Nodes may have resized
			Node = &Nodes[NodeIndex];
			Node->BoundingVolumes.SetBox(1,RightBoundingVolume);

			GKDOPNodes += 2;

			// Non-leaf node bounds are the "sum" of the left and right nodes' volumes.
			return LeftBoundingVolume + RightBoundingVolume;
		}
		else
		{
			// Build SOA triangles

			// "NULL triangle", used when a leaf can't fill all 4 triangles in a FTriangleSOA.
			// No line should ever hit these triangles, set the values so that it can never happen.
			FkDOPBuildCollisionTriangle<KDOP_IDX_TYPE> EmptyTriangle(0,FVector4f(0,0,0,0),FVector4f(0,0,0,0),FVector4f(0,0,0,0),INDEX_NONE,INDEX_NONE,INDEX_NONE, false, true);
			
			Node->t.StartIndex = SOATriangles.Num();
			Node->t.NumTriangles = Align<int32>(NumTris, 4) / 4;
			SOATriangles.AddZeroed( Node->t.NumTriangles );

			int32 BuildTriIndex = Start;
			for ( uint32 SOAIndex=0; SOAIndex < Node->t.NumTriangles; ++SOAIndex )
			{
				FkDOPBuildCollisionTriangle<KDOP_IDX_TYPE>* Tris[4] = { &EmptyTriangle, &EmptyTriangle, &EmptyTriangle, &EmptyTriangle };
				FTriangleSOA& SOA = SOATriangles[Node->t.StartIndex + SOAIndex];
				int32 SubIndex = 0;
				for ( ; SubIndex < 4 && BuildTriIndex < (Start+NumTris); ++SubIndex, ++BuildTriIndex )
				{
					Tris[SubIndex] = &BuildTriangles[BuildTriIndex];
					SOA.Payload[SubIndex] = Tris[SubIndex]->MaterialIndex;
				}
				for ( ; SubIndex < 4; ++SubIndex )
				{
					SOA.Payload[SubIndex] = 0xffffffff;
				}

				SOA.Positions[0].X = LmVectorSet( Tris[0]->V0.X, Tris[1]->V0.X, Tris[2]->V0.X, Tris[3]->V0.X );
				SOA.Positions[0].Y = LmVectorSet( Tris[0]->V0.Y, Tris[1]->V0.Y, Tris[2]->V0.Y, Tris[3]->V0.Y );
				SOA.Positions[0].Z = LmVectorSet( Tris[0]->V0.Z, Tris[1]->V0.Z, Tris[2]->V0.Z, Tris[3]->V0.Z );
				SOA.Positions[1].X = LmVectorSet( Tris[0]->V1.X, Tris[1]->V1.X, Tris[2]->V1.X, Tris[3]->V1.X );
				SOA.Positions[1].Y = LmVectorSet( Tris[0]->V1.Y, Tris[1]->V1.Y, Tris[2]->V1.Y, Tris[3]->V1.Y );
				SOA.Positions[1].Z = LmVectorSet( Tris[0]->V1.Z, Tris[1]->V1.Z, Tris[2]->V1.Z, Tris[3]->V1.Z );
				SOA.Positions[2].X = LmVectorSet( Tris[0]->V2.X, Tris[1]->V2.X, Tris[2]->V2.X, Tris[3]->V2.X );
				SOA.Positions[2].Y = LmVectorSet( Tris[0]->V2.Y, Tris[1]->V2.Y, Tris[2]->V2.Y, Tris[3]->V2.Y );
				SOA.Positions[2].Z = LmVectorSet( Tris[0]->V2.Z, Tris[1]->V2.Z, Tris[2]->V2.Z, Tris[3]->V2.Z );

				const FVector4f& Tris0LocalNormal = Tris[0]->GetLocalNormal();
				const FVector4f& Tris1LocalNormal = Tris[1]->GetLocalNormal();
				const FVector4f& Tris2LocalNormal = Tris[2]->GetLocalNormal();
				const FVector4f& Tris3LocalNormal = Tris[3]->GetLocalNormal();

				SOA.Normals.X = LmVectorSet( Tris0LocalNormal.X, Tris1LocalNormal.X, Tris2LocalNormal.X, Tris3LocalNormal.X );
				SOA.Normals.Y = LmVectorSet( Tris0LocalNormal.Y, Tris1LocalNormal.Y, Tris2LocalNormal.Y, Tris3LocalNormal.Y );
				SOA.Normals.Z = LmVectorSet( Tris0LocalNormal.Z, Tris1LocalNormal.Z, Tris2LocalNormal.Z, Tris3LocalNormal.Z );
				SOA.Normals.W = LmVectorSet( -Tris0LocalNormal.W, -Tris1LocalNormal.W, -Tris2LocalNormal.W, -Tris3LocalNormal.W );
				SOA.TwoSidedMask = LmMakeVectorRegister(
					(uint32)(Tris[0]->bTwoSided ? 0xFFFFFFFF : 0), 
					(uint32)(Tris[1]->bTwoSided ? 0xFFFFFFFF : 0),
					(uint32)(Tris[2]->bTwoSided ? 0xFFFFFFFF : 0),
					(uint32)(Tris[3]->bTwoSided ? 0xFFFFFFFF : 0));
				SOA.StaticAndOpaqueMask = LmMakeVectorRegister(
					(uint32)(Tris[0]->bStaticAndOpaque ? 0xFFFFFFFF : 0), 
					(uint32)(Tris[1]->bStaticAndOpaque ? 0xFFFFFFFF : 0),
					(uint32)(Tris[2]->bStaticAndOpaque ? 0xFFFFFFFF : 0),
					(uint32)(Tris[3]->bStaticAndOpaque ? 0xFFFFFFFF : 0));
				SOA.MeshIndices = LmVectorSet(*(float*)&Tris[0]->MeshIndex, *(float*)&Tris[1]->MeshIndex, *(float*)&Tris[2]->MeshIndex, *(float*)&Tris[3]->MeshIndex);
				SOA.LODIndices = LmVectorSet(*(float*)&Tris[0]->LODIndices, *(float*)&Tris[1]->LODIndices, *(float*)&Tris[2]->LODIndices, *(float*)&Tris[3]->LODIndices);
				SOA.HLODRange = LmVectorSet(*(float*)&Tris[0]->HLODRange, *(float*)&Tris[1]->HLODRange, *(float*)&Tris[2]->HLODRange, *(float*)&Tris[3]->HLODRange);
			}

			// No need to subdivide further so make this a leaf node
			Node->bIsLeaf = 1;
			Node->Occupancy = NumTris;
			
			// Generate bounding volume for leaf which is passed up the call chain.
			FBox3f BoundingVolume(ForceInit);
			for (int32 TriangleIndex=Start; TriangleIndex<Start + NumTris; TriangleIndex++)
			{
				BoundingVolume += BuildTriangles[TriangleIndex].V0;
				BoundingVolume += BuildTriangles[TriangleIndex].V1;
				BoundingVolume += BuildTriangles[TriangleIndex].V2;			
			}
			Node->BoundingVolumes.SetBox(0,BoundingVolume);
			Node->BoundingVolumes.SetBox(1,BoundingVolume);
			Node->BoundingVolumes.SetBox(2,BoundingVolume);
			Node->BoundingVolumes.SetBox(3,BoundingVolume);

			GKDOPTriangles += Node->t.NumTriangles * 4;
			GKDOPNumLeaves++;
			return BoundingVolume;
		}
	}

	/**
	 * Figures out whether the check even hits the root node's bounding volume. If
	 * it does, it recursively searches for a triangle to hit.
	 *
	 * @param Check -- The aggregated line check data
	 */
	bool LineCheck(TkDOPLineCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>& Check) const
	{
		// Recursively check for a hit
		TTraversalHistory<KDOP_IDX_TYPE> History;
		bool bHit = Nodes[0].LineCheck(Check, History.AddNode(0));
		return bHit;
	}

	bool BoxCheck(TkDOPBoxCollisionCheck<COLL_DATA_PROVIDER, KDOP_IDX_TYPE>& Check) const
	{
		return Nodes[0].BoxCheck(Check);
	}

	/**
	 * Dumps the kDOPTree 
	 */
	void Dump()
	{
		UE_LOGF(LogLightmass, Log, "kDOPTree [%x], %d nodes, %d triangles", this, Nodes.Num(), GKDOPTriangles);
		UE_LOGF(LogLightmass, Log, "");
		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
		{
			const NodeType& Node = Nodes(NodeIndex);
			UE_LOGF(LogLightmass, Log, " Node %03d: %ls", NodeIndex, Node.bIsLeaf ? TEXT("leaf") : TEXT("nonleaf"));
			if (Node.bIsLeaf)
			{
				UE_LOGF(LogLightmass, Log, "  StartIndex = %d, NumTris = %d", Node.t.StartIndex, Node.t.NumTriangles);
			}
			else
			{
				UE_LOGF(LogLightmass, Log, "  LeftChild = %d, RightChild= %d", Node.n.LeftNode, Node.n.RightNode);
			}
		}
		UE_LOGF(LogLightmass, Log, "");
		UE_LOGF(LogLightmass, Log, "");
	}
};

/**
 * Base struct for all collision checks. Holds a reference to the collision
 * data provider, which is a struct that abstracts out the access to a
 * particular mesh/primitives data
 */
template <typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE> struct TkDOPCollisionCheck
{
	/** Exposes data provider type to clients. */
	typedef COLL_DATA_PROVIDER DataProviderType;

	/** Exposes node type to clients. */
	typedef TkDOPNode<DataProviderType,KDOP_IDX_TYPE> NodeType;

	/** Exposes tree type to clients. */
	typedef TkDOPTree<DataProviderType,KDOP_IDX_TYPE> TreeType;

	/**
	 * Used to get access to local->world, vertices, etc. without using virtuals
	 */
	const DataProviderType& CollDataProvider;
	/**
	 * The kDOP tree
	 */
	const TreeType& kDOPTree;
	/**
	 * The array of the nodes for the kDOP tree
	 */
	const kDOPArray<NodeType, FRangeChecklessHeapAllocator>& Nodes;
	/**
	 * The collision triangle data for the kDOP tree
	 */
	const kDOPArray<FTriangleSOA, FRangeChecklessHeapAllocator>& SOATriangles;

	/**
	 * Hide the default ctor
	 */
	TkDOPCollisionCheck(const DataProviderType& InCollDataProvider) :
		CollDataProvider(InCollDataProvider),
		kDOPTree(CollDataProvider.GetkDOPTree()),
		Nodes(kDOPTree.Nodes),
		SOATriangles(kDOPTree.SOATriangles)
	{
	}
};

/**
 * This struct holds the information used to do a line check against the kDOP
 * tree. The collision provider gives access to various matrices, vertex data
 * etc. without having to use virtual functions.
 */
template <typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE> struct TkDOPLineCollisionCheck :
	public TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>
{
	/**
	 * Where the collision results get stored
	 */
	FHitResult* Result;
	// Constant input vars
	const FVector4f& Start;
	const FVector4f& End;

	/**
	 * Flags for optimizing a trace
	 */
	const bool bFindClosestIntersection;
	const bool bStaticAndOpaqueOnly;
	const bool bTwoSidedCollision;
	const bool bFlipSidedness;
	// Locally calculated vectors
	FVector4f LocalStart;
	FVector4f LocalEnd;
	FVector4f LocalDir;
	FVector4f LocalOneOverDir;
	// Normal in local space which gets transformed to world at the very end
	FVector4f LocalHitNormal;

	/** Index into the kDOP tree's nodes of the node that was hit. */
	KDOP_IDX_TYPE HitNodeIndex;
	
	/** Start of the line, where each component is replicated into their own vector registers. */
	FVector3SOA	StartSOA;
	/** End of the line, where each component is replicated into their own vector registers. */
	FVector3SOA	EndSOA;
	/** Direction of the line (not normalized, just EndSOA-StartSOA), where each component is replicated into their own vector registers. */
	FVector3SOA	DirSOA;
	/** Mesh index of the instigating mesh in every channel. */
	LmVectorRegister MeshIndexRegister;
	/** LOD indices of the instigating mesh in every channel. */
	LmVectorRegister LODIndicesRegister;
	/** HLOD tree ranges of the instigating mesh in every channel. */
	LmVectorRegister HLODRangeRegister;

	/**
	 * Sets up the FkDOPLineCollisionCheck structure for performing line checks
	 * against a kDOPTree. Initializes all of the variables that are used
	 * throughout the line check.
	 *
	 * @param InStart -- The starting point of the trace
	 * @param InEnd -- The ending point of the trace
	 * @param InbFindClosestIntersection -- Whether to stop at the first hit or not
	 * @param InCollDataProvider -- The struct that provides access to mesh/primitive
	 *		specific data, such as L2W, W2L, Vertices, and so on
	 * @param InResult -- The out param for hit result information
	 */
	TkDOPLineCollisionCheck(const FVector4f& InStart,const FVector4f& InEnd,
		bool bInbFindClosestIntersection,
		bool bInStaticAndOpaqueOnly,
		bool bInTwoSidedCollision,
		bool bInFlipSidedness,
		const COLL_DATA_PROVIDER& InCollDataProvider,
		int32 MeshIndex,
		uint32 LODIndices,
		uint32 HLODRange,
		FHitResult* InResult) 
		:
		TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>(InCollDataProvider),
		Result(InResult), 
		Start(InStart), 
		End(InEnd), 
		bFindClosestIntersection(bInbFindClosestIntersection), 
		bStaticAndOpaqueOnly(bInStaticAndOpaqueOnly),
		bTwoSidedCollision(bInTwoSidedCollision),
		bFlipSidedness(bInFlipSidedness),
		HitNodeIndex(0xFFFFFFFF)
	{
		const FMatrix44f& WorldToLocal = TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>::CollDataProvider.GetWorldToLocal();
		// Move start and end to local space
		LocalStart = WorldToLocal.TransformPosition(Start);
		LocalEnd = WorldToLocal.TransformPosition(End);
		// Calculate the vector's direction in local space
		LocalDir = LocalEnd - LocalStart;
		// Build the one over dir
		LocalOneOverDir.X = LocalDir.X ? 1.f / LocalDir.X : MAX_FLT;
		LocalOneOverDir.Y = LocalDir.Y ? 1.f / LocalDir.Y : MAX_FLT;
		LocalOneOverDir.Z = LocalDir.Z ? 1.f / LocalDir.Z : MAX_FLT;

		// Construct the SOA data
		StartSOA.X = LmVectorLoadFloat1( &LocalStart.X );
		StartSOA.Y = LmVectorLoadFloat1( &LocalStart.Y );
		StartSOA.Z = LmVectorLoadFloat1( &LocalStart.Z );
		EndSOA.X = LmVectorLoadFloat1( &LocalEnd.X );
		EndSOA.Y = LmVectorLoadFloat1( &LocalEnd.Y );
		EndSOA.Z = LmVectorLoadFloat1( &LocalEnd.Z );
		DirSOA.X = LmVectorLoadFloat1( &LocalDir.X );
		DirSOA.Y = LmVectorLoadFloat1( &LocalDir.Y );
		DirSOA.Z = LmVectorLoadFloat1( &LocalDir.Z );
		MeshIndexRegister = LmVectorLoadFloat1(&MeshIndex);
		LODIndicesRegister = LmMakeVectorRegister((uint32)LODIndices, (uint32)LODIndices, (uint32)LODIndices, (uint32)LODIndices);
		HLODRangeRegister = LmMakeVectorRegister((uint32)HLODRange, (uint32)HLODRange, (uint32)HLODRange, (uint32)HLODRange);
	}

	/**
	 * Transforms the local hit normal into a world space normal using the transpose
	 * adjoint and flips the normal if need be
	 */
	FORCEINLINE FVector4f GetHitNormal(void)
	{
		// Transform the hit back into world space using the transpose adjoint
		FVector4f Normal = TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>::CollDataProvider.GetLocalToWorldTransposeAdjoint().TransformVector(LocalHitNormal).GetSafeNormal();
		// Flip the normal if the triangle is inverted
		if (TkDOPCollisionCheck<COLL_DATA_PROVIDER,KDOP_IDX_TYPE>::CollDataProvider.GetDeterminant() < 0.f)
		{
			Normal = -Normal;
		}
		return Normal;
	}
};

template <typename COLL_DATA_PROVIDER, typename KDOP_IDX_TYPE> struct TkDOPBoxCollisionCheck :
	public TkDOPCollisionCheck<COLL_DATA_PROVIDER, KDOP_IDX_TYPE>
{
	const FBox3f Box;
	int32 Item;

	TkDOPBoxCollisionCheck(
		const FBox3f& InBox,
		const COLL_DATA_PROVIDER& InCollDataProvider) :
		TkDOPCollisionCheck<COLL_DATA_PROVIDER, KDOP_IDX_TYPE>(InCollDataProvider),
		Box(InBox),
		Item(-1)
	{}
};

} // namespace
