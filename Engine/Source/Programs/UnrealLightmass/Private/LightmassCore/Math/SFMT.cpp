// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SFMT.cpp: Modified version of 'SIMD-oriented Fast Mersenne Twister (SFMT)' version 1.3.3 to work for Lightmass.  
	http://www.math.sci.hiroshima-u.ac.jp/~m-mat/MT/SFMT/index.html#dSFMT
	Most modifications were to eliminate global storage (for thread safety), convert macros to scope and type respecting variables,
	And flatten a few header files into this self contained file.
=============================================================================*/

#include "CoreMinimal.h"
#include "LMMath.h"
#include "LMMathSSE.h"

/** 
 * @file  SFMT.c
 * @brief SIMD oriented Fast Mersenne Twister(SFMT)
 *
 * @author Mutsuo Saito (Hiroshima University)
 * @author Makoto Matsumoto (Hiroshima University)
 *
 * Copyright (C) 2006,2007 Mutsuo Saito, Makoto Matsumoto and Hiroshima
 * University. All rights reserved.
 *
 * The new BSD License is applied to this software, see LICENSE.txt
 */
#include <assert.h>

namespace Lightmass
{
#if defined(__BIG_ENDIAN__) && !defined(__amd64) && !defined(BIG_ENDIAN64)
#define BIG_ENDIAN64 1
#endif
#if defined(HAVE_ALTIVEC) && !defined(BIG_ENDIAN64)
#define BIG_ENDIAN64 1
#endif
#if defined(ONLY64) && !defined(BIG_ENDIAN64)
  #if defined(__GNUC__)
    #error "-DONLY64 must be specified with -DBIG_ENDIAN64"
  #endif
#undef ONLY64
#endif

/*--------------------------------------
  FILE GLOBAL VARIABLES
  internal state, index counter and flag 
  --------------------------------------*/

/*----------------------
the parameters of SFMT
following definitions are in paramsXXXX.h file.
----------------------*/
/** the pick up position of the array.
#define POS1 122 
*/

/** the parameter of shift left as four 32-bit registers.
#define SL1 18
*/

/** the parameter of shift left as one 128-bit register. 
* The 128-bit integer is shifted by (SL2 * 8) bits. 
#define SL2 1 
*/

/** the parameter of shift right as four 32-bit registers.
#define SR1 11
*/

/** the parameter of shift right as one 128-bit register. 
* The 128-bit integer is shifted by (SL2 * 8) bits. 
#define SR2 1 
*/

/** A bitmask, used in the recursion.  These parameters are introduced
* to break symmetry of SIMD.
#define MSK1 0xdfffffefU
#define MSK2 0xddfecb7fU
#define MSK3 0xbffaffffU
#define MSK4 0xbffffff6U 
*/

/** These definitions are part of a 128-bit period certification vector.
#define PARITY1	0x00000001U
#define PARITY2	0x00000000U
#define PARITY3	0x00000000U
#define PARITY4	0xc98e126aU
*/

#define POS1	122
#define SL1	18
#define SL2	1
#define SR1	11
#define SR2	1
#define MSK1	0xdfffffefU
#define MSK2	0xddfecb7fU
#define MSK3	0xbffaffffU
#define MSK4	0xbffffff6U
#define PARITY1	0x00000001U
#define PARITY2	0x00000000U
#define PARITY3	0x00000000U
#define PARITY4	0x13c9e684U
#define IDSTR	"SFMT-19937:122-18-1-11-1:dfffffef-ddfecb7f-bffaffff-bffffff6"

/** a parity check vector which certificate the period of 2^{MEXP} */
static const uint32_t parity[4] = {PARITY1, PARITY2, PARITY3, PARITY4};

/*----------------
  STATIC FUNCTIONS
  ----------------*/
inline static int idxof(int i);
inline static void rshift128(w128_t *out,  w128_t const *in, int shift);
inline static void lshift128(w128_t *out,  w128_t const *in, int shift);
inline static uint32_t func1(uint32_t x);
inline static uint32_t func2(uint32_t x);
#if defined(BIG_ENDIAN64) && !defined(ONLY64)
inline static void swap(w128_t *array, int size);
#endif

#if USE_SSE2_FOR_MERSENNE_TWISTER
/** 
 * @file  SFMT-sse2.h
 * @brief SIMD oriented Fast Mersenne Twister(SFMT) for Intel SSE2
 *
 * @author Mutsuo Saito (Hiroshima University)
 * @author Makoto Matsumoto (Hiroshima University)
 *
 * @note We assume LITTLE ENDIAN in this file
 *
 * Copyright (C) 2006, 2007 Mutsuo Saito, Makoto Matsumoto and Hiroshima
 * University. All rights reserved.
 *
 * The new BSD License is applied to this software, see LICENSE.txt
 */

#ifndef SFMT_SSE2_H
#define SFMT_SSE2_H

inline static __m128i mm_recursion(__m128i *a, __m128i *b, __m128i c,
				   __m128i d, __m128i mask);

/**
 * This function represents the recursion formula.
 * @param a a 128-bit part of the interal state array
 * @param b a 128-bit part of the interal state array
 * @param c a 128-bit part of the interal state array
 * @param d a 128-bit part of the interal state array
 * @param mask 128-bit mask
 * @return output
 */
inline static __m128i mm_recursion(__m128i *a, __m128i *b, 
				   __m128i c, __m128i d, __m128i mask) {
    __m128i v, x, y, z;
    
    x = _mm_load_si128(a);
    y = _mm_srli_epi32(*b, SR1);
    z = _mm_srli_si128(c, SR2);
    v = _mm_slli_epi32(d, SL1);
    z = _mm_xor_si128(z, x);
    z = _mm_xor_si128(z, v);
    x = _mm_slli_si128(x, SL2);
    y = _mm_and_si128(y, mask);
    z = _mm_xor_si128(z, x);
    z = _mm_xor_si128(z, y);
    return z;
}

/**
 * This function fills the internal state array with pseudorandom
 * integers.
 */
inline void FLMRandomStream::gen_rand_all(void) {
    int i;
    __m128i r, r1, r2, mask;
    mask = _mm_set_epi32(MSK4, MSK3, MSK2, MSK1);

    r1 = _mm_load_si128(&sfmt[N - 2].si);
    r2 = _mm_load_si128(&sfmt[N - 1].si);
    for (i = 0; i < N - POS1; i++) {
	r = mm_recursion(&sfmt[i].si, &sfmt[i + POS1].si, r1, r2, mask);
	_mm_store_si128(&sfmt[i].si, r);
	r1 = r2;
	r2 = r;
    }
    for (; i < N; i++) {
	r = mm_recursion(&sfmt[i].si, &sfmt[i + POS1 - N].si, r1, r2, mask);
	_mm_store_si128(&sfmt[i].si, r);
	r1 = r2;
	r2 = r;
    }
}

/**
 * This function fills the user-specified array with pseudorandom
 * integers.
 *
 * @param array an 128-bit array to be filled by pseudorandom numbers.  
 * @param size number of 128-bit pesudorandom numbers to be generated.
 */
inline void FLMRandomStream::gen_rand_array(w128_t *array, int size) {
    int i, j;
    __m128i r, r1, r2, mask;
    mask = _mm_set_epi32(MSK4, MSK3, MSK2, MSK1);

    r1 = _mm_load_si128(&sfmt[N - 2].si);
    r2 = _mm_load_si128(&sfmt[N - 1].si);
    for (i = 0; i < N - POS1; i++) {
	r = mm_recursion(&sfmt[i].si, &sfmt[i + POS1].si, r1, r2, mask);
	_mm_store_si128(&array[i].si, r);
	r1 = r2;
	r2 = r;
    }
    for (; i < N; i++) {
	r = mm_recursion(&sfmt[i].si, &array[i + POS1 - N].si, r1, r2, mask);
	_mm_store_si128(&array[i].si, r);
	r1 = r2;
	r2 = r;
    }
    /* main loop */
    for (; i < size - N; i++) {
	r = mm_recursion(&array[i - N].si, &array[i + POS1 - N].si, r1, r2,
			 mask);
	_mm_store_si128(&array[i].si, r);
	r1 = r2;
	r2 = r;
    }
    for (j = 0; j < 2 * N - size; j++) {
	r = _mm_load_si128(&array[j + size - N].si);
	_mm_store_si128(&sfmt[j].si, r);
    }
    for (; i < size; i++) {
	r = mm_recursion(&array[i - N].si, &array[i + POS1 - N].si, r1, r2,
			 mask);
	_mm_store_si128(&array[i].si, r);
	_mm_store_si128(&sfmt[j++].si, r);
	r1 = r2;
	r2 = r;
    }
}

#endif

#endif

/**
 * This function simulate a 64-bit index of LITTLE ENDIAN 
 * in BIG ENDIAN machine.
 */
#ifdef ONLY64
inline static int idxof(int i) {
    return i ^ 1;
}
#else
inline static int idxof(int i) {
    return i;
}
#endif
/**
 * This function simulates SIMD 128-bit right shift by the standard C.
 * The 128-bit integer given in in is shifted by (shift * 8) bits.
 * This function simulates the LITTLE ENDIAN SIMD.
 * @param out the output of this function
 * @param in the 128-bit data to be shifted
 * @param shift the shift value
 */
#ifdef ONLY64
inline static void rshift128(w128_t *out, w128_t const *in, int shift) {
    uint64_t th, tl, oh, ol;

    th = ((uint64_t)in->u[2] << 32) | ((uint64_t)in->u[3]);
    tl = ((uint64_t)in->u[0] << 32) | ((uint64_t)in->u[1]);

    oh = th >> (shift * 8);
    ol = tl >> (shift * 8);
    ol |= th << (64 - shift * 8);
    out->u[0] = (uint32_t)(ol >> 32);
    out->u[1] = (uint32_t)ol;
    out->u[2] = (uint32_t)(oh >> 32);
    out->u[3] = (uint32_t)oh;
}
#else
inline static void rshift128(w128_t *out, w128_t const *in, int shift) {
    uint64_t th, tl, oh, ol;

    th = ((uint64_t)in->u[3] << 32) | ((uint64_t)in->u[2]);
    tl = ((uint64_t)in->u[1] << 32) | ((uint64_t)in->u[0]);

    oh = th >> (shift * 8);
    ol = tl >> (shift * 8);
    ol |= th << (64 - shift * 8);
    out->u[1] = (uint32_t)(ol >> 32);
    out->u[0] = (uint32_t)ol;
    out->u[3] = (uint32_t)(oh >> 32);
    out->u[2] = (uint32_t)oh;
}
#endif
/**
 * This function simulates SIMD 128-bit left shift by the standard C.
 * The 128-bit integer given in in is shifted by (shift * 8) bits.
 * This function simulates the LITTLE ENDIAN SIMD.
 * @param out the output of this function
 * @param in the 128-bit data to be shifted
 * @param shift the shift value
 */
#ifdef ONLY64
inline static void lshift128(w128_t *out, w128_t const *in, int shift) {
    uint64_t th, tl, oh, ol;

    th = ((uint64_t)in->u[2] << 32) | ((uint64_t)in->u[3]);
    tl = ((uint64_t)in->u[0] << 32) | ((uint64_t)in->u[1]);

    oh = th << (shift * 8);
    ol = tl << (shift * 8);
    oh |= tl >> (64 - shift * 8);
    out->u[0] = (uint32_t)(ol >> 32);
    out->u[1] = (uint32_t)ol;
    out->u[2] = (uint32_t)(oh >> 32);
    out->u[3] = (uint32_t)oh;
}
#else
inline static void lshift128(w128_t *out, w128_t const *in, int shift) {
    uint64_t th, tl, oh, ol;

    th = ((uint64_t)in->u[3] << 32) | ((uint64_t)in->u[2]);
    tl = ((uint64_t)in->u[1] << 32) | ((uint64_t)in->u[0]);

    oh = th << (shift * 8);
    ol = tl << (shift * 8);
    oh |= tl >> (64 - shift * 8);
    out->u[1] = (uint32_t)(ol >> 32);
    out->u[0] = (uint32_t)ol;
    out->u[3] = (uint32_t)(oh >> 32);
    out->u[2] = (uint32_t)oh;
}
#endif

/**
 * This function represents the recursion formula.
 * @param r output
 * @param a a 128-bit part of the internal state array
 * @param b a 128-bit part of the internal state array
 * @param c a 128-bit part of the internal state array
 * @param d a 128-bit part of the internal state array
 */
#if !USE_SSE2_FOR_MERSENNE_TWISTER
#ifdef ONLY64
inline static void do_recursion(w128_t *r, w128_t *a, w128_t *b, w128_t *c,
				w128_t *d) {
    w128_t x;
    w128_t y;

    lshift128(&x, a, SL2);
    rshift128(&y, c, SR2);
    r->u[0] = a->u[0] ^ x.u[0] ^ ((b->u[0] >> SR1) & MSK2) ^ y.u[0] 
	^ (d->u[0] << SL1);
    r->u[1] = a->u[1] ^ x.u[1] ^ ((b->u[1] >> SR1) & MSK1) ^ y.u[1] 
	^ (d->u[1] << SL1);
    r->u[2] = a->u[2] ^ x.u[2] ^ ((b->u[2] >> SR1) & MSK4) ^ y.u[2] 
	^ (d->u[2] << SL1);
    r->u[3] = a->u[3] ^ x.u[3] ^ ((b->u[3] >> SR1) & MSK3) ^ y.u[3] 
	^ (d->u[3] << SL1);
}
#else
inline static void do_recursion(w128_t *r, w128_t *a, w128_t *b, w128_t *c,
				w128_t *d) {
    w128_t x;
    w128_t y;

    lshift128(&x, a, SL2);
    rshift128(&y, c, SR2);
    r->u[0] = a->u[0] ^ x.u[0] ^ ((b->u[0] >> SR1) & MSK1) ^ y.u[0] 
	^ (d->u[0] << SL1);
    r->u[1] = a->u[1] ^ x.u[1] ^ ((b->u[1] >> SR1) & MSK2) ^ y.u[1] 
	^ (d->u[1] << SL1);
    r->u[2] = a->u[2] ^ x.u[2] ^ ((b->u[2] >> SR1) & MSK3) ^ y.u[2] 
	^ (d->u[2] << SL1);
    r->u[3] = a->u[3] ^ x.u[3] ^ ((b->u[3] >> SR1) & MSK4) ^ y.u[3] 
	^ (d->u[3] << SL1);
}
#endif
#endif

#if !USE_SSE2_FOR_MERSENNE_TWISTER
/**
 * This function fills the internal state array with pseudorandom
 * integers.
 */
void FLMRandomStream::gen_rand_all(void) {
    int i;
    w128_t *r1, *r2;

    r1 = &sfmt[N - 2];
    r2 = &sfmt[N - 1];
    for (i = 0; i < N - POS1; i++) {
	do_recursion(&sfmt[i], &sfmt[i], &sfmt[i + POS1], r1, r2);
	r1 = r2;
	r2 = &sfmt[i];
    }
    for (; i < N; i++) {
	do_recursion(&sfmt[i], &sfmt[i], &sfmt[i + POS1 - N], r1, r2);
	r1 = r2;
	r2 = &sfmt[i];
    }
}

/**
 * This function fills the user-specified array with pseudorandom
 * integers.
 *
 * @param array an 128-bit array to be filled by pseudorandom numbers.  
 * @param size number of 128-bit pseudorandom numbers to be generated.
 */
void FLMRandomStream::gen_rand_array(w128_t *array, int size) {
    int i, j;
    w128_t *r1, *r2;

    r1 = &sfmt[N - 2];
    r2 = &sfmt[N - 1];
    for (i = 0; i < N - POS1; i++) {
	do_recursion(&array[i], &sfmt[i], &sfmt[i + POS1], r1, r2);
	r1 = r2;
	r2 = &array[i];
    }
    for (; i < N; i++) {
	do_recursion(&array[i], &sfmt[i], &array[i + POS1 - N], r1, r2);
	r1 = r2;
	r2 = &array[i];
    }
    for (; i < size - N; i++) {
	do_recursion(&array[i], &array[i - N], &array[i + POS1 - N], r1, r2);
	r1 = r2;
	r2 = &array[i];
    }
    for (j = 0; j < 2 * N - size; j++) {
	sfmt[j] = array[j + size - N];
    }
    for (; i < size; i++, j++) {
	do_recursion(&array[i], &array[i - N], &array[i + POS1 - N], r1, r2);
	r1 = r2;
	r2 = &array[i];
	sfmt[j] = array[i];
    }
}
#endif

#if defined(BIG_ENDIAN64) && !defined(ONLY64) && !defined(HAVE_ALTIVEC)
inline static void swap(w128_t *array, int size) {
    int i;
    uint32_t x, y;

    for (i = 0; i < size; i++) {
	x = array[i].u[0];
	y = array[i].u[2];
	array[i].u[0] = array[i].u[1];
	array[i].u[2] = array[i].u[3];
	array[i].u[1] = x;
	array[i].u[3] = y;
    }
}
#endif
/**
 * This function represents a function used in the initialization
 * by init_by_array
 * @param x 32-bit integer
 * @return 32-bit integer
 */
static uint32_t func1(uint32_t x) {
    return (x ^ (x >> 27)) * (uint32_t)1664525UL;
}

/**
 * This function represents a function used in the initialization
 * by init_by_array
 * @param x 32-bit integer
 * @return 32-bit integer
 */
static uint32_t func2(uint32_t x) {
    return (x ^ (x >> 27)) * (uint32_t)1566083941UL;
}

/**
 * This function certificate the period of 2^{MEXP}
 */
void FLMRandomStream::period_certification(void) {
    int inner = 0;
    int i, j;
    uint32_t work;

    for (i = 0; i < 4; i++)
	inner ^= psfmt32[idxof(i)] & parity[i];
    for (i = 16; i > 0; i >>= 1)
	inner ^= inner >> i;
    inner &= 1;
    /* check OK */
    if (inner == 1) {
	return;
    }
    /* check NG, and modification */
    for (i = 0; i < 4; i++) {
	work = 1;
	for (j = 0; j < 32; j++) {
	    if ((work & parity[i]) != 0) {
		psfmt32[idxof(i)] ^= work;
		return;
	    }
	    work = work << 1;
	}
    }
}

/*----------------
  PUBLIC FUNCTIONS
  ----------------*/
/**
 * This function returns the identification string.
 * The string shows the word size, the Mersenne exponent,
 * and all parameters of this generator.
 */
const char * FLMRandomStream::get_idstring(void) {
    return IDSTR;
}

/**
 * This function returns the minimum size of array used for \b
 * fill_array32() function.
 * @return minimum size of array used for fill_array32() function.
 */
int FLMRandomStream::get_min_array_size32(void) {
    return N32;
}

/**
 * This function returns the minimum size of array used for \b
 * fill_array64() function.
 * @return minimum size of array used for fill_array64() function.
 */
int FLMRandomStream::get_min_array_size64(void) {
    return N64;
}

#ifndef ONLY64
/**
 * This function generates and returns 32-bit pseudorandom number.
 * init_gen_rand or init_by_array must be called before this function.
 * @return 32-bit pseudorandom number
 */
uint32_t FLMRandomStream::gen_rand32(void) {
    uint32_t r;

    assert(initialized);
    if (idx >= N32) {
	gen_rand_all();
	idx = 0;
    }
    r = psfmt32[idx++];
    return r;
}
#endif
/**
 * This function generates and returns 64-bit pseudorandom number.
 * init_gen_rand or init_by_array must be called before this function.
 * The function gen_rand64 should not be called after gen_rand32,
 * unless an initialization is again executed. 
 * @return 64-bit pseudorandom number
 */
uint64_t FLMRandomStream::gen_rand64(void) {
#if defined(BIG_ENDIAN64) && !defined(ONLY64)
    uint32_t r1, r2;
#else
    uint64_t r;
#endif

    assert(initialized);
    assert(idx % 2 == 0);

    if (idx >= N32) {
	gen_rand_all();
	idx = 0;
    }
#if defined(BIG_ENDIAN64) && !defined(ONLY64)
    r1 = psfmt32[idx];
    r2 = psfmt32[idx + 1];
    idx += 2;
    return ((uint64_t)r2 << 32) | r1;
#else
    r = psfmt64[idx / 2];
    idx += 2;
    return r;
#endif
}

#ifndef ONLY64
/**
 * This function generates pseudorandom 32-bit integers in the
 * specified array[] by one call. The number of pseudorandom integers
 * is specified by the argument size, which must be at least 624 and a
 * multiple of four.  The generation by this function is much faster
 * than the following gen_rand function.
 *
 * For initialization, init_gen_rand or init_by_array must be called
 * before the first call of this function. This function can not be
 * used after calling gen_rand function, without initialization.
 *
 * @param array an array where pseudorandom 32-bit integers are filled
 * by this function.  The pointer to the array must be \b "aligned"
 * (namely, must be a multiple of 16) in the SIMD version, since it
 * refers to the address of a 128-bit integer.  In the standard C
 * version, the pointer is arbitrary.
 *
 * @param size the number of 32-bit pseudorandom integers to be
 * generated.  size must be a multiple of 4, and greater than or equal
 * to (MEXP / 128 + 1) * 4.
 *
 * @note \b memalign or \b posix_memalign is available to get aligned
 * memory. Mac OSX doesn't have these functions, but \b malloc of OSX
 * returns the pointer to the aligned memory block.
 */
void FLMRandomStream::fill_array32(uint32_t *array, int size) {
    assert(initialized);
    assert(idx == N32);
    assert(size % 4 == 0);
    assert(size >= N32);

    gen_rand_array((w128_t *)array, size / 4);
    idx = N32;
}
#endif

/**
 * This function generates pseudorandom 64-bit integers in the
 * specified array[] by one call. The number of pseudorandom integers
 * is specified by the argument size, which must be at least 312 and a
 * multiple of two.  The generation by this function is much faster
 * than the following gen_rand function.
 *
 * For initialization, init_gen_rand or init_by_array must be called
 * before the first call of this function. This function can not be
 * used after calling gen_rand function, without initialization.
 *
 * @param array an array where pseudorandom 64-bit integers are filled
 * by this function.  The pointer to the array must be "aligned"
 * (namely, must be a multiple of 16) in the SIMD version, since it
 * refers to the address of a 128-bit integer.  In the standard C
 * version, the pointer is arbitrary.
 *
 * @param size the number of 64-bit pseudorandom integers to be
 * generated.  size must be a multiple of 2, and greater than or equal
 * to (MEXP / 128 + 1) * 2
 *
 * @note \b memalign or \b posix_memalign is available to get aligned
 * memory. Mac OSX doesn't have these functions, but \b malloc of OSX
 * returns the pointer to the aligned memory block.
 */
void FLMRandomStream::fill_array64(uint64_t *array, int size) {
    assert(initialized);
    assert(idx == N32);
    assert(size % 2 == 0);
    assert(size >= N64);

    gen_rand_array((w128_t *)array, size / 2);
    idx = N32;

#if defined(BIG_ENDIAN64) && !defined(ONLY64)
    swap((w128_t *)array, size /2);
#endif
}

/**
 * This function initializes the internal state array with a 32-bit
 * integer seed.
 *
 * @param seed a 32-bit integer used as the seed.
 */
void FLMRandomStream::init_gen_rand(uint32_t seed) {
    int i;

    psfmt32[idxof(0)] = seed;
    for (i = 1; i < N32; i++) {
	psfmt32[idxof(i)] = 1812433253UL * (psfmt32[idxof(i - 1)] 
					    ^ (psfmt32[idxof(i - 1)] >> 30))
	    + i;
    }
    idx = N32;
    period_certification();
    initialized = 1;
}

/**
 * This function initializes the internal state array,
 * with an array of 32-bit integers used as the seeds
 * @param init_key the array of 32-bit integers, used as a seed.
 * @param key_length the length of init_key.
 */
void FLMRandomStream::init_by_array(uint32_t *init_key, int key_length) {
    int i, j, count;
    uint32_t r;
    int lag;
    int mid;
    int size = N * 4;

    if (size >= 623) {
	lag = 11;
    } else if (size >= 68) {
	lag = 7;
    } else if (size >= 39) {
	lag = 5;
    } else {
	lag = 3;
    }
    mid = (size - lag) / 2;

    memset(sfmt, 0x8b, sizeof(sfmt));
    if (key_length + 1 > N32) {
	count = key_length + 1;
    } else {
	count = N32;
    }
    r = func1(psfmt32[idxof(0)] ^ psfmt32[idxof(mid)] 
	      ^ psfmt32[idxof(N32 - 1)]);
    psfmt32[idxof(mid)] += r;
    r += key_length;
    psfmt32[idxof(mid + lag)] += r;
    psfmt32[idxof(0)] = r;

    count--;
    for (i = 1, j = 0; (j < count) && (j < key_length); j++) {
	r = func1(psfmt32[idxof(i)] ^ psfmt32[idxof((i + mid) % N32)] 
		  ^ psfmt32[idxof((i + N32 - 1) % N32)]);
	psfmt32[idxof((i + mid) % N32)] += r;
	r += init_key[j] + i;
	psfmt32[idxof((i + mid + lag) % N32)] += r;
	psfmt32[idxof(i)] = r;
	i = (i + 1) % N32;
    }
    for (; j < count; j++) {
	r = func1(psfmt32[idxof(i)] ^ psfmt32[idxof((i + mid) % N32)] 
		  ^ psfmt32[idxof((i + N32 - 1) % N32)]);
	psfmt32[idxof((i + mid) % N32)] += r;
	r += i;
	psfmt32[idxof((i + mid + lag) % N32)] += r;
	psfmt32[idxof(i)] = r;
	i = (i + 1) % N32;
    }
    for (j = 0; j < N32; j++) {
	r = func2(psfmt32[idxof(i)] + psfmt32[idxof((i + mid) % N32)] 
		  + psfmt32[idxof((i + N32 - 1) % N32)]);
	psfmt32[idxof((i + mid) % N32)] ^= r;
	r -= i;
	psfmt32[idxof((i + mid + lag) % N32)] ^= r;
	psfmt32[idxof(i)] = r;
	i = (i + 1) % N32;
    }

    idx = N32;
    period_certification();
    initialized = 1;
}
}
