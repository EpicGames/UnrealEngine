// Copyright Epic Games, Inc. All Rights Reserved.

#include "LMDebug.h"
#include "LMCore.h"
#include "HAL/ExceptionHandling.h"
#include "UnrealLightmass.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CString.h"
#include "Misc/CommandLine.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

namespace Lightmass
{

FString InstigatorUserName;

/** Crash reporter URL, as set by AutoReporter.exe after being launched by appHandleCriticalError(). */
static FString GCrashReporterURL;

/**
 * Returns the crash reporter URL after appHandleCriticalError() has been called.
 */
const FString& appGetCrashReporterURL()
{
	return GCrashReporterURL;
}


void appHandleCriticalError()
{
	/** Only handle the first critical error. */
	static bool bAlreadyHandledCriticalError = false;
	if ( bAlreadyHandledCriticalError == false )
	{
		bAlreadyHandledCriticalError = true;

		GCrashReporterURL.Empty();

		// Dump the error and flush the log.
		UE_LOG(LogLightmass, Error, TEXT("=== Critical error: === %s") LINE_TERMINATOR TEXT("%s"), GErrorExceptionDescription, GErrorHist);
		GLog->Flush();

	// Create an AutoReporter report.
#if !UE_BUILD_DEBUG
		{
			TCHAR ReportDumpVersion[] = TEXT("4");
			TCHAR ReportDumpFilename[] = TEXT("UE5AutoReportDump.txt");
			TCHAR AutoReportExe[] = TEXT("../DotNET/AutoReporter.exe");
			TCHAR IniDumpFilename[] = TEXT("UE5AutoReportIniDump.txt");

			FArchive* AutoReportFile = IFileManager::Get().CreateFileWriter(ReportDumpFilename);
			if (AutoReportFile != NULL)
			{
				TCHAR CompName[256];
				FCString::Strcpy(CompName, FPlatformProcess::ComputerName());
				TCHAR UserName[256];
				if (InstigatorUserName.Len() > 0 && !InstigatorUserName.Contains(FPlatformProcess::UserName()) )
				{
					// Override the current machine's username with the instigator's username, 
					// So that it's easy to track crashes on remote machines back to the person launching the lighting build.
					FCString::Strcpy(UserName, *InstigatorUserName);
				}
				else
				{
					FCString::Strcpy(UserName, FPlatformProcess::UserName());
				}
				TCHAR GameName[256];
				FCString::Strcpy(GameName, TEXT("Lightmass"));
				TCHAR PlatformName[32];
#if PLATFORM_WINDOWS
#if _WIN64
				FCString::Strcpy(PlatformName, TEXT("PC 64-bit"));
#else
				FCString::Strcpy(PlatformName, TEXT("PC 32-bit"));
#endif
#elif PLATFORM_MAC
				FCString::Strcpy(PlatformName, TEXT("Mac"));
#elif PLATFORM_LINUX
				FCString::Strcpy(PlatformName, TEXT("Linux"));
#endif
				TCHAR LangExt[10];
				FCString::Strcpy(LangExt, TEXT("English"));
				TCHAR SystemTime[256];
				FCString::Strcpy(SystemTime, *FDateTime::Now().ToString());
				TCHAR EngineVersionStr[32];
				FCString::Strcpy(EngineVersionStr, *FString::FromInt(1));

				TCHAR ChangelistVersionStr[32];
				FCString::Strcpy(ChangelistVersionStr, *FString::FromInt(0));

				TCHAR CmdLine[2048];
				FCString::Strncpy(CmdLine, FCommandLine::Get(), UE_ARRAY_COUNT(CmdLine));
				FCString::StrncatTruncateDest(CmdLine, UE_ARRAY_COUNT(CmdLine), TEXT(" -unattended"));
				TCHAR BaseDir[260];
				FCString::Strncpy(BaseDir, FPlatformProcess::BaseDir(), UE_ARRAY_COUNT(BaseDir));
				TCHAR separator = 0;

				TCHAR EngineMode[64];
				FCString::Strcpy(EngineMode, TEXT("Tool"));

				//build the report dump file
				AutoReportFile->Serialize( ReportDumpVersion, FCString::Strlen(ReportDumpVersion) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( CompName, FCString::Strlen(CompName) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( UserName, FCString::Strlen(UserName) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( GameName, FCString::Strlen(GameName) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( PlatformName, FCString::Strlen(PlatformName) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( LangExt, FCString::Strlen(LangExt) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( SystemTime, FCString::Strlen(SystemTime) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( EngineVersionStr, FCString::Strlen(EngineVersionStr) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( ChangelistVersionStr, FCString::Strlen(ChangelistVersionStr) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( CmdLine, FCString::Strlen(CmdLine) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( BaseDir, FCString::Strlen(BaseDir) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( GErrorHist, FCString::Strlen(GErrorHist) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Serialize( EngineMode, FCString::Strlen(EngineMode) * sizeof(TCHAR) );
				AutoReportFile->Serialize( &separator, sizeof(TCHAR) );
				AutoReportFile->Flush();
				delete AutoReportFile;

				FString UserLogFile( FLightmassLog::Get()->GetLogFilename() );
				//start up the auto reporting app, passing the report dump file path, the games' log file, the ini dump path and the minidump path
				//protect against spaces in paths breaking them up on the commandline
				FString CallingCommandLine = FString::Printf(TEXT("%d \"%s\" \"%s\" \"%s\" \"%s\" -unattended"), 
					(uint32)(FPlatformProcess::GetCurrentProcessId()), ReportDumpFilename, *UserLogFile, IniDumpFilename, MiniDumpFilenameW);
				FProcHandle ProcHandle = FPlatformProcess::CreateProc(AutoReportExe, *CallingCommandLine, true, false, false, NULL, 0, NULL, NULL);
				if ( ProcHandle.IsValid() )
				{
					FPlatformProcess::WaitForProc(ProcHandle);
					{
						// Read the URL from the crash report log file
						FILE *GeneratedAutoReportFile;
#if PLATFORM_WINDOWS
						if ( fopen_s( &GeneratedAutoReportFile, "AutoReportLog.txt", "r" ) == 0 )
#else
						if ( ( GeneratedAutoReportFile = fopen( "AutoReportLog.txt", "r" ) ) != NULL )
#endif
						{
							// Read each line, looking for the URL
							const uint32 LineBufferSize = 1024;
							char LineBuffer[LineBufferSize];
							const char* URLSearchText = "CrashReport url = ";
							char* URLFoundText = NULL;
							while( fgets( LineBuffer, LineBufferSize, GeneratedAutoReportFile ) != NULL )
							{
								if( ( URLFoundText = FCStringAnsi::Strstr( LineBuffer, URLSearchText ) ) != NULL )
								{
									URLFoundText += FCStringAnsi::Strlen( URLSearchText );
									GCrashReporterURL = StringCast<TCHAR>(URLFoundText).Get();
									break;
								}
							}
							fclose( GeneratedAutoReportFile );
						}
						else
						{
							GCrashReporterURL = TEXT("Not found (unable to open log file)!");
						}
					}
				}
				else
				{
					UE_LOGF(LogLightmass, Error, "Couldn't start up the Auto Reporting process!");
				}
			}
		}
#endif	// UE_BUILD_DEBUG
	}
}

}	//namespace Lightmass

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS
