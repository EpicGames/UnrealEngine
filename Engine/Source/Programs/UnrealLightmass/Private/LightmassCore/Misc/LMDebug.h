// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Lightmass
{

extern FString InstigatorUserName;

/**
 * Handles critical error. The call only performs anything on the first call.
 */
void appHandleCriticalError();

/**
 * Returns the crash reporter URL after appHandleCriticalError() has been called.
 */
const class FString& appGetCrashReporterURL();

}
