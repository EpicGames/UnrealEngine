// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Lightmass
{

class FLightmassLog : public FOutputDevice
{
public:

	FLightmassLog();
	~FLightmassLog();

	// BEGIN FOutputDevice Interface 
	virtual void Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category ) override;
	virtual void Flush() override;
	// END FOutputDevice Interface

	/**
	 * Returns the filename of the log file.
	 */
	const TCHAR* GetLogFilename() const
	{
		return *Filename;
	}

	/**
	 * Singleton interface.
	 */
	static FLightmassLog* Get();

private:

	/** Handle to log file */
	FArchive*	File;

	/** Filename of the log file. */
	FString Filename;
};

/** Stores an SHA hash generated by FSHA1. */
class FSHAHash
{
public:
	uint8 Hash[20];

	inline FString ToString() const
	{
		return BytesToHex((const uint8*)Hash, sizeof(Hash));
	}

	friend bool operator==(const FSHAHash& X, const FSHAHash& Y)
	{
		return FMemory::Memcmp(&X.Hash, &Y.Hash, sizeof(X.Hash)) == 0;
	}

	friend bool operator!=(const FSHAHash& X, const FSHAHash& Y)
	{
		return FMemory::Memcmp(&X.Hash, &Y.Hash, sizeof(X.Hash)) != 0;
	}

	friend uint32 GetTypeHash(const FSHAHash& H)
	{
		return FCrc::MemCrc32(H.Hash, sizeof(H.Hash));
	}
};


}
