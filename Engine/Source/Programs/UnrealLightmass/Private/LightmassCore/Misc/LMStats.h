// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// basic stuff needed by everybody
#pragma warning( disable: 4799 )		// function '...' has no EMMS instruction)

#if (PLATFORM_MAC && __clang_major__ < 9) || PLATFORM_LINUX
#if !__has_builtin(__rdtsc)
inline unsigned long long __rdtsc()
{
	unsigned long long Low, High;
	__asm__ __volatile__ ("rdtsc" : "=a" (Low), "=d" (High));
	return (High << 32) | Low;
}
#endif //!__has_builtin(__rdtsc))
#endif //(PLATFORM_MAC || PLATFORM_LINUX)

namespace Lightmass
{

/**
 * Helper class encapsulating using RDTSC to read the CPU time stamp on x86 CPUs. 
 * 
 * Limitations:
 * - the cycle counts used cannot be compared with cycle counts from other threads
 * - this won't work right if the OS decides to change the CPU a task is being executed 
 * - spread spectrum frequency modulation as required by the FCC is going to add some noise
 * - newer CPUs implement RDTSC as bus clock * multiplier to avoid the above, limiting accuracy
 *
 * Even with all those caveats it is sometimes useful to have a very high resolution timer 
 * for low level optimizations to accurately measure the impact of optimizations in cases 
 * where there is no reliable reproduction case to benchmark like e.g. gameplay.
 *
 * @warning: do not use outside temporary profiling
 */
struct FRDTSCCycleTimer
{
	/**
	 * Constructor, initializing cycle counter and potentially overhead
	 *
	 * @param	InOverhead	Overhead in cycles or -1 if it should be determined programatically
	 */
	FRDTSCCycleTimer( int64 InOverhead = -1 )
		:	PreviousCycles(0)
		,	Cycles(0)
		,	Calls(0)
		,	Overhead( InOverhead )
	{
		// Calculate overhead if not passed in.
		if( Overhead == -1 )
		{
			// Reset overhead to 0 as it is used by Stop.
			Overhead = 0;

			// Run capture loop to determine accumulated overhead.
			const int32 LoopCount = 100;
			for( int32 i=0; i<LoopCount; i++ )
			{
				Start();
				Stop();
			}
			Overhead = Cycles / LoopCount;

			// Reset counters again.
			Reset();
		}
	}

	/**
	 * Resets the state of the timer.
	 */
	void Reset()
	{
		PreviousCycles = 0;
		Cycles = 0;
		Calls = 0;
	}

	/**
	 * Starts the timer capture. This is a very lightweight operation.
	 */
	void FORCEINLINE Start()
	{
		// We access cycles before RDTSC to ensure us not timing a cache miss accessing it.
		PreviousCycles = Cycles;
		Cycles -= __rdtsc();
	}

	/**
	 * Stops the timer capture. This is a very lightweight operation.
	 */
	void FORCEINLINE Stop()
	{
		Cycles += __rdtsc();		
		// Overhead is the measured average cycle overhead of capturing so measuring start/ stop should
		// result in roughly 0 or very few cycles. We explicitly avoid decreasing elapsed cycles in case
		// overhead was higher during setup than time elapsed. This can happen if we don't capture anything
		// and RDTSC makes it through the pipeline more efficiently.
		if( Cycles - PreviousCycles - Overhead > 0 )
		{		
			Cycles -= Overhead;
		}
		else
		{
			Cycles = PreviousCycles;
		}
		Calls++;
	}

	/**
	 * @return	number of CPU cycles elapsed while the timer was active
	 */
	int64 GetCycleCount()
	{
		return Cycles;
	}

	/**
	 * @return	number of times timer has been started/ stopped
	 */
	int64 GetCallCount()
	{
		return Calls;
	}

private:
	/** Previous cycles, used to warm cache and to avoid negative delta times. */
	int64 PreviousCycles;
	/** Cycle counter used to keep track of timer duration. */
	int64 Cycles;
	/** Number of callst o the cycle counter. */
	int64 Calls;
	/** Timing overhead in cycles. Factored out during capture. */
	int64 Overhead;
};

/** CPU frequency for stats, only used for inner loop timing with rdtsc. */
extern double CPUFrequency /*= 3000000000.0*/;

/** Number of CPU clock cycles per second (as counted by __rdtsc). */
extern double GSecondPerCPUCycle /*= 1.0 / 3000000000.0*/;

/** Start initializing CPU frequency (as counted by __rdtsc). */
void StartInitCPUFrequency();

/** Finishe initializing CPU frequency (as counted by __rdtsc), and set up CPUFrequency and CPUCyclesPerSecond. */
void FinishInitCPUFrequency();

class FScopedRDTSCTimer
{
public:

	FScopedRDTSCTimer(float& InValue) :
		Timer(0),
		Value(InValue)
	{
		Timer.Start();
	}

	~FScopedRDTSCTimer()
	{
		Timer.Stop();
		Value += Timer.GetCycleCount() * GSecondPerCPUCycle;
	}

private:
	FRDTSCCycleTimer Timer;
	float& Value;
};

class FManualRDTSCTimer
{
public:

	FManualRDTSCTimer(float& InValue) :
		Timer(0),
		Value(InValue)
	{
		Timer.Start();
	}

	void Stop()
	{
		Timer.Stop();
		Value += Timer.GetCycleCount() * GSecondPerCPUCycle;
	}

	~FManualRDTSCTimer()
	{
		// Verify that the timer has been stopped before it is destroyed.
		checkSlow(Timer.GetCallCount() > 0);
	}

private:
	FRDTSCCycleTimer Timer;
	float& Value;
};

}
