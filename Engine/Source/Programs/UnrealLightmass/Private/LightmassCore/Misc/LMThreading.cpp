// Copyright Epic Games, Inc. All Rights Reserved.

/*============================================================================
	LMThreading.cpp: Threading/process functionality
=============================================================================*/

#include "LMThreading.h"
#include "LMCore.h"
#include "HAL/PlatformProcess.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

#if PLATFORM_MAC || PLATFORM_LINUX
#include <sys/time.h>	// for gettimeofday()
#endif

namespace Lightmass
{

/**
 * Constructor that zeroes the handle
 */
FSemaphore::FSemaphore(void)
{
#if PLATFORM_WINDOWS
	Semaphore = NULL;
#else
	bInitialized = false;
	bBeingDestroyed = false;
#endif
}

/**
 * Cleans up the handle if valid
 */
FSemaphore::~FSemaphore(void)
{
#if PLATFORM_WINDOWS
	if (Semaphore != NULL)
	{
		CloseHandle(Semaphore);
	}
#else
	if( bInitialized )
	{
		// Release all threads waiting on semaphore, as abandoned
		bBeingDestroyed = true;
		pthread_cond_broadcast(&Condition);

		// Wait until they go. Warning: if one of those threads is killed while waiting somehow, this will hang.
		while( Counter < 0 )
		{
			FPlatformProcess::Sleep(0);	// yield to other threads
		}

		// Destroy objects
		pthread_cond_destroy(&Condition);
		pthread_mutex_destroy(&Guard);

		bInitialized = false;
	}
#endif // PLATFORM_WINDOWS
}

/**
 * Creates the semaphore.
 * Named events share the same underlying event.
 *
 * @apram MaxCount Maximum value of the semaphore. The semaphore will always be [0..MaxCount].
 *
 * @return Returns true if the event was created, false otherwise
 */
bool FSemaphore::Create(int32 InMaxCount)
{
	// Create the event and default it to non-signaled
#if PLATFORM_WINDOWS
	Semaphore = CreateSemaphore(NULL, 0, InMaxCount, NULL);
	return Semaphore != NULL;
#else
	if( !bInitialized )
	{
		check( InMaxCount > 0 );
		MaxCount = InMaxCount;
		Counter = 0;

		int Result = pthread_mutex_init( &Guard, NULL );
		if( Result == 0 )
		{
			Result = pthread_cond_init( &Condition, NULL );
			if( Result == 0 )
			{
				bInitialized = true;
			}
			else
			{
				pthread_mutex_destroy( &Guard );
			}
		}
	}

	return bInitialized;
#endif
}

/**
 * Increments the semaphore by 1.
 * If multiple threads are waiting, one of them will be released.
 */
void FSemaphore::Trigger(void)
{
#if PLATFORM_WINDOWS
	check( Semaphore );
	LONG PreviousCount;
	ReleaseSemaphore( Semaphore, 1, &PreviousCount );
#else
	if( bInitialized && !bBeingDestroyed )
	{
		pthread_mutex_lock(&Guard);
		if( Counter < MaxCount )
		{
			if( Counter < 0 )
			{
				pthread_cond_signal(&Condition);
			}
			else
			{
				++Counter;
			}
		}
		pthread_mutex_unlock(&Guard);
	}
#endif // PLATFORM_WINDOWS
}

/**
 * Waits for the semaphore to be signaled (non-zero value).
 * Upon return, decrement the semaphore by 1 (unless the timeout was reached).
 *
 * @param WaitTime Time in milliseconds to wait before abandoning the event
 * (uint32)-1 is treated as wait infinite
 *
 * @return true if the semaphore was signaled, false if the wait timed out
 */
bool FSemaphore::Wait(uint32 WaitTime /*= (uint32)-1*/)
{
#if PLATFORM_WINDOWS
	check( Semaphore );
	return WaitForSingleObject(Semaphore,WaitTime) == WAIT_OBJECT_0;
#else
	if( !bInitialized || bBeingDestroyed )
	{
		return false;
	}

	pthread_mutex_lock(&Guard);

	if( Counter > 0 )
	{
		// Signalled, we can pass
		--Counter;
		pthread_mutex_unlock(&Guard);
		return !bBeingDestroyed;
	}

	if( WaitTime == 0 )
	{
		// Just yield to other threads; instant timeout
		pthread_mutex_unlock(&Guard);
		FPlatformProcess::Sleep(0);
		return false;	// timeout
	}

	--Counter;

	int Result;
	if( WaitTime == (uint32)-1 )
	{
		Result = pthread_cond_wait( &Condition, &Guard );
	}
	else
	{
		struct timespec Ts;

		// Calculate the moment we need to wait for
		struct timeval Tv;
		gettimeofday( &Tv, NULL );
		Ts.tv_sec = Tv.tv_sec + ( WaitTime / 1000 );
		Ts.tv_nsec = Tv.tv_usec * 1000 + ( WaitTime % 1000 ) * 1000000;
		if( Ts.tv_nsec > 1000000000 )	// if there's overflow, carry it to tv_sec
		{
			Ts.tv_nsec -= 1000000000;
			++Ts.tv_sec;
		}

		Result = pthread_cond_timedwait( &Condition, &Guard, &Ts );
	}

	++Counter;

	bool bRet = !bBeingDestroyed && Result == 0;

	pthread_mutex_unlock(&Guard);

	FPlatformProcess::Sleep(0);	// on Windows, timeout always yields thread (but it also temporarily increases priority of it, and we don't do this)

	return bRet;
#endif
}

}

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS
