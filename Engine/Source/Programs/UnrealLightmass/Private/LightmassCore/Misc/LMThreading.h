// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LMQueue.h"
#include "Misc/ScopeLock.h"

#if PLATFORM_USE_PTHREADS
#include <pthread.h>
#endif

namespace Lightmass
{

/**
 * This is the Windows version of a semaphore.
 * It works like a thread-safe counter which is considered 'signaled' when non-zero.
 * Trigger() will increment the counter and a successful Wait() will decrement it.
 */
class FSemaphore
{
public:
	/**
	 * Constructor that zeroes the handle
	 */
	FSemaphore(void);

	/**
	 * Cleans up the handle if valid
	 */
	virtual ~FSemaphore(void);

	/**
	 * Creates the semaphore.
	 * Named events share the same underlying event.
	 *
	 * @apram MaxCount Maximum value of the semaphore. The semaphore will always be [0..MaxCount].
	 *
	 * @return Returns true if the event was created, false otherwise
	 */
	virtual bool Create(int32 MaxCount);

	/**
	 * Increments the semaphore by 1.
	 * If multiple threads are waiting, one of them will be released.
	 */
	virtual void Trigger(void);

	/**
	 * Waits for the semaphore to be signaled (non-zero value).
	 * Upon return, decrement the semaphore by 1 (unless the timeout was reached).
	 *
	 * @param WaitTime Time in milliseconds to wait before abandoning the event
	 * (uint32)-1 is treated as wait infinite
	 *
	 * @return true if the semaphore was signaled, false if the wait timed out
	 */
	virtual bool Wait(uint32 WaitTime = (uint32)-1);

protected:
	/**
	 * The handle to the semaphore
	 */
#if PLATFORM_WINDOWS
	Windows::HANDLE Semaphore;
#else
	pthread_mutex_t Guard;
	pthread_cond_t	Condition;
	bool			bInitialized;
	bool			bBeingDestroyed;
	int				Counter;
	int				MaxCount;
#endif
};

/**
 * Thread-safe FIFO queue with a fixed maximum size.
 */
template <typename ElementType>
class TQueueThreadSafe : private TQueue<ElementType>
{
public:
	/**
	 * Constructor, allocates the buffer.
	 * @param InMaxNumElements	- Maximum number of elements in the queue
	 */
	TQueueThreadSafe( int32 InMaxNumElements )
	:	TQueue<ElementType>( InMaxNumElements )
	{
	}

	/**
	 * Returns the current number of elements stored in the queue.
	 * @return	- Current number of elements stored in the queue
	 */
	volatile int32 Num() const
	{
		return TQueue<ElementType>::Num();
	}

	/**
	 * Returns the maximum number of elements that can be stored in the queue.
	 * @return	- Maximum number of elements that can be stored in the queue
	 */
	volatile int32 GetMaxNumElements() const
	{
		return TQueue<ElementType>::GetMaxNumElements();
	}

	/**
	 * Adds an element to the head of the queue.
	 * @param Element	- Element to be added
	 * @return			- true if the element got added, false if the queue was already full
	 */
	bool	Push( const ElementType& Element )
	{
		FScopeLock Lock( &CriticalSecion );
		return TQueue<ElementType>::Push( Element );
	}

	/**
	 * Removes and returns the tail of the queue (the 'oldest' element).
	 * @param Element	- [out] If successful, contains the element that was removed
	 * @return			- true if the element got removed, false if the queue was empty
	 */
	bool	Pop( ElementType& Element )
	{
		FScopeLock Lock( &CriticalSecion );
		return TQueue<ElementType>::Pop( Element );
	}

protected:
	/** Used for synchronizing access to the queue. */
	FCriticalSection	CriticalSecion;
};


/**
 * Thread-safe Producer/Consumer FIFO queue with a fixed maximum size.
 * It supports multiple producers and multiple consumers.
 */
template <typename ElementType>
class TProducerConsumerQueue : private TQueueThreadSafe<ElementType>
{
public:
	/**
	 * Constructor, allocates the buffer.
	 * @param InMaxNumElements	- Maximum number of elements in the queue
	 */
	TProducerConsumerQueue( int32 InMaxNumElements )
	:	TQueueThreadSafe<ElementType>( InMaxNumElements )
	{
		Semaphore.Create( InMaxNumElements );
	}

	/**
	 * Returns the current number of elements stored in the queue.
	 * @return	- Current number of elements stored in the queue
	 */
	volatile int32 Num() const
	{
		return TQueueThreadSafe<ElementType>::Num();
	}

	/**
	 * Returns the maximum number of elements that can be stored in the queue.
	 * @return	- Maximum number of elements that can be stored in the queue
	 */
	volatile int32 GetMaxNumElements() const
	{
		return TQueueThreadSafe<ElementType>::GetMaxNumElements();
	}

	/**
	 * Adds an element to the head of the queue.
	 *
	 * @param Element	- Element to be added
	 * @return			- true if the element got added, false if the queue was already full
	 */
	bool	Push( const ElementType& Element )
	{
		bool bSucceeded = TQueueThreadSafe<ElementType>::Push( Element );
		if ( bSucceeded )
		{
			Semaphore.Trigger();
		}
		return bSucceeded;
	}

	/**
	 * Removes and returns the tail of the queue (the 'oldest' element).
	 * If the queue is empty, wait for an element to come in.
	 *
	 * @param Element	- [out] If successful, contains the element that was removed
	 * @param Timeout	- How long to wait for new elements if the queue is empty, in milliseconds. -1 means INFINITE.
	 * @return			- true if the element got removed, false if the queue was empty
	 */
	bool	Pop( ElementType& Element, uint32 Timeout )
	{
		Semaphore.Wait( Timeout );
		return TQueueThreadSafe<ElementType>::Pop( Element );
	}

	/** Bumps the semaphore to maximum, to release up to 'MaxNumElements' waiting threads right away. */
	void	TriggerAll()
	{
		for ( int32 NumPotentialThreads=0; NumPotentialThreads < GetMaxNumElements(); ++NumPotentialThreads )
		{
			Semaphore.Trigger();
		}
	}

protected:
	/** Used for blocking Pops. The count represents the current number of elements in the queue. */
	FSemaphore		Semaphore;
};


}
