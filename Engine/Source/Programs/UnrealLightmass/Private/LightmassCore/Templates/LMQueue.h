// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace Lightmass
{


/**
 * FIFO queue with a fixed maximum size.
 */
template <typename ElementType>
class TQueue
{
public:
	/**
	 * Constructor, allocates the buffer.
	 * @param InMaxNumElements	- Maximum number of elements in the queue
	 */
	TQueue( int32 InMaxNumElements )
	:	HeadIndex(0)
	,	TailIndex(0)
	,	NumElements(0)
	,	MaxNumElements( InMaxNumElements )
	{
		Elements = new ElementType[MaxNumElements];
	}

	/**
	 * Destructor, frees the buffer.
	 */
	~TQueue()
	{
		delete [] Elements;
	}

	/**
	 * Returns the current number of elements stored in the queue.
	 * @return	- Current number of elements stored in the queue
	 */
	int32		Num() const
	{
		return NumElements;
	}

	/**
	 * Returns the maximum number of elements that can be stored in the queue.
	 * @return	- Maximum number of elements that can be stored in the queue
	 */
	int32		GetMaxNumElements() const
	{
		return MaxNumElements;
	}

	/**
	 * Adds an element to the head of the queue.
	 * @param Element	- Element to be added
	 * @return			- true if the element got added, false if the queue was already full
	 */
	bool	Push( const ElementType& Element )
	{
		if ( NumElements < MaxNumElements )
		{
			Elements[HeadIndex] = Element;
			HeadIndex = (HeadIndex + 1) % MaxNumElements;
			NumElements++;
			return true;
		}
		else
		{
			return false;
		}
	}

	/**
	 * Removes and returns the tail of the queue (the 'oldest' element).
	 * @param Element	- [out] If successful, contains the element that was removed
	 * @return			- true if the element got removed, false if the queue was empty
	 */
	bool	Pop( ElementType& Element )
	{
		if ( NumElements > 0 )
		{
			Element = Elements[TailIndex];
			TailIndex = (TailIndex + 1) % MaxNumElements;
			NumElements--;
			return true;
		}
		else
		{
			return false;
		}
	}

protected:
	/** Index to where a new element will be added (the head of the queue). */
	int32						HeadIndex;
	/** Index to the least recently added element (the tail of the queue). */
	int32						TailIndex;
	/** Current number of elements stored in the queue. */
	int32						NumElements;
	/** Maximum number of elements that can be stored in the queue. */
	const int32				MaxNumElements;
	/** Array of elements. The allocated size is MaxNumElements. */
	ElementType*			Elements;
};

}	//namespace Lightmass
