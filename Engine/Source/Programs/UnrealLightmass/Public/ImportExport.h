// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "CoreMinimal.h"
//#include "HAL/UnrealMemory.h"
//#include "Math/UnrealMathUtility.h"
//#include "Containers/UnrealString.h"
//#include "Misc/Guid.h"
//#include "Math/Color.h"
//#include "Math/Vector.h"
//#include "Math/Vector4.h"
//#include "Math/Matrix.h"
//#include "Math/Float16.h"

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/Matrix.h"
#include "Math/Float16.h"
#include "Misc/Guid.h"
#include "SceneExport.h"

struct FDebugLightingCacheRecord;
struct FDebugLightingOutput;
struct FDebugOctreeNode;
struct FDebugPhoton;
struct FDebugStaticLightingRay;
struct FDebugStaticLightingVertex;
struct FDebugVolumeLightingSample;

namespace Lightmass
{	
/** Whether or not to request compression on heavyweight input file types */
#define LM_COMPRESS_INPUT_DATA 1

#define LM_NUM_SH_COEFFICIENTS 9

	/** The number of coefficients that are stored for each light sample. */ 
	static const int32 LM_NUM_STORED_LIGHTMAP_COEF = 4;
	/** The number of high quality coefficients which the lightmap stores for each light sample. */ 
	static const int32 LM_NUM_HQ_LIGHTMAP_COEF = 2;
	/** The index at which low quality coefficients are stored in any array containing all LM_NUM_STORED_LIGHTMAP_COEF coefficients. */ 
	static const int32 LM_LQ_LIGHTMAP_COEF_INDEX = 2;

	/** Output channel types (extensions) */
	static const TCHAR LM_TEXTUREMAPPING_EXTENSION[]	= TEXT("tmap");
	static const TCHAR LM_VOLUMESAMPLES_EXTENSION[]		= TEXT("vols");
	static const TCHAR LM_VOLUMEDEBUGOUTPUT_EXTENSION[]	= TEXT("vold");
	static const TCHAR LM_VOLUMETRICLIGHTMAP_EXTENSION[]	= TEXT("irvol");
	static const TCHAR LM_PRECOMPUTEDVISIBILITY_EXTENSION[]	= TEXT("vis");
	static const TCHAR LM_DOMINANTSHADOW_EXTENSION[]	= TEXT("doms");
	static const TCHAR LM_MESHAREALIGHTDATA_EXTENSION[]	= TEXT("arealights");
	static const TCHAR LM_DEBUGOUTPUT_EXTENSION[]		= TEXT("dbgo");

	/** Input channel types (extensions) */
#if LM_COMPRESS_INPUT_DATA
	static const TCHAR LM_SCENE_EXTENSION[]				= TEXT("scenegz");
	static const TCHAR LM_STATICMESH_EXTENSION[]		= TEXT("meshgz");
	static const TCHAR LM_MATERIAL_EXTENSION[]			= TEXT("mtrlgz");
#else
	static const TCHAR LM_SCENE_EXTENSION[]				= TEXT("scene");
	static const TCHAR LM_STATICMESH_EXTENSION[]		= TEXT("mesh");
	static const TCHAR LM_MATERIAL_EXTENSION[]			= TEXT("mtrl");
#endif

	/** 
	 * Channel versions 
	 * Adding a new guid for any of these forces re-exporting of that data type.
	 * This must be done whenever modifying code that changes how each data type is exported!
	 */

	static const int32 LM_TEXTUREMAPPING_VERSION		= 2;
	static const int32 LM_VOLUMESAMPLES_VERSION			= 2;
	static const int32 LM_VOLUMETRICLIGHTMAP_VERSION		= 4;
	static const int32 LM_PRECOMPUTEDVISIBILITY_VERSION	= 2;
	static const int32 LM_VOLUMEDEBUGOUTPUT_VERSION		= 2;
	static const int32 LM_DOMINANTSHADOW_VERSION		= 2;
	static const int32 LM_MESHAREALIGHTDATA_VERSION		= 2;
	static const int32 LM_DEBUGOUTPUT_VERSION			= 2;
	static const int32 LM_SCENE_VERSION					= 2;
	static const int32 LM_STATICMESH_VERSION			= 2;
	static const int32 LM_MATERIAL_VERSION				= 2;


	/** Alert source object type identifiers... */
	enum ESourceObjectType
	{
		SOURCEOBJECTTYPE_Unknown = 0,
		SOURCEOBJECTTYPE_Scene,
		SOURCEOBJECTTYPE_Material,
		SOURCEOBJECTTYPE_BSP,
		SOURCEOBJECTTYPE_StaticMesh,
		SOURCEOBJECTTYPE_Fluid,
		SOURCEOBJECTTYPE_SpeedTree,
		SOURCEOBJECTTYPE_TextureMapping,
		SOURCEOBJECTTYPE_VertexMapping,
		SOURCEOBJECTTYPE_Mapping
	};

#if !PLATFORM_MAC && !PLATFORM_LINUX
	#pragma pack(push, 1)
#endif

	/** 
	 * Incident lighting for a single sample, as produced by a lighting build. 
	 * FGatheredLightSample is used for gathering lighting instead of this format as FLightSampleData is not additive.
	 */
	struct FLightSampleData
	{
		FLightSampleData()
		{
			bIsMapped = false;
			FMemory::Memzero(Coefficients,sizeof(Coefficients));
			SkyOcclusion[0] = 0;
			SkyOcclusion[1] = 0;
			SkyOcclusion[2] = 0;
			AOMaterialMask = 0;
		}

		/** 
		 * Coefficients[0] stores the normalized average color, 
		 * Coefficients[1] stores the maximum color component in each lightmap basis direction,
		 * and Coefficients[2] stores the simple lightmap which is colored incident lighting along the vertex normal.
		 */
		float Coefficients[LM_NUM_STORED_LIGHTMAP_COEF][3];

		float SkyOcclusion[3];

		float AOMaterialMask;

		/** True if this sample maps to a valid point on a triangle.  This is only meaningful for texture lightmaps. */
		bool bIsMapped;

		/**
		 * Export helper
		 * @param Component Which directional lightmap component to retrieve
		 * @return An FColor for this component, clamped to White
		 */
		FColor GetColor(int32 Component) const
		{
			return FColor(
				(uint8)FMath::Clamp<int32>(Coefficients[Component][0] * 255, 0, 255), 
				(uint8)FMath::Clamp<int32>(Coefficients[Component][1] * 255, 0, 255), 
				(uint8)FMath::Clamp<int32>(Coefficients[Component][2] * 255, 0, 255),
				0);
		}
	};

	/** The light incident for a point on a surface. */
	class FLightSample : public FLightSampleData
	{
	public:
		/** Initialization constructor. */
		FLightSample() : FLightSampleData() {}
		FLightSample(EForceInit) : FLightSampleData() {}
	};

	/**
	 * The quantized coefficients for a single light-map texel.
	 */
	struct FQuantizedLightSampleData
	{
		uint8 Coverage;
		uint8 Coefficients[LM_NUM_STORED_LIGHTMAP_COEF][4];
		uint8 SkyOcclusion[4];
		uint8 AOMaterialMask;
	};

	struct FLightMapDataBase
	{
		/** Size of compressed lightmap data */
		uint32 CompressedDataSize;

		/** Size of uncompressed lightmap data */
		uint32 UncompressedDataSize;

		/** Scale applied to the quantized light samples */
		float Multiply[LM_NUM_STORED_LIGHTMAP_COEF][4];

        /** Bias applied to the quantized light samples */
		float Add[LM_NUM_STORED_LIGHTMAP_COEF][4];
	};

	/** LightMap data 2D */
	struct FLightMapData2DData : public FLightMapDataBase
	{
		FLightMapData2DData(uint32 InSizeX,uint32 InSizeY):
			SizeX(InSizeX),
			SizeY(InSizeY),
			bHasSkyShadowing(false)
		{
		}

		/** The width of the light-map. */
		uint32 SizeX;
		/** The height of the light-map. */
		uint32 SizeY;
		bool bHasSkyShadowing;
	};

	struct FShadowMapDataBase
	{
		/** Size of compressed shadowmap data */
		uint32 CompressedDataSize;

		/** Size of uncompressed shadowmap data */
		uint32 UncompressedDataSize;
	};

	/** A sample of the visibility factor between a light and a single point. */
	struct FShadowSampleData
	{
		/** The fraction of light that reaches this point from the light, between 0 and 1. */
		float Visibility;
		/** True if this sample maps to a valid point on a surface. */
		bool bIsMapped;

		/**
		 * Export helper
		 * @param Component Which directional lightmap component to retrieve
		 * @return An FColor for this component, clamped to White
		 */
		FColor GetColor(int32 Component) const
		{
			uint8 Gray = (uint8)FMath::Clamp<int32>(Visibility * 255, 0, 255);
			return FColor(Gray, Gray, Gray, 0);
		}

	};

	/** The quantized value for a single shadowmap texel */
	struct FQuantizedShadowSampleData
	{
		uint8 Visibility;
		uint8 Coverage;
	};

	/** ShadowMap data 2D */
	struct FShadowMapData2DData : public FShadowMapDataBase
	{
		FShadowMapData2DData(uint32 InSizeX,uint32 InSizeY):
			SizeX(InSizeX),
			SizeY(InSizeY)
		{
		}

		/** The width of the shadow-map. */
		uint32 SizeX;
		/** The height of the shadow-map. */
		uint32 SizeY;
	};

	struct FSignedDistanceFieldShadowSampleData
	{
		/** Normalized and encoded distance to the nearest shadow transition, in the range [0,1], where .5 is the transition. */
		float Distance;
		/** Normalized penumbra size, in [0,1] */
		float PenumbraSize;
		/** True if this sample maps to a valid point on a surface. */
		bool bIsMapped;
	};

	/** The quantized value for a single signed-distance field texel */
	struct FQuantizedSignedDistanceFieldShadowSampleData
	{
		uint8 Distance;
		uint8 PenumbraSize;
		uint8 Coverage;
	};

	/** 2D distance field data. */
	struct FSignedDistanceFieldShadowMapData2DData : public FShadowMapData2DData
	{
		FSignedDistanceFieldShadowMapData2DData(uint32 InSizeX,uint32 InSizeY):
			FShadowMapData2DData(InSizeX, InSizeY)
		{
		}
	};

	/** Lighting for a point in space. */
	class FVolumeLightingSampleData
	{
	public:
		/** World space position and radius. */
		FVector4f PositionAndRadius;

		/** SH coefficients used with high quality lightmaps. */
		float HighQualityCoefficients[LM_NUM_SH_COEFFICIENTS][3];

		/** SH coefficients used with low quality lightmaps. */
		float LowQualityCoefficients[LM_NUM_SH_COEFFICIENTS][3];

		FVector3f SkyBentNormal;

		/** Shadow factor for the stationary directional light. */
		float DirectionalLightShadowing;
	};

	/** Data used by the editor import process and not uploaded into textures. */
	struct FIrradianceVoxelImportProcessingData
	{
		bool bInsideGeometry;
		bool bBorderVoxel;
		float ClosestGeometryDistance;
	};

	/** */
	class FStaticShadowDepthMapSampleData
	{
	public:
		FFloat16 Distance;
	};

	class FStaticShadowDepthMapData
	{
	public:
		/** Transform from world space to the coordinate space that FStaticShadowDepthMapSampleData's are stored in. */
		FMatrix44f WorldToLight;
		/** Dimensions of the generated shadow map. */
		int32 ShadowMapSizeX;
		int32 ShadowMapSizeY;
	};

	static const FGuid MeshAreaLightDataGuid = FGuid(0xe11f4760, 0xfa454d2b, 0xa090c388, 0x33326646);

	static const FGuid VolumeDistanceFieldGuid = FGuid(0x4abf306e, 0x4c2f4a6e, 0x9feb5fa4, 0x5b910a8f);
	
	class FMeshAreaLightData
	{
	public:
		FGuid LevelGuid;
		FVector4f Position;
		FVector4f Direction;
		float Radius;
		float ConeAngle;
		FColor Color;
		float Brightness;
		float FalloffExponent;
	};

	/** 
	 * Types used for transfering debug information back to Unreal. 
	 * NOTE: These must stay binary compatible with the corresponding versions in Unreal.
	 */

	struct FDebugStaticLightingRay
	{
		FVector4f Start;
		FVector4f End;
		bool bHit;
		bool bPositive;

		FDebugStaticLightingRay() {}
		FDebugStaticLightingRay(const FVector4f& InStart, const FVector4f& InEnd, bool bInHit, bool bInPositive = false) :
			Start(InStart),
			End(InEnd),
			bHit(bInHit),
			bPositive(bInPositive)
		{}
	};

	struct FDebugStaticLightingVertex
	{
		FVector4f VertexNormal;
		FVector4f VertexPosition;
	};

	struct FDebugLightingCacheRecord
	{
		bool bNearSelectedTexel;
		bool bAffectsSelectedTexel;
		int32 RecordId;
		FDebugStaticLightingVertex Vertex;
		float Radius;
		 
		FDebugLightingCacheRecord() :
			bNearSelectedTexel(false),
			bAffectsSelectedTexel(false)
		{}
	};

	struct FDebugPhoton
	{
		int32 Id;
		FVector4f Position;
		FVector4f Direction;
		FVector4f Normal;

		FDebugPhoton() :
			Id(-1)
		{}

		FDebugPhoton(int32 InId, const FVector4f& InPosition, const FVector4f& InDirection, const FVector4f& InNormal) :
			Id(InId),
			Position(InPosition),
			Direction(InDirection),
			Normal(InNormal)
		{}
	};

	struct FDebugOctreeNode
	{
		FVector4f Center;
		FVector4f Extent;

		FDebugOctreeNode(const FVector4f& InCenter, const FVector4f& InExtent) :
			Center(InCenter),
			Extent(InExtent)
		{}
	};

	/** Guid used by Unreal to determine when the debug channel with the same Guid can be opened. */
	static const FGuid DebugOutputGuid = FGuid(0x23219c9e, 0xb5934266, 0xb2144a7d, 0x3448abac);

	/** 
	 * Debug output from the static lighting build.  
	 */
	struct FDebugLightingOutput
	{
		/** Whether the debug output is valid. */
		bool bValid;
		/** Final gather, hemisphere sample and path rays */
		TArray<FDebugStaticLightingRay> PathRays;
		/** Area shadow rays */
		TArray<FDebugStaticLightingRay> ShadowRays;
		/** Photon paths used for guiding indirect photon emission */
		TArray<FDebugStaticLightingRay> IndirectPhotonPaths;
		/** Indices into Vertices of the selected sample's vertices */
		TArray<int32> SelectedVertexIndices;
		/** Vertices near the selected sample */
		TArray<FDebugStaticLightingVertex> Vertices;
		/** Lighting cache records */
		TArray<FDebugLightingCacheRecord> CacheRecords;
		/** Photons in the direct photon map */
		TArray<FDebugPhoton> DirectPhotons;
		/** Photons in the indirect photon map */
		TArray<FDebugPhoton> IndirectPhotons;
		/** Photons in the irradiance photon map */
		TArray<FDebugPhoton> IrradiancePhotons;
		/** Normal and irradiance photons that were gathered for the selected sample */
		TArray<FDebugPhoton> GatheredPhotons;
		/** Importance photons that were gathered for the selected sample */
		TArray<FDebugPhoton> GatheredImportancePhotons;
		/** Photon map octree nodes gathered during a photon map search. */
		TArray<FDebugOctreeNode> GatheredPhotonNodes;
		/** Whether GatheredDirectPhoton is valid */
		bool bDirectPhotonValid;
		/** Direct photon that was found during direct lighting. */
		FDebugPhoton GatheredDirectPhoton;
		/** Positions of the selected texel's corners */
		FVector4f TexelCorners[NumTexelCorners];
		/** Whether each of the selected texel's corners were valid */
		bool bCornerValid[NumTexelCorners];
		/** World space radius of the selected sample */
		float SampleRadius;

		FDebugLightingOutput() :
			bValid(false),
			bDirectPhotonValid(false)
		{
			for (int32 CornerIndex = 0; CornerIndex < NumTexelCorners; CornerIndex++)
			{
				bCornerValid[CornerIndex] = false;
			}
		}
	};

	struct FDebugVolumeLightingSample
	{
		FVector4f PositionAndRadius;
		FLinearColor AverageIncidentRadiance;

		FDebugVolumeLightingSample(const FVector4f& InPositionAndRadius, const FLinearColor& InAverageIncidentRadiance) :
			PositionAndRadius(InPositionAndRadius),
			AverageIncidentRadiance(InAverageIncidentRadiance)
		{}
	};

	struct FVolumeLightingDebugOutput
	{
		TArray<FDebugVolumeLightingSample> VolumeLightingSamples;
	};

	/** Guid used by Unreal to determine when the volume lighting debug channel with the same Guid can be opened. */
	static const FGuid VolumeLightingDebugOutputGuid = FGuid(0x1e8119ff, 0xa46f48f8, 0x92b18d49, 0x172c5832);
	/** Guid used by Unreal to determine when the volume lighting sample channel with the same Guid can be opened. */
	static const FGuid PrecomputedVolumeLightingGuid = FGuid(0xce97c5c3, 0xab614fd3, 0xb2da55c0, 0xe6c33fb4);

#if !PLATFORM_MAC && !PLATFORM_LINUX
	#pragma pack(pop)
#endif

	/**
	 * Creates a standardized channel name based on Guid, version and type
	 *
	 * @param Guid Unique ID of the channel
	 * @param Version Version of the data inside
	 * @param LMExecutableHash Hash of UnrealLightmass executable, to avoid stale cache content from a different build
	 * @param Extension Type of the data
	 *
	 * @return Standard channel name
	*/
	inline FString CreateChannelNameWithLMExecutableHash(const FGuid& Guid, const int32 Version, const FSHAHash& LMExecutableHash, const FString& Extension)
	{
		return FString::Printf(TEXT("v%d.%08X%08X%08X%08X.%s.%s"), Version, Guid.A, Guid.B, Guid.C, Guid.D, *LMExecutableHash.ToString(), *Extension);
	}
	inline FString CreateChannelName(const FGuid& Guid, const int32 Version, const FString& Extension)
	{
		return FString::Printf(TEXT("v%d.%08X%08X%08X%08X.%s"), Version, Guid.A, Guid.B, Guid.C, Guid.D, *Extension);
	}

	/**
	 * Creates a standardized channel name based a FSA Hash, version and type. Used for exporting materials
	 *
	 * @param Hash Unique ID of the channel
	 * @param Version Version of the data inside
	 * @param LMExecutableHash Hash of UnrealLightmass executable, to avoid stale cache content from a different build
	 * @param Extension Type of the data
	 *
	 * @return Standard channel name
	 */
	inline FString CreateChannelNameWithLMExecutableHash(const FSHAHash& Hash, const int32 Version, const FSHAHash& LMExecutableHash, const FString& Extension)
	{
		return FString::Printf(TEXT("v%d.%s.%s.%s"), Version, *Hash.ToString(), *LMExecutableHash.ToString(), *Extension);
	}
} // namespace Lightmass
