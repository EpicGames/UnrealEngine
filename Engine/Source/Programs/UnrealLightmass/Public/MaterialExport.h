// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreMinimal.h"
#include "Misc/Guid.h"

namespace Lightmass
{

#if !PLATFORM_MAC && !PLATFORM_LINUX
#pragma pack(push, 1)
#endif

//----------------------------------------------------------------------------
//	Helper definitions
//----------------------------------------------------------------------------

//----------------------------------------------------------------------------
//	Mesh export file header
//----------------------------------------------------------------------------
struct FMaterialFileHeader
{
	/** FourCC cookie: 'MTRL' */
	uint32		Cookie;
	FGuid		FormatVersion;
};

//----------------------------------------------------------------------------
//	Base material
//----------------------------------------------------------------------------
struct FBaseMaterialData
{
	FGuid		Guid;
};

//----------------------------------------------------------------------------
//	Material data, builds upon FBaseMaterialData
//----------------------------------------------------------------------------
/**
 *	Material blend mode.
 *	MUST MATCH UNREAL EXACTLY!!!
 */
enum EBlendMode
{
    BLEND_Opaque                          =0,
    BLEND_Masked                          =1,
    BLEND_Translucent                     =2,
    BLEND_Additive                        =3,
    BLEND_Modulate                        =4,
	BLEND_AlphaComposite                  =5,
	BLEND_AlphaHoldout                    =6,
	BLEND_TranslucentColoredTransmittance =7,					/*Substrate only */
	BLEND_MAX                             =8,
	BLEND_TranslucentGreyTransmittance    = BLEND_Translucent,	/*Substrate only */
	BLEND_ColoredTransmittanceOnly        = BLEND_Modulate,		/*Substrate only */
};

struct FMaterialData
{
	/** The BLEND mode of the material */
	EBlendMode BlendMode;
	/** Whether the material is two-sided or not */
	uint32 bTwoSided:1;
	/** Whether the material is thin-surface or not */
	uint32 bIsThinSurface: 1;
	/** Whether the material should cast shadows as masked even though it has a translucent blend mode. */
	uint32 bCastShadowAsMasked:1;
	uint32 bSurfaceDomain:1;
	/** Scales the emissive contribution for this material. */
	float EmissiveBoost;
	/** Scales the diffuse contribution for this material. */
	float DiffuseBoost;
	/** The clip value for masked rendering */
	float OpacityMaskClipValue;

	/** 
	 *	The sizes of the material property samples
	 */
	int32 EmissiveSize;
	int32 DiffuseSize;
	int32 TransmissionSize;
	int32 NormalSize;

	FMaterialData() :
		  EmissiveBoost(1.0f)
		, DiffuseBoost(1.0f)
		, EmissiveSize(0)
		, DiffuseSize(0)
		, TransmissionSize(0)
		, NormalSize(0)
	{
	}
};

#if !PLATFORM_MAC && !PLATFORM_LINUX
#pragma pack(pop)
#endif

}	// namespace Lightmass
