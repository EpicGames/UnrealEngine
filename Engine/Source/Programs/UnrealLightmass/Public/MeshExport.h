// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "CoreMinimal.h"
#include "Misc/Guid.h"
//#include "Math/Vector2D.h"
//#include "Math/Vector4.h"

namespace Lightmass
{

enum { MAX_TEXCOORDS=4 };

#if !PLATFORM_MAC && !PLATFORM_LINUX
#pragma pack(push, 1)
#endif

//----------------------------------------------------------------------------
//	Helper definitions
//----------------------------------------------------------------------------


//----------------------------------------------------------------------------
//	Mesh export file header
//----------------------------------------------------------------------------
struct FMeshFileHeader
{
	/** FourCC cookie: 'MESH' */
	uint32			Cookie;
	FGuid		FormatVersion;

	// These structs follow immediately after this struct.
	//
	//	FBaseMeshData			BaseMeshData;
	//	FStaticMeshData			MeshData;
	//	StaticMeshLODAggregate	MeshLODs[ MeshData.NumLODs ];
	//
	//	Where:
	//
	// 	struct StaticMeshLODAggregate
	// 	{
	//		FStaticMeshLODData		LOD;
	// 		FStaticMeshElementData	MeshElements[ LOD.NumElements ];
	//		UINT16					Indices[ LOD.NumIndices ];
	//		FStaticMeshVertex		Vertices[ LOD.NumVertices ];
	// 	};
};


//----------------------------------------------------------------------------
//	Base mesh
//----------------------------------------------------------------------------
struct FBaseMeshData
{
	FGuid		Guid;
};

//----------------------------------------------------------------------------
//	Static mesh, builds upon FBaseMeshData
//----------------------------------------------------------------------------
struct FStaticMeshData
{
	uint32			LightmapCoordinateIndex;
	uint32			NumLODs;
};

//----------------------------------------------------------------------------
//	Static mesh LOD
//----------------------------------------------------------------------------
struct FStaticMeshLODData
{
	uint32			NumElements;
	/** Total number of triangles for all elements in the LOD. */
	uint32			NumTriangles;
	/** Total number of indices in the LOD. */
	uint32			NumIndices;
	/** Total number of vertices in the LOD. */
	uint32			NumVertices;
};

//----------------------------------------------------------------------------
//	Static mesh element
//----------------------------------------------------------------------------
struct FStaticMeshElementData
{
	uint32		FirstIndex;
	uint32		NumTriangles;
	uint32	bEnableShadowCasting : 1;
};

//----------------------------------------------------------------------------
//	Static mesh vertex
//----------------------------------------------------------------------------
struct FStaticMeshVertex
{
	FVector4f		Position;
	FVector4f		TangentX;
	FVector4f		TangentY;
	FVector4f		TangentZ;
	FVector2f		UVs[MAX_TEXCOORDS];
};

#if !PLATFORM_MAC && !PLATFORM_LINUX
#pragma pack(pop)
#endif

}	// namespace Lightmass
