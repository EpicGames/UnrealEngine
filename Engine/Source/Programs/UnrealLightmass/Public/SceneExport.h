// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Containers/EnumAsByte.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/Box.h"
#include "Math/Matrix.h"
#include "Math/SHMath.h"
#include "Misc/Guid.h"
#include "MeshExport.h"
#ifdef UE_LIGHTMASS
#include "Misc/LMHelpers.h"
#endif

struct FSplineMeshParams;

namespace Lightmass
{

#if !PLATFORM_MAC && !PLATFORM_LINUX
#pragma pack(push, 1)
#endif

static const int32 NumTexelCorners = 4;

/** 
 * General and misc solver settings.
 * Settings prefixed by Debugging are only useful for development, 
 * Either because one of the options have been proven superior or because it produces a visualization of an intermediate result.
 * All angles are in Radians, distances are in world space units.
 */
class FStaticLightingSettings 
{
public:
	/** Debugging - whether to allow multiple static lighting threads. */
	bool bAllowMultiThreadedStaticLighting;

	/**
	 * Number of local cores to leave unused
	 */
	int32 NumUnusedLocalCores;

	/** 
	 * Number of indirect lighting bounces to simulate, 0 is direct lighting only. 
	 * The first bounce always costs the most in terms of computation time, with the second bounce following.  
	 * With photon mapping, bounces after the second are nearly free.
	 */
	int32 NumIndirectLightingBounces;

	/** 
	 * Number of skylight and emissive bounces to simulate.  
	 * Lightmass uses a non-distributable radiosity method for skylight bounces whose cost is proportional to the number of bounces.
	 */
	int32 NumSkyLightingBounces;

	/** 
	 * Whether to use Embree for ray tracing or not.
	 */
	bool bUseEmbree;

	/** 
	 * Whether to check for Embree coherency.
	 */
	bool bVerifyEmbree;

	/** Whether to build Embree data structures for packet tracing. WIP feature - no lightmass algorithms emit packet tracing requests yet. */
	bool bUseEmbreePacketTracing;

	/** Whether to use kDOP trees to accelerate volumetric lightmap voxelization. Useful in scenes like large forest. */
	bool bUseFastVoxelization;

	/** Whether to use static mesh instancing to reduce memory consumption in scenes like large forest. Might slow down small scenes. */
	bool bUseEmbreeInstancing;

	/** 
	 * Direct lighting, skylight radiosity and irradiance photons are cached on mapping surfaces to accelerate final gathering.
	 * This controls the downsample factor for that cache, relative to the mapping's lightmap resolution.
	 */
	int32 MappingSurfaceCacheDownsampleFactor;

	/** 
	 * Smoothness factor to apply to indirect lighting.  This is useful in some lighting conditions when Lightmass cannot resolve accurate indirect lighting.
	 * 1 is default smoothness tweaked for a variety of lighting situations.
	 * Higher values like 3 smooth out the indirect lighting more, but at the cost of indirect shadows losing detail as well.
	 */
	float IndirectLightingSmoothness;

	/** 
	 * Warning: Setting this higher than 1 will greatly increase build times!
	 * Can be used to increase the GI solver sample counts in order to get higher quality for levels that need it.
	 * It can be useful to reduce IndirectLightingSmoothness somewhat (~.75) when increasing quality to get defined indirect shadows.
	 * Note that this can't affect compression artifacts or other texture based artifacts.
	 */
	float IndirectLightingQuality;

	/** 
	 * Debugging - which single light bounce to view or -1 for all.  
	 * This setting has been carefully implemented to only affect the final color and not affect any other part of the solver (sample positions, ray directions, etc).
	 * Most of the debug visualizations are affected by ViewSingleBounceNumber.
	 */
	int32 ViewSingleBounceNumber;

	/** 
	 * Debugging - when enabled, multiple samples will be used to detect all the texels that are mapped to geometry. 
	 * Otherwise only the center and corner of each texel will be sampled. 
	 */
	bool bUseConservativeTexelRasterization;

	/** Debugging - whether to use the texel size in various calculations in an attempt to compensate for point sampling a texel. */
	bool bAccountForTexelSize;

	/** Debugging - whether to use the sample with the largest weight when rasterizing texels or a linear combination. */
	bool bUseMaxWeight;

	/** Maximum lighting samples per triangle for vertex lightmaps. */
	int32 MaxTriangleLightingSamples;

	/** Maximum samples for caching irradiance photons per triangle for vertex lightmaps. */
	int32 MaxTriangleIrradiancePhotonCacheSamples;

	/** Debugging - whether to color texels when invalid settings are detected. */
	bool bUseErrorColoring;

	/** Unmapped texel color */
	FLinearColor UnmappedTexelColor;
};

/** Scale dependent constants */
class FStaticLightingSceneConstants
{
public:
	/** 
	 * Scale of the level being lit. 
	 * Games using a different scale should use this to convert the defaults into the game-specific scale.
	 */
	float StaticLightingLevelScale;

	/** 
	 * World space distance to offset the origin of the ray, along the direction of the ray. 
	 * This is used to prevent incorrect self shadowing due to floating point precision.
	 */
	float VisibilityRayOffsetDistance;

	/** 
	 * World space distance to offset the origin of the ray along the direction of the normal.
	 * This is used to push triangle shaped self shadowing artifacts onto the backfaces of curved objects.
	 */
	float VisibilityNormalOffsetDistance;

	/** 
	 * Fraction of the sample radius to offset the origin of the ray along the sample normal. 
	 * This is applied instead of VisibilityNormalOffsetDistance whenever sample radius is known as it adapts to differently sized texels.
	 */
	float VisibilityNormalOffsetSampleRadiusScale;

	/** 
	 * Fraction of the sample radius to offset the origin of the ray in the tangent XY plane, based on the direction of the ray.
	 * This is only used when bAccountForTexelSize is true.
	 */
	float VisibilityTangentOffsetSampleRadiusScale;

	/** 
	 * Smallest texel radius allowed, useful for clamping edge cases where some texels have a radius of 0. 
	 * This should be smaller than the smallest valid texel radius in the scene.
	 */
	float SmallestTexelRadius;

	/** 
	 * Size of the grid that each light will use to cache information.  
	 * Larger grids take longer to precompute, but result in accelerated light sampling.
	 */
	int32 LightGridSize;
};

/** Settings for which material attribute to visualize */
enum EViewMaterialAttribute
{
	VMA_None,
	VMA_Emissive,
	VMA_Diffuse,
	VMA_Transmission,
	VMA_Normal
};

/** The scenes material settings */
struct FSceneMaterialSettings
{
	/** Debugging - Whether to use the debug material */
	bool bUseDebugMaterial;

	/** Debugging - Indicates which material attribute to visualize. */
	EViewMaterialAttribute ViewMaterialAttribute;

	/** The size of the emissive sample */
	int32 EmissiveSize;

	/** The size of the diffuse sample */
	int32 DiffuseSize;

	/** The size of the Transmission sample */
	int32 TransmissionSize;

	/** The size of the normal sample */
	int32 NormalSize;

	/** Whether to use the normal map for lighting, if false the smoothed vertex normal will be used. */
	bool bUseNormalMapsForLighting;

	/** 
	 * Debugging - Amount of incoming light to reflect diffusely (equally in all directions). 
	 * This is the diffuse term in the modified Phong BRDF, not the original.
	 */
	FLinearColor DebugDiffuse;

	/** Debugging - Emissive value assigned to secondary rays which miss all geometry in the scene. */
	FLinearColor EnvironmentColor;
};

/** Settings for meshes which emit light from their emissive areas. */
struct FMeshAreaLightSettings
{
	/** Whether to draw debug lines showing the corners of mesh area light primitives when a texel has been selected. */
	bool bVisualizeMeshAreaLightPrimitives;

	/** Emissive intensities must be larger than this to contribute toward scene lighting. */
	float EmissiveIntensityThreshold;

	/** 
	* Size of the grid that each mesh area light will use to cache information.  
	* Larger grids take longer to precompute, but result in accelerated light sampling.
	*/
	int32 MeshAreaLightGridSize;

	/** Cosine of the maximum angle allowed between mesh area light primitives that get merged into the same simplified primitive. */
	float MeshAreaLightSimplifyNormalCosAngleThreshold;

	/** Maximum distance allowed between any mesh area light primitive corners that get merged into the same simplified primitive. */
	float MeshAreaLightSimplifyCornerDistanceThreshold;

	/** Fraction of a mesh's bounds that an emissive texel can be from a simplified primitive and still get merged into that primitive. */
	float MeshAreaLightSimplifyMeshBoundingRadiusFractionThreshold;

	/** Distance along the average normal from the bounds origin of the mesh area light to place a light to handle influencing dynamic objects. */
	float MeshAreaLightGeneratedDynamicLightSurfaceOffset;
};

/** AO settings */
class FAmbientOcclusionSettings
{
public:
	/** 
	 * Whether to calculate ambient occlusion. 
	 * When enabled, some final gather rays will be traced even if only direct lighting is being calculated.
	 */
	bool bUseAmbientOcclusion;

	/** 
	 * Whether to generate textures storing the AO computed by Lightmass.
	 * These can be accessed through the PrecomputedAmbientOcclusion material node, 
	 * Which is useful for blending between material layers on environment assets.
	 */
	bool bGenerateAmbientOcclusionMaterialMask;

	/** Debugging - whether to only show the ambient occlusion term, useful for seeing the impact of AO settings in isolation. */
	bool bVisualizeAmbientOcclusion;

	/** 
	 * How much of the ambient occlusion term should be applied to direct lighting.
	 * A value of 0 will leave the direct lighting untouched, a value of 1 will apply all of the occlusion.
	 */
	float DirectIlluminationOcclusionFraction;
	
	/** Same as DirectIlluminationOcclusionFraction, but applied to indirect lighting. */
	float IndirectIlluminationOcclusionFraction;
	
	/** 
	 * Controls the ambient occlusion contrast.  
	 * Higher powers result in more contrast, which effectively shorten the occlusion gradient and push it into corners.
	 * An exponent of 1 gives equivalent occlusion as the indirect shadows from a constant environment color, with no other lights in the scene.
	 */
	float OcclusionExponent;
	
	/** 
	 * Fraction of the samples taken that have to be occluded before an occlusion value of 1 is reached for that texel. 
	 * A value of 1 means all the samples in the hemisphere have to be occluded, lower values darken areas that are not fully occluded.
	 */
	float FullyOccludedSamplesFraction;
	
	/** 
	 * Maximum distance over which an object can affect the occlusion of a texel.
	 * Lowering this cuts off lower frequencies in the ambient occlusion.
	 */
	float MaxOcclusionDistance;
};

/** Settings related to precomputations used by dynamic objects. */
class FDynamicObjectSettings
{
public:
	/** Debugging - Whether to draw points in the editor to visualize volume lighting samples. */
	bool bVisualizeVolumeLightSamples;

	/** Debugging - Whether to interpolate indirect lighting for surfaces from the precomputed light volume. */
	bool bVisualizeVolumeLightInterpolation;

	/** Scales the number of hemisphere samples that should be used for lighting volume samples. */
	float NumHemisphereSamplesScale;
	/** World space distance between samples placed on upward facing surfaces. */
	float SurfaceLightSampleSpacing;
	/** Height of the first sample layer above the surface. */
	float FirstSurfaceSampleLayerHeight;
	/** Height difference of successive layers. */
	float SurfaceSampleLayerHeightSpacing;
	/** Number of layers to place above surfaces. */
	int32 NumSurfaceSampleLayers;
	/** Distance between samples placed in a 3d uniform grid inside detail volumes. */ 
	float DetailVolumeSampleSpacing;
	/** Distance between samples placed in a 3d uniform grid inside the importance volume. */
	float VolumeLightSampleSpacing;
	/** Maximum samples placed in the 3d volume, used to limit memory usage. */
	int32 MaxVolumeSamples;
	/** Use Maximum number restriction for Surface Light Sample. */
	bool bUseMaxSurfaceSampleNum;
	/** Maximum samples placed in the surface light sample(only for Landscape currently), used to limit memory usage. */
	int32 MaxSurfaceLightSamples;
};

/** Settings for the volumetric lightmap. */
class FVolumetricLightmapSettings
{
public:

	/** Size of the top level grid covering the volumetric lightmap, in bricks. */
	FIntVector TopLevelGridSize;

	/** World space size of the volumetric lightmap. */
	FVector3f VolumeMin;
	FVector3f VolumeSize;

	/** 
	 * Size of a brick of unique lighting data.  Must be a power of 2.  
	 * Smaller values provide more granularity, but waste more memory due to padding.
	 */
	int32 BrickSize;

	/** Maximum number of times to subdivide bricks around geometry. */
	int32 MaxRefinementLevels;

	/** 
	 * Fraction of a cell's size to expand it by when voxelizing.  
	 * Larger values add more resolution around geometry, improving the lighting gradients but costing more memory.
	 */
	float VoxelizationCellExpansionForSurfaceGeometry;
	float VoxelizationCellExpansionForVolumeGeometry;
	float VoxelizationCellExpansionForLights;

	/** Bricks with RMSE below this value are culled. */
	float MinBrickError;

	/** Triangles with fewer lightmap texels than this don't cause refinement. */
	float SurfaceLightmapMinTexelsPerVoxelAxis;

	/** Whether to cull bricks entirely below landscape.  This can be an invalid optimization if the landscape has holes and caves that pass under landscape. */
	bool bCullBricksBelowLandscape;

	/** Subdivide bricks when a static point or spot light affects some part of the brick with brightness higher than this. */
	float LightBrightnessSubdivideThreshold;

	/** Maximum desired curvature in the lighting stored in Volumetric Lightmaps, used to reduce Spherical Harmonic ringing via a windowing filter. */
	float WindowingTargetLaplacian;
};

/** Settings for precomputed visibility. */
class FPrecomputedVisibilitySettings
{
public:

	/** Whether to export debug lines for visibility. */
	bool bVisualizePrecomputedVisibility;

	/** Whether to only place visibility cells on opaque surfaces. */
	bool bPlaceCellsOnOpaqueOnly;

	/** Whether to place visibility cells only along camera tracks or to also include shadow casting surfaces. */
	bool bPlaceCellsOnlyAlongCameraTracks;

	/** World space size of visibility cells in the x and y dimensions. */
	float CellSize;

	/** Number of tasks that visibility cells are being split up into. */
	int32 NumCellDistributionBuckets;

	/** World space size of visibility cells in the z dimension. */
	float PlayAreaHeight;

	/** Amount to increase the bounds of meshes when querying their visibility.  Larger scales reduce visibility errors at the cost of less effective culling. */
	float MeshBoundsScale;

	/** Minimum number of samples on the mesh for each cell - mesh query.  Small meshes use less samples. */
	int32 MinMeshSamples;

	/** Maximum number of samples on the mesh for each cell - mesh query.  Small meshes use less samples. */
	int32 MaxMeshSamples;

	/** Number of samples on each cell for each cell - mesh query. */
	int32 NumCellSamples;

	/** Number of samples to use when importance sampling each cell - mesh query. */
	int32 NumImportanceSamples;
};

/** Settings for volume distance field generation. */
class FVolumeDistanceFieldSettings
{
public:

	/** World space size of a voxel.  Smaller values use significantly more memory and processing time, but allow more detailed shadows. */
	float VoxelSize;

	/** 
	 * Maximum world space distance represented in the distance field, used for normalization. 
	 * Larger values increase build time and decrease distance field precision, but allow distance field traversal to skip larger areas. 
	 */
	float VolumeMaxDistance;

	 /** Number of distance traces for each voxel. */
	int32 NumVoxelDistanceSamples;

	/** Upper limit on the number of voxels that can be generated. */
	int32 MaxVoxels;
};

/** Shadow settings */
class FStaticShadowSettings
{
public:
	/** Debugging - Whether to filter a single shadow sample per texel in texture space instead of calculating area shadows. */
	bool bUseZeroAreaLightmapSpaceFilteredLights;

	/** Number of shadow rays to trace to each area light for each texel. */
	int32 NumShadowRays;

	/** Number of shadow rays to trace to each area light once a texel has been determined to lie in a shadow penumbra. */
	int32 NumPenumbraShadowRays;

	/** 
	 * Number of shadow rays to trace to each area light for bounced lighting. 
	 * This number is divided by bounce number for successive bounces.
	 */
	int32 NumBounceShadowRays;

	/**
	 * Settings for enabling and adjusting a filter pass on the computed shadow factor. The shadow factor is
	 * a value [0,1] that approximates the percentage of an area light visible at a texel. A value of 0 indicates
	 * that the texel is completely in shadow and 1 indicates that it is completely lit. The tolerance value is the
	 * maximum difference in the absolute shadow factor value allowed between any two adjacent texels before enabling
	 * filtering, meant to catch shadow transitions sharper than can be captured in the lightmap.
	 */
	bool bFilterShadowFactor;
	float ShadowFactorGradientTolerance;

	/** Whether to allow signed distance field shadows, or fall back on area shadows. */
	bool bAllowSignedDistanceFieldShadows;

	/** 
	 * Maximum world space distance stored from a texel to the shadow transition.
	 * Larger distances decrease precision but increase the maximum penumbra size that can be reconstructed from the distance field.
	 */
	float MaxTransitionDistanceWorldSpace;

	/** 
	 * The number of high resolution samples to calculate per MaxTransitionDistanceWorldSpace. 
	 * Higher values increase the distance field reconstruction quality, at the cost of longer build times.
	 */
	int32 ApproximateHighResTexelsPerMaxTransitionDistance;

	/** 
	 * The minimum upsample factor to calculate the high resolution samples at.  
	 * Larger values increase distance field reconstruction quality on small, high resolution meshes, at the cost of longer build times.
	 */
	int32 MinDistanceFieldUpsampleFactor;

	/** Distance in world space units between dominant light shadow map cells. */
	float StaticShadowDepthMapTransitionSampleDistanceX;
	float StaticShadowDepthMapTransitionSampleDistanceY;

	/** Amount to super sample dominant shadow map generation, in each dimension.  Larger factors increase build time but produce a more conservative shadow map. */
	int32 StaticShadowDepthMapSuperSampleFactor;

	/** Maximum number of dominant shadow samples to generate for one dominant light, used to limit memory used. */
	int32 StaticShadowDepthMapMaxSamples;

	/** Fraction of valid lighting samples (mapped texels or vertex samples) that must be unoccluded in a precomputed shadowmap for the shadowmap to be kept. */
	float MinUnoccludedFraction;
};

/** 
 * Settings related to solving the light transport equation by starting from the source of importance, 
 * as opposed to starting from the source of radiant power. 
 */
class FImportanceTracingSettings
{
public:

	/** 
	 * Debugging - whether to stratify hemisphere samples, which reduces variance. 
	 */
	bool bUseStratifiedSampling;

	/** 
	 * Number of hemisphere samples to evaluate from each irradiance cache sample when not using path tracing.
	 * When photon mapping is enabled, these are called final gather rays.
	 */
	int32 NumHemisphereSamples;

	/** Number of recursive levels allowed for adaptive refinement.  This has a huge impact on build time but also quality. */
	int32 NumAdaptiveRefinementLevels;

	/** 
	 * Largest angle from the sample normal that a hemisphere sample direction can be. 
	 * Useful for preventing rays nearly perpendicular to the normal which may self intersect a flat surface due to imprecision in the normal.
	 */
	float MaxHemisphereRayAngle;

	/** Starting threshold for what relative brightness difference causes a refinement.  At each depth the effective threshold will be reduced. */
	float AdaptiveBrightnessThreshold;

	/** Starting threshold for what angle around a first bounce photon causes a refinement on all cells affected.  At each depth the effective threshold will be reduced. */
	float AdaptiveFirstBouncePhotonConeAngle;

	float AdaptiveSkyVarianceThreshold;

	/** 
	 * Whether to use radiosity iterations for solving skylight 2nd bounce and up, plus emissive 1st bounce and up. 
	 * These light sources are not represented by photons so they need to be handled separately to have multiple bounces.
	 */
	bool bUseRadiositySolverForSkylightMultibounce;

	/** 
	 * Whether to cache final gather hit points for the radiosity algorithm, which reduces radiosity iteration time significantly but uses a lot of memory.
	 * Memory use is proportional to the lightmap texels in the scene and number of final gather rays.
	 */
	bool bCacheFinalGatherHitPointsForRadiosity;

	/**
	 * Whether to use radiosity iterations for point / spot / directional lights, instead of photons. 
	 * Has lower quality than photons in difficult indoor scenarios but useful as a reference.
	 */
	bool bUseRadiositySolverForLightMultibounce;
};

/** Settings controlling photon mapping behavior. */
class FPhotonMappingSettings
{
public:
	/** 
	 * Debugging - whether to use photon mapping.  
	 * Photon mapping benefits many parts of the solver so this is mainly for comparing against other methods.
	 */
	bool bUsePhotonMapping;

	/** 
	 * Debugging - whether to estimate the first bounce lighting by tracing rays from the sample point,
	 * Which is called final gathering, or by using the density of nearby first bounce photons.  
	 * Final gathering is slow but gets vastly better results than using first bounce photons.
	 */
	bool bUseFinalGathering;

	/** 
	 * Whether to use photons to represent direct lighting in final gathers.
	 * This is useful to visualize direct photons.  When false, explicitly sampled direct lighting will be used instead, which has less leaking.
	 */
	bool bUsePhotonDirectLightingInFinalGather;

	/** 
	* Debugging - whether to replace direct lighting with direct lighting as seen by the final gather.
	* This will either be direct lighting from photons, or approximate explicitly sampled direct lighting, depending on bUsePhotonDirectLightingInFinalGather.
	* This is mainly useful for debugging what the final gather rays see.
	*/
	bool bVisualizeCachedApproximateDirectLighting;

	/** Debugging - whether to use the optimization of caching irradiance calculations in deposited photons (called Irradiance photons). */
	bool bUseIrradiancePhotons;

	/** 
	* Debugging - whether to cache the result of the search for the nearest irradiance photon on surfaces. 
	* This results in a constant time lookup at the end of each final gather ray instead of a photon map search.
	* Only visible photons are cached, which reduces light leaking.
	*/
	bool bCacheIrradiancePhotonsOnSurfaces;

	/** 
	 * Debugging - whether to draw lines in the editor representing photon paths.  
	 * They will only be drawn if a texel is selected and ViewSingleBounceNumber is >= 0.
	 */
	bool bVisualizePhotonPaths;

	/** 
	 * Debugging - whether to draw the photons gathered for the selected texel or the photons gathered by final gather rays from the selected texel.
	 * This includes importance, irradiance and direct photons.
	 */
	bool bVisualizePhotonGathers;

	/** 
	 * Debugging - whether to draw lines in the editor showing where rays were traced due to importance photons from the selected texel.  
	 * If this is false, lines will be drawn for the rays traced with uniform hemisphere sampling instead.
	 * Note that when irradiance caching is enabled, rays will only be gathered if the selected texel created an irradiance cache sample.
	 */
	bool bVisualizePhotonImportanceSamples;

	/** 
	 * Debugging - whether to gather and draw photon map octree nodes that get traversed during the irradiance calculation for the irradiance photon nearest to the select position.
	 * Which photon map search the nodes come from depends on ViewSingleBounceNumber.
	 */
	bool bVisualizeIrradiancePhotonCalculation;

	/** 
	 * Debugging - whether to emit any photons outside the importance volume, if one exists.  
	 * If this is false, nothing outside the volume will bounce lighting and will be lit with direct lighting only.
	 */
	bool bEmitPhotonsOutsideImportanceVolume;

	/** Cone filter constant, which characterizes the falloff of the filter applied to photon density estimations. */
	float ConeFilterConstant;

	/** Number of photons to find in each photon map when calculating irradiance for an irradiance photon. */
	int32 NumIrradianceCalculationPhotons;

	/** 
	 * Fraction of NumHemisphereSamples to use for importance sampling instead of uniform sampling the final gather.
	 * If this fraction is close to 1, no uniform samples will be done and irradiance caching will be forced off as a result.
	 * If this is 0, only uniform samples will be taken.
	 */
	float FinalGatherImportanceSampleFraction;

	/** Cosine of the cone angle from an importance photon direction to generate ray directions for importance sampled final gathering. */
	float FinalGatherImportanceSampleCosConeAngle;

	/** World space radius of the disk around an indirect photon path in which indirect photons will be emitted from directional lights. */
	float IndirectPhotonEmitDiskRadius;

	/** Angle around an indirect photon path in which indirect photons will be emitted from point, spot and mesh area lights. */
	float IndirectPhotonEmitConeAngle;

	/** Maximum distance to search for importance photons. */
	float MaxImportancePhotonSearchDistance;

	/** 
	 * Distance to start searching for importance photons at. 
	 * For some scenes a small start distance (relative to MaxImportancePhotonSearchDistance) will speed up the search
	 * Since it can early out once enough photons are found.  For other scenes a small start distance will just cause redundant photon map searches.
	 */
	float MinImportancePhotonSearchDistance;

	/** Number of importance photons to find at each irradiance cache sample that will be used to guide the final gather. */
	int32 NumImportanceSearchPhotons;

	/** Scales the density at which to gather photons outside of the importance volume, if one exists. */
	float OutsideImportanceVolumeDensityScale;

	/** 
	 * Density of direct photons to emit per light, in number of photons per million surface area units.
	 */
	float DirectPhotonDensity;

	/** Density of direct photons which have irradiance cached at their position, in number of photons per million surface area units. */
	float DirectIrradiancePhotonDensity;

	/** Distance to use when searching for direct photons. */
	float DirectPhotonSearchDistance;

	/**
	 * Target density of indirect photon paths to gather, in number of paths per million surface area units. 
	 * Densities too small will result in indirect lighting not making it into small pockets.
	 */
	float IndirectPhotonPathDensity;

	/** 
	 * Density of indirect photons to emit, in number of photons per million surface area units.
	 * This should be high because first bounce photons are used to guide the final gather.
	 */
	float IndirectPhotonDensity;

	/** Density of indirect photons which have irradiance cached at their position, in number of photons per million surface area units. */
	float IndirectIrradiancePhotonDensity;

	/** Distance to use when searching for indirect photons. */
	float IndirectPhotonSearchDistance;

	/** Maximum cosine of the angle between the search normal and the surface normal of a candidate photon for that photon to be a valid search result. */
	float PhotonSearchAngleThreshold;

	/** Cosine of the angle from the search normal that defines a cone which irradiance photons must be outside of to be valid for that search. */
	float MinCosIrradiancePhotonSearchCone;

	/** 
	 * Whether to build a photon segment map, to guide importance sampling for volume queries.  
	 * Currently costs too much memory and queries are too slow to be a net positive.
	 */
	bool bUsePhotonSegmentsForVolumeLighting;

	/** Maximum world space length of segments that photons are split into for volumetric queries. */
	float PhotonSegmentMaxLength;

	/** Probability that a first bounce photon will be put into the photon segment map for volumetric queries. */
	float GeneratePhotonSegmentChance;
};

/** Settings controlling irradiance caching behavior. */
class FIrradianceCachingSettings
{
public:
	/** Debugging - whether to allow irradiance caching.  When this is disabled, indirect lighting will be many times slower. */
	bool bAllowIrradianceCaching;

	/** Debugging - whether to use irradiance gradients, which effectively allows higher order irradiance cache interpolation. */
	bool bUseIrradianceGradients;

	/** Debugging - whether to only show irradiance gradients. */
	bool bShowGradientsOnly;

	/** Debugging - whether to draw debug elements in the editor showing which irradiance cache samples were used to shade the selected texel. */
	bool bVisualizeIrradianceSamples;

	/** Scale applied to the radius of irradiance cache records.  This directly affects sample placement and therefore quality. */
	float RecordRadiusScale;

	/** Maximum angle between a record and the point being shaded allowed for the record to contribute. */
	float InterpolationMaxAngle;

	/** 
	 * Maximum angle from the plane defined by the average normal of a record and the point being shaded 
	 * That the vector from the point to the record can be for that record to contribute. 
	 */
	float PointBehindRecordMaxAngle;

	/** 
	 * How much to increase RecordRadiusScale for the shading pass. 
	 * This effectively filters the irradiance on flat surfaces.
	 */
	float DistanceSmoothFactor;

	/** 
	 * How much to increase InterpolationMaxAngle for the shading pass. 
	 * This effectively filters the irradiance on curved surfaces.
	 */
	float AngleSmoothFactor;

	/** 
	 * Scale applied to smoothness thresholds for sky occlusion.  
	 * This is useful because sky occlusion tends to be less noisy than GI, so less smoothing is needed to hide noise.
	 */
	float SkyOcclusionSmoothnessReduction;

	/** Largest radius an irradiance cache record can have. */
	float MaxRecordRadius;

	/** 
	 * Task size for parallelization of irradiance cache population within a mapping.  A mapping will be split into pieces of this size which allows other threads to help.
	 * Smaller settings result in more redundant irradiance cache samples, but allow better parallelization.
	 * Setting this to a large value like 4096 will effectively make the irradiance caching single threaded for a given mapping, which is useful for debugging.
	 */
	int32 CacheTaskSize;

	/** 
	 * Task size for parallelization of irradiance cache interpolation within a mapping.  A mapping will be split into pieces of this size which allows other threads to help.
	 * Smaller settings allow better parallelization.
	 * Setting this to a large value like 4096 will effectively make the irradiance cache interpolation single threaded for a given mapping, which is useful for debugging.
	 */
	int32 InterpolateTaskSize;
};

struct FDebugLightingInputData
{
	/** Whether the solver should send stats back to Unreal */
	bool bRelaySolverStats;
	/** Guid of the mapping to debug */
	FGuid MappingGuid;
	/** Index of the BSP node to debug if the mapping is a BSP mapping */
	int32 NodeIndex;
	/** World space position of the position that was clicked to select the debug sample */
	FVector4f Position;
	/** Position in the texture mapping of the texel to debug */
	int32 LocalX;
	int32 LocalY;
	/** Size of the texture mapping that was selected to debug */
	int32 MappingSizeX;
	int32 MappingSizeY;
	/** Position of the camera */
	FVector4f CameraPosition;
	/** VisibilityId of a component from the currently selected actor or BSP surface. */
	int32 DebugVisibilityId;
};

//----------------------------------------------------------------------------
//	Scene export file header
//----------------------------------------------------------------------------
struct FSceneFileHeader
{
	/** FourCC cookie: 'SCEN' */
	uint32		Cookie;
	FGuid		FormatVersion;
	FGuid		Guid;

	/** Settings for the GI solver */
	FStaticLightingSettings			GeneralSettings;
	FStaticLightingSceneConstants	SceneConstants;
	FSceneMaterialSettings			MaterialSettings;
	FMeshAreaLightSettings			MeshAreaLightSettings;
	FAmbientOcclusionSettings		AmbientOcclusionSettings;
	FDynamicObjectSettings			DynamicObjectSettings;
	FVolumetricLightmapSettings		VolumetricLightmapSettings;
	FPrecomputedVisibilitySettings	PrecomputedVisibilitySettings;
	FVolumeDistanceFieldSettings	VolumeDistanceFieldSettings;
	FStaticShadowSettings			ShadowSettings;
	FImportanceTracingSettings		ImportanceTracingSettings;
	FPhotonMappingSettings			PhotonMappingSettings;
	FIrradianceCachingSettings		IrradianceCachingSettings;
	
	FDebugLightingInputData			DebugInput;

	FSceneFileHeader() {}

	/** Copy ctor that doesn't modify padding in FSceneFileHeader. */
	FSceneFileHeader(const FSceneFileHeader& Other);

	/** If true, pad the mappings (shrink the requested size and then pad) */
	uint32	bPadMappings:1;
	/** If true, draw a solid border as the padding around mappings */
	uint32	bDebugPadding:1;
	/** If true, only calculate lighting on debugged texel's mappings */
	uint32	bOnlyCalcDebugTexelMappings:1;
	/** If true, color lightmaps based on execution time (brighter red = slower) */
	uint32	bColorByExecutionTime:1;
	/** If true, color lightmaps a random color */
	uint32	bUseRandomColors:1;
	/** If true, a green border will be placed around the edges of mappings */
	uint32	bColorBordersGreen:1;

	/** Amount of time to color full red (Color.R = Time / ExecutionTimeDivisor) */
	float		ExecutionTimeDivisor;

	int32		NumImportanceVolumes;
	int32		NumCharacterIndirectDetailVolumes;
	int32		NumVolumetricLightmapDensityVolumes;
	int32		NumPortals;
	int32		NumDirectionalLights;
	int32		NumPointLights;
	int32		NumSpotLights;
	int32		NumRectLights;
	int32		NumSkyLights;
	int32		NumStaticMeshes;
	int32		NumStaticMeshInstances;
	int32		NumFluidSurfaceInstances;
	int32		NumLandscapeInstances;
	int32		NumBSPMappings;
	int32		NumStaticMeshTextureMappings;
	int32		NumFluidSurfaceTextureMappings;
	int32		NumLandscapeTextureMappings;
	int32		NumSpeedTreeMappings;
	int32		NumVolumeMappings;
	int32		NumLandscapeVolumeMappings;
	int32		NumPrecomputedVisibilityBuckets;
	int32		NumVolumetricLightmapTasks;
};

//----------------------------------------------------------------------------
//	Base light struct
//----------------------------------------------------------------------------
enum EDawnLightFlags
{
	// maps to ULightComponent::CastShadows
	GI_LIGHT_CASTSHADOWS					= 0x00000001,
	// maps to ULightComponent::HasStaticLighting()
	GI_LIGHT_HASSTATICLIGHTING				= 0x00000002,
	// maps to ULightComponent::HasStaticShadowing()
	GI_LIGHT_HASSTATICSHADOWING				= 0x00000008,
	// maps to ULightComponent::CastStaticShadows
	GI_LIGHT_CASTSTATICSHADOWS				= 0x00000010,
	GI_LIGHT_STORE_SEPARATE_SHADOW_FACTOR	= 0x00000020,
	GI_LIGHT_INVERSE_SQUARED				= 0x00000080,
	GI_LIGHT_USE_LIGHTPROFILE				= 0x00000100,
	// Whether a stationary light should generate a standard shadowmap (area shadows) or a distance field shadow map
	GI_LIGHT_USE_AREA_SHADOWS_FOR_SEPARATE_SHADOW_FACTOR		= 0x00000200
};

struct FLightData
{
	static const int32 LightProfileTextureDataSize = 256 * 256;

	FGuid			Guid;
	/** Bit-wise combination of flags from EDawnLightFlags */
	uint32			LightFlags;
	/** Homogeneous coordinates */
	FVector4f		Position;
	FVector4f		Direction;
	FLinearColor	Color;
	float			Brightness;
	/** The radius of the light's surface, not the light's influence. */
	float			LightSourceRadius;
	/** The length of the light source*/
	float			LightSourceLength;
	/** Scale factor for the indirect lighting */
	float			IndirectLightingScale;
	/** 0 will be completely desaturated, 1 will be unchanged, 2 will be completely saturated */
	float			IndirectLightingSaturation;
	/** Controls the falloff of shadow penumbras */
	float			ShadowExponent;
	/** Scales resolution of the static shadowmap for this light. */
	float			ShadowResolutionScale;

	// @param WorldPosition		position of the point to light in the world
	// @param LightPosition		position of the light in the world
	// @param LightDirection	direction at which the light is emitting (x axis)
	// @param LightTangent		tangent to the direction at which the light is emitting (z axis)
	inline float ComputeLightProfileMultiplier(const TArray< uint8 >& LightProfileTextureData, FVector3f WorldPosition, FVector3f LightPosition, FVector3f LightDirection, FVector3f LightTangent) const
	{
		// optimization - only evaluate this function if needed
		if(LightFlags & Lightmass::GI_LIGHT_USE_LIGHTPROFILE)
		{
			FVector3f LightBitangent = FVector3f::CrossProduct( LightTangent, LightDirection ).GetSafeNormal();

			FMatrix44f LightTransform = FMatrix44f( LightDirection, LightBitangent, LightTangent, FVector4f(0.f, 0.f, 0.f, 1.f) );
			FMatrix44f InvLightTransform = LightTransform.GetTransposed();

			FVector3f ToLight = (LightPosition - WorldPosition).GetSafeNormal();
			FVector3f LocalToLight = InvLightTransform.TransformVector( ToLight );

			// -1..1
			float DotProd = FVector3f::DotProduct(ToLight, LightDirection);
			// -PI..PI (this distortion could be put into the texture but not without quality loss or more memory)
			float Angle = FMath::Asin(DotProd);
			// 0..1
			float NormAngle = Angle / PI + 0.5f;

			float TangentAngle = FMath::Atan2( -LocalToLight.Z, -LocalToLight.Y ); // -Y represents 0/360 horizontal angle and we're rotating counter-clockwise
			float NormTangentAngle = TangentAngle / (PI * 2.f) + 0.5f;

			return FilterLightProfile( LightProfileTextureData, NormAngle, NormTangentAngle );
		}

		return 1.0f;
	}
	
private:
	// @param X clamped in range 0..1
	// @return 0..1
	inline float FilterLightProfile(const TArray< uint8 >& LightProfileTextureData, const float X, const float Y) const
	{
		const uint32 SizeX = FMath::Sqrt( static_cast<float>(LightProfileTextureDataSize) );
		const uint32 SizeY = LightProfileTextureDataSize / SizeX;

		// can be optimized

		// not 100% like GPU hardware but simple and almost the same
		float UnNormalizedX = FMath::Clamp(X * SizeX, 0.0f, (float)(SizeX - 1));
		float UnNormalizedY = FMath::Clamp(Y * SizeY, 0.0f, (float)(SizeY - 1));

		uint32 X0 = (uint32)UnNormalizedX;
		uint32 X1 = FMath::Min(X0 + 1, SizeX - 1);

		uint32 Y0 = (uint32)UnNormalizedY;
		uint32 Y1 = FMath::Min(Y0 + 1, SizeY - 1);

		float XFraction = UnNormalizedX - X0;
		float YFraction = UnNormalizedY - Y0;

		float V00 = LightProfileTextureData[Y0 * SizeX + X0] / 255.0f;
		float V10 = LightProfileTextureData[Y1 * SizeX + X0] / 255.0f;
		float V01 = LightProfileTextureData[Y0 * SizeX + X1] / 255.0f;
		float V11 = LightProfileTextureData[Y1 * SizeX + X1] / 255.0f;

		float V0 = FMath::Lerp(V00, V10, YFraction);
		float V1 = FMath::Lerp(V01, V11, YFraction);

		return FMath::Lerp(V0, V1, XFraction);
	}
};

//----------------------------------------------------------------------------
//	Direction light, extending FLightData
//----------------------------------------------------------------------------
struct FDirectionalLightData
{
	/** Angle that the directional light's emissive surface extends from any receiver position, in radians. */
	float		LightSourceAngle;
};

//----------------------------------------------------------------------------
//	Point light, extending FLightData
//----------------------------------------------------------------------------
struct FPointLightData
{
	float		Radius;
	float		FalloffExponent;
	// Point lights need an additional axis to specify the direction of IES profiles, also used by spot lights for the direction of tube lights
	FVector3f		LightTangent;
};

//----------------------------------------------------------------------------
//	Spot light, extending FPointLightData
//----------------------------------------------------------------------------
struct FSpotLightData
{
	/** Unclamped, in degrees */
	float		InnerConeAngle;
	/** Unclamped, in degrees */
	float		OuterConeAngle;
};

//----------------------------------------------------------------------------
//	Sky light, extending FLightData
//----------------------------------------------------------------------------
struct FSkyLightData
{
	/** 
	 * Whether to use a filtered cubemap matching the Skylight Component's CubemapResolution to represent the skylight, or a 3rd order Spherical Harmonic. 
	 * The filtered cubemap is much more accurate than 3rd order SH, especially in mostly shadowed areas.
	 */ 
	bool bUseFilteredCubemap;
	int32 RadianceEnvironmentMapDataSize;
	FSHVectorRGB3 IrradianceEnvironmentMap;
};

struct FRectLightData
{
	uint32			SourceTextureSizeX;
	uint32			SourceTextureSizeY;
	FLinearColor	SourceTextureAvgColor;
};

//----------------------------------------------------------------------------
//	Material
//----------------------------------------------------------------------------

struct FMaterialElementData
{
	/** Used to find Material on import */
	FSHAHash MaterialHash;
	/** If true, this object will be lit as if it receives light from both sides of its polygons. */
	uint32 bUseTwoSidedLighting:1;
	/** If true, this material element will only shadow indirect lighting.  					*/
	uint32 bShadowIndirectOnly:1;
	/** If true, allow using the emissive for static lighting.						*/
	uint32 bUseEmissiveForStaticLighting:1;
	/** 
	 * Typically the triangle normal is used for hemisphere gathering which prevents incorrect self-shadowing from artist-tweaked vertex normals. 
	 * However in the case of foliage whose vertex normal has been setup to match the underlying terrain, gathering in the direction of the vertex normal is desired.
	 */
	uint32 bUseVertexNormalForHemisphereGather:1;
	/** Direct lighting falloff exponent for mesh area lights created from emissive areas on this primitive. */
	float EmissiveLightFalloffExponent;
	/** 
	 * Direct lighting influence radius.  
	 * The default is 0, which means the influence radius should be automatically generated based on the emissive light brightness.
	 * Values greater than 0 override the automatic method.
	 */
	float EmissiveLightExplicitInfluenceRadius;
	/** Scales the emissive contribution of this material. */
	float EmissiveBoost;
	/** Scales the diffuse contribution of this material. */
	float DiffuseBoost;
	/** 
	 * Fraction of the samples taken that have to be occluded before an occlusion value of 1 is reached for that texel. 
	 * A value of 1 means all the samples in the hemisphere have to be occluded, lower values darken areas that are not fully occluded.
	 */
	float FullyOccludedSamplesFraction;

	FMaterialElementData() :
		  bUseTwoSidedLighting(false)
		, bShadowIndirectOnly(false)
		, bUseEmissiveForStaticLighting(true)
		, bUseVertexNormalForHemisphereGather(false)
	    , EmissiveLightFalloffExponent(2.0f)
		, EmissiveLightExplicitInfluenceRadius(0.0f)
		, EmissiveBoost(1.0f)
		, DiffuseBoost(1.0f)
		, FullyOccludedSamplesFraction(1.0f)
	{
	}
};

enum EMeshInstanceLightingFlags
{
	/** Whether the mesh casts a shadow. */
	GI_INSTANCE_CASTSHADOW			= 1<<0,
	/** Whether the mesh uses a two-sided material. */
	GI_INSTANCE_TWOSIDED			= 1<<1,
	/** Whether the mesh only casts a shadow on itself. */
	GI_INSTANCE_SELFSHADOWONLY		= 1<<2,
	/** Whether to disable casting a shadow on itself. */
	GI_INSTANCE_SELFSHADOWDISABLE	= 1<<3
};

struct FStaticLightingMeshInstanceData
{
	FGuid Guid;
	/** The number of triangles in the mesh, used for visibility testing. */
	int32 NumTriangles;
	/** The number of shading triangles in the mesh. */
	int32 NumShadingTriangles;
	/** The number of vertices in the mesh, used for visibility testing. */
	int32 NumVertices;
	/** The number of shading vertices in the mesh. */
	int32 NumShadingVertices;
	/** The texture coordinate index which is used to parametrize materials. */
	int32 TextureCoordinateIndex;
	int32 MeshIndex;
	FGuid LevelGuid;
	uint32 LightingFlags;		// EMeshInstanceLightingFlags
	bool bCastShadowAsTwoSided;
	/** Whether the mesh can be moved in game or not. */
	bool bMovable;
	/** The lights which affect the mesh's primitive. */
	int32 NumRelevantLights;
	/** The bounding box of the mesh. */
	FBox3f BoundingBox;
};

namespace ESplineMeshAxis
{
	enum Type
	{
		X,
		Y,
		Z,
	};
}

/** 
 * Parameters used to transform a static mesh based on a spline.
 * Be sure to update this structure if spline functionality changes in Unreal!
 */
struct FSplineMeshParams
{
	/** Start location of spline, in component space */
	FVector3f StartPos;
	/** Start tangent of spline, in component space */
	FVector3f StartTangent;
	/** X and Y scale applied to mesh at start of spline */
	FVector2f StartScale;
	/** Roll around spline applied at start */
	float StartRoll;
	/** Offset from the spline at start */
	FVector2f StartOffset;
	/** End location of spline, in component space */
	FVector3f EndPos;
	/** End tangent of spline, in component space */
	FVector3f EndTangent;
	/** X and Y scale applied to mesh at end of spline */
	FVector2f EndScale;
	/** Roll around spline applied at end */
	float EndRoll;
	/** Offset from the base spline at end */
	FVector2f EndOffset;
	/** Axis (in component space) that is used to determine X axis for co-ordinates along spline */
	FVector3f SplineUpDir;
	/** Smoothly (cubic) interpolate the Roll and Scale params over spline. */
	bool bSmoothInterpRollScale;
	/** Minimum Z value of the entire mesh */
	float MeshMinZ;
	/** Range of Z values over entire mesh */
	float MeshRangeZ;
	/** Chooses the forward axis for the spline mesh orientation */
	TEnumAsByte<ESplineMeshAxis::Type> ForwardAxis;
};

struct FStaticMeshStaticLightingMeshData
{
	/** The LOD this mesh represents. */
	uint32 EncodedLODIndices;
	uint32 EncodedHLODRange;
	FMatrix44f LocalToWorld;
	/** true if the primitive has a transform which reverses the winding of its triangles. */
	bool bReverseWinding;
	bool bShouldSelfShadow;
	FGuid StaticMeshGuid;
	bool bIsSplineMesh;
	FSplineMeshParams SplineParameters;
};

struct FMinimalStaticLightingVertex
{
	FVector4f WorldPosition;
	FVector4f WorldTangentZ;
	FVector2f TextureCoordinates[MAX_TEXCOORDS];
};

struct FStaticLightingVertexData : public FMinimalStaticLightingVertex
{
	FVector4f WorldTangentX;
	FVector4f WorldTangentY;
};

struct FBSPSurfaceStaticLightingData
{
	FVector4f	TangentX;
	FVector4f	TangentY;
	FVector4f	TangentZ;

	FMatrix44f		MapToWorld;
	FMatrix44f		WorldToMap;

	FGuid		MaterialGuid;
};

struct FStaticLightingMappingData
{
	FGuid Guid;
	FGuid StaticLightingMeshInstance;
};

struct FStaticLightingTextureMappingData
{
	/** The width of the static lighting textures used by the mapping. */
	int32 SizeX;
	/** The height of the static lighting textures used by the mapping. */
	int32 SizeY;
	/** The lightmap texture coordinate index which is used for the mapping. */
	int32 LightmapTextureCoordinateIndex;
	/** Whether to apply a bilinear filter to the sample or not. */
	bool bBilinearFilter;
};

struct FStaticLightingVertexMappingData
{
	/** Lighting will be sampled at a random number of samples/surface area proportional to this factor. */
	float SampleToAreaRatio;
	/** true to sample at vertices instead of on the surfaces. */
	bool bSampleVertices;
};


//
//	Fluid surfaces
//
struct FFluidSurfaceStaticLightingMeshData
{
	/** The primitive's local to world transform. */
	FMatrix44f LocalToWorld;
	/** The inverse transpose of the primitive's local to world transform. */
	FMatrix44f LocalToWorldInverseTranspose;
	/** The mesh data of the fluid surface, which is represented as a quad. */
	FVector4f QuadCorners[4];
	FVector4f QuadUVCorners[4];
	int32 QuadIndices[6];
};

//
// Landscape
//
struct FLandscapeStaticLightingMeshData
{
	/** The primitive's local to world transform. */
	FMatrix44f LocalToWorld;
	int32 ComponentSizeQuads;
	float LightMapRatio;
	/** The number of quads we are expanding to eliminate seams. */
	int32 ExpandQuadsX;
	int32 ExpandQuadsY;
};

struct FVolumetricLightmapDensityVolumeData
{
	FBox3f Bounds;
	FIntPoint AllowedMipLevelRange;
	int32 NumPlanes;
};

#if !PLATFORM_MAC && !PLATFORM_LINUX
#pragma pack(pop)
#endif

}	// namespace Lightmass

