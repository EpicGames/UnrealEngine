// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class UnrealLightmass : ModuleRules
{
	public UnrealLightmass(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(new string[] { "Core", "CoreUObject", "zlib", "SwarmInterface", "Projects", "ApplicationCore" });

		PublicDefinitions.Add("UE_LIGHTMASS=1");

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PublicDependencyModuleNames.Add("IntelTBB");
			PublicDependencyModuleNames.Add("Embree");

			// Unreallightmass requires GetProcessMemoryInfo exported by psapi.dll. http://msdn.microsoft.com/en-us/library/windows/desktop/ms683219(v=vs.85).aspx
			PublicSystemLibraries.Add("psapi.lib");
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Messaging",
				}
			);
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Linux)
		{
			PublicDependencyModuleNames.Add("IntelTBB");
			PublicDependencyModuleNames.Add("Embree");

			// On Mac/Linux UnrealLightmass is executed locally and communicates with the editor using Messaging module instead of SwarmAgent
			// @todo: allow for better plug-in support in standalone Slate apps
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Networking",
					"Sockets",
					"Messaging",
					"UdpMessaging",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Messaging",
				}
			);
		}
		else
		{
			PublicDefinitions.Add("USE_EMBREE=0");
		}
		
		// Lightmass ray tracing is 8% faster with buffer security checks disabled due to fixed size arrays on the stack in the kDOP ray tracing functions
		// Warning: This means buffer overwrites will not be detected
		bEnableBufferSecurityChecks = false;

		PrivateIncludePaths.Add("Programs/UnrealLightmass/Private/Launch");
		PrivateIncludePaths.Add("Programs/UnrealLightmass/Private/ImportExport");
		PrivateIncludePaths.Add("Programs/UnrealLightmass/Private/CPUSolver");
		PrivateIncludePaths.Add("Programs/UnrealLightmass/Private/Lighting");
		PrivateIncludePaths.Add("Programs/UnrealLightmass/Private/LightmassCore");
		PrivateIncludePaths.Add("Programs/UnrealLightmass/Private/LightmassCore/Misc");
		PrivateIncludePaths.Add("Programs/UnrealLightmass/Private/LightmassCore/Math");
		PrivateIncludePaths.Add("Programs/UnrealLightmass/Private/LightmassCore/Templates");
    }
}
