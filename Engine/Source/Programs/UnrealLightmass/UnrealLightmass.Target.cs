// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
[SupportedConfigurations(UnrealTargetConfiguration.Debug, UnrealTargetConfiguration.Development)]
public class UnrealLightmassTarget : TargetRules
{
	public UnrealLightmassTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Modular;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		AdditionalPlugins.Add("UdpMessaging");
		LaunchModuleName = "UnrealLightmass";

		// Lean and mean
		bBuildDeveloperTools = false;

		// Editor-only data, however, is needed
		bBuildWithEditorOnlyData = true;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		//bCompileAgainstCoreUObject = false;

		if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Linux)
		{
			// On Mac/Linux UnrealLightmass is executed locally and communicates with the editor using Messaging module instead of SwarmAgent
			// Plugins and developer tools are needed for that
			bCompileWithPluginSupport = true;
			bBuildDeveloperTools = true;
		}

		// This app is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;

		// Disable logging, lightmass will create its own unique logging file
		GlobalDefinitions.Add("ALLOW_LOG_FILE=0");
	}
}
